// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

[SupportedPlatformGroups("Microsoft")]
public class D3D12RHI : ModuleRules
{
	public D3D12RHI(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddAll(
			"Shaders"
		);

		PrivateDependencyModuleNames.AddAll(
			"CoreUObject",
			"Engine",
			"RHICore",
			"RenderCore",
			"TraceLog"
		);
		
		PublicIncludePathModuleNames.AddAll(
			"DX12"
		);

		PublicDependencyModuleNames.AddAll(
			"Core",
			"RHI"
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "AMD_AGS");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "IntelExtensionsFramework");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAftermath");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAPI");

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDependencyModuleNames.Add("WindowsD3D");
			PrivateDependencyModuleNames.Add("HeadMountedDisplay");

			PublicDefinitions.Add("D3D12RHI_PLATFORM_HAS_CUSTOM_INTERFACE=0");

			if (Target.WindowsPlatform.bPixProfilingEnabled &&
				(Target.Configuration != UnrealTargetConfiguration.Shipping || Target.bAllowProfileGPUInShipping) &&
				(Target.Configuration != UnrealTargetConfiguration.Test || Target.bAllowProfileGPUInTest))
			{
				PublicDefinitions.Add("PROFILE");
				PublicDependencyModuleNames.Add("WinPixEventRuntime");
			}
		}
	}
}
