// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RayTracingResources.h: Internal D3D12 RHI RayTracing definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "RayTracingBuiltInResources.h"

using FD3D12_GPU_VIRTUAL_ADDRESS = uint64;

// Built-in local root parameters that are always bound to all hit shaders
// Contains union for bindless and non-bindless index/vertex buffer data to make the code handling the hit group parameters easier to use
// (otherwise all cached hit parameter code has to be done twice and stored twice making everything more complicated)
// Ideally the non bindless code path should be removed 'soon' - this constant buffer size for FD3D12HitGroupSystemParameters in Bindless is 8 bytes bigger than needed
struct FD3D12HitGroupSystemParameters
{	
	FHitGroupSystemRootConstants RootConstants;
	union
	{
		struct
		{
			uint32 BindlessHitGroupSystemIndexBuffer;
			uint32 BindlessHitGroupSystemVertexBuffer;
		};
		struct
		{
			FD3D12_GPU_VIRTUAL_ADDRESS IndexBuffer;
			FD3D12_GPU_VIRTUAL_ADDRESS VertexBuffer;
		};
	};
};

struct FD3D12RayTracingOfflineBvhHeader
{
	uint32 Size = 0;
	uint32 SerializedSize = 0;
};
