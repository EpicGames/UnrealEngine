// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "D3D12ThirdParty.h"

/**
 * Build a serialized root signature for a compute shader from its UE shader blob.
 * Uses the same root signature construction logic as RHICreateComputeShader.
 *
 * @param ShaderCode            Full UE shader blob (SRT + native bytecode + optional data)
 * @param ResourceBindingTier   Resource binding tier to use for the root signature layout
 * @param OutRootSigBlob        Receives the serialized root signature blob
 * @param OutNativeBytecodeOffset  Receives the offset to native DXBC/DXIL bytecode within ShaderCode
 * @return true on success
 */
D3D12RHI_API bool BuildComputeShaderRootSignature(
	const TConstArrayView<uint8>& ShaderCode,
	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier,
	TRefCountPtr<ID3DBlob>& OutRootSigBlob,
	int32& OutNativeBytecodeOffset);

/**
 * Build a serialized root signature for a graphics PSO from multiple UE shader blobs.
 * Merges FShaderCodePackedResourceCounts across all active shader stages to produce
 * a single root signature that covers the union of resources required by all stages.
 *
 * @param ShaderBlobs           Array of full UE shader blobs, one per active stage (SRT + native bytecode + optional data).
 *                              Null/empty entries are skipped.
 * @param ResourceBindingTier   Resource binding tier to use for the root signature layout
 * @param OutRootSigBlob        Receives the serialized root signature blob
 * @return true on success
 */
D3D12RHI_API bool BuildGraphicsShaderRootSignature(
	const TConstArrayView<TConstArrayView<uint8>> ShaderBlobs,
	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier,
	TRefCountPtr<ID3DBlob>& OutRootSigBlob);
