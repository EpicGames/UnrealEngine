// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

#include "RHIAccess.h"

#include <type_traits>

using D3D12AccessUnderlyingType = std::underlying_type_t<ERHIAccess>;

enum class ED3D12Access : D3D12AccessUnderlyingType
{
	Unknown             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::Unknown),
	CPURead             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::CPURead),
	Present             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::Present),
	IndirectArgs        = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::IndirectArgs),
	VertexOrIndexBuffer = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::VertexOrIndexBuffer),
	SRVCompute          = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::SRVCompute),
	SRVGraphicsPixel    = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::SRVGraphicsPixel),
	SRVGraphicsNonPixel = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::SRVGraphicsNonPixel),
	CopySrc             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::CopySrc),
	ResolveSrc          = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ResolveSrc),
	DSVRead             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::DSVRead),
	UAVCompute          = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::UAVCompute),
	UAVGraphics         = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::UAVGraphics),
	RTV                 = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::RTV),
	CopyDest            = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::CopyDest),
	ResolveDst          = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ResolveDst),
	DSVWrite            = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::DSVWrite),
	BVHRead             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::BVHRead),
	BVHWrite            = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::BVHWrite),
	Discard             = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::Discard),
	ShadingRateSource   = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ShadingRateSource),
	RHIAccessLast       = ShadingRateSource,

	// D3D12 specific extensions
	Common              = RHIAccessLast << 1,
	GenericRead         = RHIAccessLast << 2,
	Last                = GenericRead,

	Mask = (Last << 1) - 1,

	SRVGraphics                  = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::SRVGraphics),
	SRVMask                      = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::SRVMask),
	UAVMask                      = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::UAVMask),
	ReadOnlyExclusiveMask        = (static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ReadOnlyExclusiveMask) | GenericRead) & ~BVHRead,
	ReadOnlyExclusiveComputeMask = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ReadOnlyExclusiveComputeMask) | GenericRead,
	ReadOnlyMask                 = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ReadOnlyMask),
	ReadableMask                 = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::ReadableMask) | BVHRead,
	WriteOnlyExclusiveMask       = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::WriteOnlyExclusiveMask),
	WriteOnlyMask                = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::WriteOnlyMask),
	WritableMask                 = static_cast<D3D12AccessUnderlyingType>(ERHIAccess::WritableMask),
};

static_assert(
	static_cast<D3D12AccessUnderlyingType>(ED3D12Access::RHIAccessLast)
	== static_cast<D3D12AccessUnderlyingType>(ERHIAccess::Last));

ENUM_CLASS_FLAGS(ED3D12Access);

constexpr ED3D12Access ConvertToD3D12Access(ERHIAccess InRHIAccess)
{
	return static_cast<ED3D12Access>((static_cast<D3D12AccessUnderlyingType>(InRHIAccess)));
}

constexpr bool IsInvalidAccess(ED3D12Access InD3D12Access)
{
	return
		((EnumHasAnyFlags(InD3D12Access, ED3D12Access::ReadOnlyExclusiveMask) && EnumHasAnyFlags(InD3D12Access, ED3D12Access::WritableMask))
		|| (EnumHasAnyFlags(InD3D12Access, ED3D12Access::WriteOnlyExclusiveMask) && EnumHasAnyFlags(InD3D12Access, ED3D12Access::ReadableMask))
		// Common should always be by itself
		|| (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Common) && !EnumHasOneFlag(InD3D12Access)));
}

constexpr bool IsValidAccess(ED3D12Access InD3D12Access)
{
	return !IsInvalidAccess(InD3D12Access);
}