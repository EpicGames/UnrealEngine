// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Adapter.cpp:D3D12 Adapter implementation.
=============================================================================*/

#include "D3D12Adapter.h"
#include "D3D12RHIPrivate.h"
#include "D3D12AmdExtensions.h"
#include "D3D12IntelExtensions.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/EngineVersion.h"
#include "Misc/OutputDeviceRedirector.h"
#include "RenderingThread.h"
#include "ShaderDiagnostics.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#if PLATFORM_WINDOWS
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/WindowsPlatformStackWalk.h"
#include "Windows/WindowsPlatformCrashContext.h"
#endif
#include "Modules/ModuleManager.h"
#include "Windows/HideWindowsPlatformTypes.h"


#if ENABLE_RESIDENCY_MANAGEMENT
bool GEnableResidencyManagement = true;
static TAutoConsoleVariable<int32> CVarResidencyManagement(
	TEXT("D3D12.ResidencyManagement"),
	1,
	TEXT("Controls whether D3D12 resource residency management is active (default = on)."),
	ECVF_ReadOnly
);

bool GD3D12StartResourceResident = false;
static FAutoConsoleVariableRef CVarResourcesStartResident(
	TEXT("D3D12.ResourcesStartResident"),
	GD3D12StartResourceResident,
	TEXT("When disabled (default): all D3D12 resources start EVICTED. GPU Residency will be managed during command list submission.")
	TEXT("When enabled: all D3D12 resources are RESIDENT on the GPU at the time of creation."),
	ECVF_ReadOnly
);
#endif // ENABLE_RESIDENCY_MANAGEMENT

#if TRACK_RESOURCE_ALLOCATIONS
int32 GTrackedReleasedAllocationFrameRetention = 100;
static FAutoConsoleVariableRef CTrackedReleasedAllocationFrameRetention(
	TEXT("D3D12.TrackedReleasedAllocationFrameRetention"),
	GTrackedReleasedAllocationFrameRetention,
	TEXT("Amount of frames for which we keep freed allocation data around when resource tracking is enabled"),
	ECVF_RenderThreadSafe
);
#endif

int32 GAllowAsyncCompute = 1;
static FAutoConsoleVariableRef CVarAllowAsyncCompute(
	TEXT("r.D3D12.AllowAsyncCompute"),
	GAllowAsyncCompute,
	TEXT("Allow usage of async compute"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> GD3D12DebugCvar (
	TEXT("r.D3D12.EnableD3DDebug"),
	0,
	TEXT("0 to disable d3ddebug layer (default)\n")
	TEXT("1 to enable error logging (-d3ddebug) \n")
	TEXT("2 to enable error & warning logging (-d3dlogwarnings)\n")
	TEXT("3 to enable breaking on errors & warnings (-d3dbreakonwarning)\n")
	TEXT("4 to enable CONTINUING on errors (-d3dcontinueonerrors)\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

TAutoConsoleVariable<int32> CVarD3D12RHIStablePowerState(
	TEXT("D3D12.StablePowerState"),
	0,
	TEXT("Enable stable power state. This increases GPU timing measurement accuracy but may decrease overall GPU clock rate.\n")
	TEXT("    0 (default): off\n")
	TEXT("    1          : set during profiling (no longer implemented)\n")
	TEXT("    2          : set on startup\n"),
	ECVF_Default
);

bool D3D12_ShouldLogD3DDebugWarnings()
{
	return GD3D12DebugCvar.GetValueOnAnyThread() > 1;
}

bool D3D12_ShouldBreakOnD3DDebugErrors()
{
	return GD3D12DebugCvar.GetValueOnAnyThread() > 0 && GD3D12DebugCvar.GetValueOnAnyThread() != 4;
}

bool D3D12_ShouldBreakOnD3DDebugWarnings()
{
	return GD3D12DebugCvar.GetValueOnAnyThread() == 3;
}

// 1 for now (e.g. EBs only on platforms where that's the only available implementation type)
// There's a hole in the EB API that means we have to do some hacking to make it "work" and that's
// not guaranteed to do the right thing on a wide range of drivers so we can't use them by default
int32 GPreferredBarrierImplementation = 1;
static FAutoConsoleVariableRef CVarPreferredBarrierImplementation(
	TEXT("r.D3D12.PreferredBarrierImplementation"),
	GPreferredBarrierImplementation,
	TEXT("The preferred barrier implementation to use\n")
	TEXT("0 Default\n")
	TEXT("1 Legacy Barriers\n")
	TEXT("2 Enhanced Barriers\n")
	TEXT("All other values treated as Default"),
	ECVF_ReadOnly);

static ED3D12BarrierImplementationType GetImplementationTypeFromCVarValue(
	int32 CVarValue)
{
	switch (CVarValue)
	{
	default:
	case 0:
		return ED3D12BarrierImplementationType::Invalid;
	case 1:
		return ED3D12BarrierImplementationType::Legacy;
	case 2:
		return ED3D12BarrierImplementationType::Enhanced;
	}
}

static const TCHAR* GetImplementationTypeString(ED3D12BarrierImplementationType ImplType)
{
	switch (ImplType)
	{
	case ED3D12BarrierImplementationType::Legacy:
		return TEXT("Legacy");
	case ED3D12BarrierImplementationType::Enhanced:
		return TEXT("Enhanced");
	default:
		checkNoEntry();
		return TEXT("Unknown");
	}
}

ED3D12BarrierImplementationType FD3D12Adapter::PreferredBarrierImplType = ED3D12BarrierImplementationType::Invalid;
TUniquePtr<const FD3D12BarriersFactory::BarriersForAdapterType> FD3D12Adapter::Barriers = nullptr;

#if PLATFORM_WINDOWS

enum class ED3D12DredMode
{
	Disabled,
	Lightweight,
	Full
};

static TAutoConsoleVariable<int32> CVarD3D12EnableDRED(
	TEXT("r.D3D12.DRED"),
	0,
	TEXT("Enable DRED GPU Crash debugging mode to track the current GPU state and logs information what operations the GPU executed last.")
	TEXT("Has GPU overhead but gives the most information on the current GPU state when it crashes or hangs.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarD3D12EnableLightweightDRED(
	TEXT("r.D3D12.LightweightDRED"),
	0,
	TEXT("Enable Lightweight DRED GPU Crash debugging mode to track the current GPU state and logs information what operations the GPU executed last.")
	TEXT("Gives the basic information on the current GPU state when it crashes or hangs on all PC hardware.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

ED3D12DredMode D3D12_GetDredMode()
{
	if (UE::RHI::ShouldEnableGPUCrashFeature(*CVarD3D12EnableDRED, TEXT("dred")))
	{
		return ED3D12DredMode::Full;
	}
	else if (UE::RHI::ShouldEnableGPUCrashFeature(*CVarD3D12EnableLightweightDRED, TEXT("lightdred")))
	{
		// Intel suffers a significant performance hit.
		return IsRHIDeviceIntel()
			 ? ED3D12DredMode::Disabled
			 : ED3D12DredMode::Lightweight;
	}
	else
	{
		return ED3D12DredMode::Disabled;
	}
}

bool GD3D12TrackAllAllocations = false;
static FAutoConsoleVariableRef CVarD3D12TrackAllAllocations(
	TEXT("D3D12.TrackAllAllocations"),
	GD3D12TrackAllAllocations,
	TEXT("Controls whether D3D12 RHI should track all allocation information (default = off)."),
	ECVF_ReadOnly
);

bool GD3D12EvictAllResidentResourcesInBackground = false;
static FAutoConsoleVariableRef CVarD3D12EvictAllResidentResourcesInBackground(
	TEXT("D3D12.EvictAllResidentResourcesInBackground"),
	GD3D12EvictAllResidentResourcesInBackground,
	TEXT("Force D3D12 resource residency manager to evict all tracked unused resources when the application is not focused\n"),
	ECVF_Default);

int32 GD3D12ResidencyDebugBudgetMB = -1;
static FAutoConsoleVariableRef CVarD3D12ResidencyDebugBudgetMB(
	TEXT("D3D12.ResidencyDebugBudgetMB"),
	GD3D12ResidencyDebugBudgetMB,
	TEXT("When >= 0, overrides the local memory budget for the residency manager (in MB).\n")
	TEXT("Use a small value (e.g. 128) to force aggressive eviction/MakeResident churn,\n")
	TEXT("exposing missing UpdateResidency calls that would otherwise be hidden by implicit driver paging.\n")
	TEXT("Set to 0 for maximum eviction pressure. Default: -1 (disabled)."),
	ECVF_Default);

#endif // PLATFORM_WINDOWS

#if D3D12_SUPPORTS_INFO_QUEUE
D3D12_MESSAGE_SEVERITY GDebugLayerMinimumSeverity = D3D12_MESSAGE_SEVERITY_ERROR;

struct MessageFilter
{
	MessageFilter(D3D12_MESSAGE_ID ID, const TCHAR* MessageSubString ): MessageID(ID), SubString(MessageSubString)
	{}

	D3D12_MESSAGE_ID MessageID;
	FString SubString;
};

TArray<MessageFilter> GIgnoreD3D12MessageArray;

// Returns true if a message is in the ignore list, or with not enough severity, and we don't want to show it
static bool IsMessageFilteredOut(D3D12_MESSAGE_SEVERITY Severity, D3D12_MESSAGE_ID ID, LPCSTR pDescription)
{
	if (Severity > GDebugLayerMinimumSeverity)
	{
		return true;
	}

	for (const MessageFilter& MessageToFilter : GIgnoreD3D12MessageArray)
	{
		if (ID == MessageToFilter.MessageID)
		{
			if ((MessageToFilter.SubString.IsEmpty()) || (FCStringAnsi::Strstr(pDescription, TCHAR_TO_ANSI(*MessageToFilter.SubString)) != nullptr))
			{
				return true;
			}
		}
	}

	return false;
}

// Log with the UE_LOG the message received from D3D12 debug layer
// Return if we need to break execution or not
static bool LogD3D12Message(D3D12_MESSAGE_SEVERITY Severity, D3D12_MESSAGE_ID ID, LPCSTR pDescription)
{
	bool bBreakOnLog = false;

	if (pDescription)
	{
		if (IsMessageFilteredOut(Severity, ID, pDescription))
		{
			return false;
		}

		switch (Severity)
		{
		case D3D12_MESSAGE_SEVERITY_CORRUPTION:
		case D3D12_MESSAGE_SEVERITY_ERROR:
		{
			UE_LOGF(LogD3D12RHI, Error, "[D3DDebug] %ls", ANSI_TO_TCHAR(pDescription));
			bBreakOnLog = D3D12_ShouldBreakOnD3DDebugErrors();
			break;
		}
		case D3D12_MESSAGE_SEVERITY_WARNING:
		{
			UE_LOGF(LogD3D12RHI, Warning, "[D3DDebug] %ls", ANSI_TO_TCHAR(pDescription));
			bBreakOnLog = D3D12_ShouldBreakOnD3DDebugWarnings();
			break;
		}
		default:
		{
			UE_LOGF(LogD3D12RHI, Log, "[D3DDebug] %ls", ANSI_TO_TCHAR(pDescription));
			break;
		}
		}
	}

	return bBreakOnLog;
}

static bool CheckD3DStoredMessages(ID3D12Device* Device)
{
	bool bResult = false;

	TRefCountPtr<ID3D12Debug> d3dDebug;
	if (SUCCEEDED(D3D12GetDebugInterface(__uuidof(ID3D12Debug), (void**)d3dDebug.GetInitReference())))
	{
		TRefCountPtr<ID3D12InfoQueue> d3dInfoQueue;
		if (SUCCEEDED(Device->QueryInterface(__uuidof(ID3D12InfoQueue), (void**)d3dInfoQueue.GetInitReference())))
		{
			D3D12_MESSAGE* d3dMessage = nullptr;
			SIZE_T AllocateSize = 0;

			int StoredMessageCount = d3dInfoQueue->GetNumStoredMessagesAllowedByRetrievalFilter();
			for (int MessageIndex = 0; MessageIndex < StoredMessageCount; MessageIndex++)
			{
				SIZE_T MessageLength = 0;
				HRESULT hr = d3dInfoQueue->GetMessage(MessageIndex, nullptr, &MessageLength);

				// Ideally the exception handler should not allocate any memory because it could fail
				// and can cause another exception to be triggered and possible even cause a deadlock.
				// But for these D3D error message it should be fine right now because they are requested
				// exceptions when making an error against the API.
				// Not allocating memory for the messages is easy (cache memory in Adapter), but ANSI_TO_TCHAR
				// and UE_LOG will also allocate memory and aren't that easy to fix.

				// realloc the message
				if (MessageLength > AllocateSize)
				{
					if (d3dMessage)
					{
						FMemory::Free(d3dMessage);
						d3dMessage = nullptr;
						AllocateSize = 0;
					}

					d3dMessage = (D3D12_MESSAGE*)FMemory::Malloc(MessageLength);
					AllocateSize = MessageLength;
				}

				if (d3dMessage)
				{
					// get the actual message data from the queue
					hr = d3dInfoQueue->GetMessage(MessageIndex, d3dMessage, &MessageLength);

					bResult = LogD3D12Message(d3dMessage->Severity, d3dMessage->ID, d3dMessage->pDescription);
				}
			}
			d3dInfoQueue->ClearStoredMessages();
			if (AllocateSize > 0)
			{
				FMemory::Free(d3dMessage);
			}
		}
	}

	return bResult;
}
#endif // #if D3D12_SUPPORTS_INFO_QUEUE

DWORD GD3D12CallBackCookie = 0;

#if PLATFORM_WINDOWS
/** Handle d3d messages and write them to the log file **/
static LONG __stdcall D3DVectoredExceptionHandler(EXCEPTION_POINTERS* InInfo)
{
	// Only handle D3D error codes here
	if (InInfo->ExceptionRecord->ExceptionCode == _FACDXGI)
	{
		// Logging and break in code is managed in the D3D12MessageCallBack if ID3D12InfoQueue1 but we handle and continue D3D exception in order to not break while the debugger is not attached
		if (GD3D12CallBackCookie == 0)
		{
			FD3D12DynamicRHI* const D3D12RHI = FD3D12DynamicRHI::GetD3DRHI();
			if (D3D12RHI && D3D12RHI->HasAdapter() &&
				CheckD3DStoredMessages(D3D12RHI->GetAdapter().GetD3DDevice()))
			{
				if (FPlatformMisc::IsDebuggerPresent())
				{
					// when we get here, then it means that BreakOnSeverity was set for this error message, so request the debug break here as well
					// when the debugger is attached
					UE_DEBUG_BREAK();
				}
			}
		}

		// Handles the exception
		return EXCEPTION_CONTINUE_EXECUTION;
	}

	// continue searching
	return EXCEPTION_CONTINUE_SEARCH;
}
#endif // #if PLATFORM_WINDOWS

#if D3D12_SUPPORTS_INFO_QUEUE
void D3D12MessageCallBack(D3D12_MESSAGE_CATEGORY Category, D3D12_MESSAGE_SEVERITY Severity, D3D12_MESSAGE_ID ID, LPCSTR pDescription, void* pContext)
{
	bool bBreakOnLog = LogD3D12Message(Severity, ID, pDescription);
	
	if (FPlatformMisc::IsDebuggerPresent() && bBreakOnLog)
	{
		// When we get here, then it means that BreakOnSeverity was set for this error message, so request the debug break here as well when the debugger is attached
		GLog->Flush();
		UE_DEBUG_BREAK();
	}

}
#endif


FTransientUniformBufferAllocator::~FTransientUniformBufferAllocator()
{
	if (Adapter)
	{
		Adapter->ReleaseTransientUniformBufferAllocator(this);
	}
}

void FTransientUniformBufferAllocator::Cleanup()
{
	ClearResource();
	Adapter = nullptr;
}

FD3D12AdapterDesc::FD3D12AdapterDesc() = default;

FD3D12AdapterDesc::FD3D12AdapterDesc(const DXGI_ADAPTER_DESC& InDesc, int32 InAdapterIndex, const FD3D12DeviceBasicInfo& DeviceInfo)
	: Desc(InDesc)
	, AdapterIndex(InAdapterIndex)
	, MaxSupportedFeatureLevel(DeviceInfo.MaxFeatureLevel)
	, MaxSupportedShaderModel(DeviceInfo.MaxShaderModel)
	, ResourceBindingTier(DeviceInfo.ResourceBindingTier)
	, ResourceHeapTier(DeviceInfo.ResourceHeapTier)
	, MaxRHIFeatureLevel(DeviceInfo.MaxRHIFeatureLevel)
	, bTileBasedRenderer(DeviceInfo.bTileBasedRenderer)
	, bUMA(DeviceInfo.bUMA)
	, bCacheCoherentUMA(DeviceInfo.bCacheCoherentUMA)
	, bSupportsGPUUploadHeaps(DeviceInfo.bSupportsGPUUploadHeaps)
	, bSupportsWaveOps(DeviceInfo.bSupportsWaveOps)
	, bSupportsAtomic64(DeviceInfo.bSupportsAtomic64)
{
}

bool FD3D12AdapterDesc::IsValid() const
{
	return MaxSupportedFeatureLevel != (D3D_FEATURE_LEVEL)0 && AdapterIndex >= 0;
}

#if DXGI_MAX_FACTORY_INTERFACE >= 6
HRESULT FD3D12AdapterDesc::EnumAdapters(int32 AdapterIndex, DXGI_GPU_PREFERENCE GpuPreference, IDXGIFactory2* DxgiFactory2, IDXGIFactory6* DxgiFactory6, IDXGIAdapter** TempAdapter)
{
	if (!DxgiFactory6 || GpuPreference == DXGI_GPU_PREFERENCE_UNSPECIFIED)
	{
		return DxgiFactory2->EnumAdapters(AdapterIndex, TempAdapter);
	}
	else
	{
		return DxgiFactory6->EnumAdapterByGpuPreference(AdapterIndex, GpuPreference, IID_PPV_ARGS(TempAdapter));
	}
}

HRESULT FD3D12AdapterDesc::EnumAdapters(IDXGIFactory2* DxgiFactory2, IDXGIFactory6* DxgiFactory6, IDXGIAdapter** TempAdapter) const
{
	return EnumAdapters(AdapterIndex, GpuPreference, DxgiFactory2, DxgiFactory6, TempAdapter);
}
#endif

FD3D12Adapter::FD3D12Adapter(FD3D12AdapterDesc& DescIn)
	: Desc(DescIn)
	, RootSignatureManager(this)
	, PipelineStateCache(this)
	, DefaultContextRedirector(this, ED3D12QueueType::Direct, true)
#if USE_STATIC_ROOT_SIGNATURE
	, StaticGraphicsRootSignature(this)
	, StaticGraphicsWithConstantsRootSignature(this)
	, StaticComputeRootSignature(this)
	, StaticComputeWithConstantsRootSignature(this)
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, StaticBindlessGraphicsRootSignature(this)
	, StaticBindlessGraphicsWithConstantsRootSignature(this)
	, StaticBindlessComputeRootSignature(this)
	, StaticBindlessComputeWithConstantsRootSignature(this)
#endif
	, StaticRayTracingGlobalRootSignature(this)
	, StaticRayTracingLocalRootSignature(this)
#endif
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, BindlessDescriptorAllocator(this)
#endif
{
	FMemory::Memzero(&UploadHeapAllocator, sizeof(UploadHeapAllocator));
	FMemory::Memzero(&Devices, sizeof(Devices));

	uint32 MaxGPUCount = 1; // By default, multi-gpu is disabled.
#if WITH_MGPU
	FParse::Value(FCommandLine::Get(), TEXT("MaxGPUCount="), MaxGPUCount);

	if (FParse::Param(FCommandLine::Get(), TEXT("VMGPU")))
	{
		GVirtualMGPU = 1;
		UE_LOGF(LogD3D12RHI, Log, "Enabling virtual multi-GPU mode");
	}
#endif

	if (GVirtualMGPU)
	{
		Desc.NumDeviceNodes = FMath::Min<uint32>(MaxGPUCount, MAX_NUM_GPUS);
	}
	else
	{
		Desc.NumDeviceNodes = FMath::Min3<uint32>(Desc.NumDeviceNodes, MaxGPUCount, (uint32)MAX_NUM_GPUS);
	}
}

void FD3D12Adapter::CreateRootDevice(bool bWithDebug)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12Adapter::CreateRootDevice);

	// -d3ddebug is always allowed on Windows, but only allowed in non-shipping builds on other platforms.
	// -gpuvalidation is only supported on Windows.
#if PLATFORM_WINDOWS || !UE_BUILD_SHIPPING
#if PLATFORM_WINDOWS
	bool bWithGPUValidation =  (FParse::Param(FCommandLine::Get(), TEXT("d3d12gpuvalidation")) || FParse::Param(FCommandLine::Get(), TEXT("gpuvalidation")));
	// If GPU validation is requested, automatically enable the debug layer.
	bWithDebug |= bWithGPUValidation;
#endif
	if (bWithDebug)
	{
		TRefCountPtr<ID3D12Debug> DebugController;
		HRESULT hr = D3D12GetDebugInterface(IID_PPV_ARGS(DebugController.GetInitReference()));
		if (SUCCEEDED(hr))
		{
			DebugController->EnableDebugLayer();
			bDebugDevice = true;

#if PLATFORM_WINDOWS
			if (bWithGPUValidation)
			{
				TRefCountPtr<ID3D12Debug1> DebugController1;
				VERIFYD3D12RESULT(DebugController->QueryInterface(IID_PPV_ARGS(DebugController1.GetInitReference())));
				DebugController1->SetEnableGPUBasedValidation(true);
				SetEmitDrawEvents(true);
			}
#endif
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Fatal, "Failed to create D3D debug interface, error %x. The debug interface requires the D3D12 SDK Layers. Please install the Graphics Tools for Windows. See: https://docs.microsoft.com/en-us/windows/uwp/gaming/use-the-directx-runtime-and-visual-studio-graphics-diagnostic-features", hr);
		}
	}

	FGenericCrashContext::SetEngineData(TEXT("RHI.D3DDebug"), bWithDebug ? TEXT("true") : TEXT("false"));
#if PLATFORM_WINDOWS
	UE_LOGF(LogD3D12RHI, Log, "InitD3DDevice: -D3DDebug = %ls -D3D12GPUValidation = %ls", bWithDebug ? TEXT("on") : TEXT("off"), bWithGPUValidation ? TEXT("on") : TEXT("off"));
#else
	UE_LOGF(LogD3D12RHI, Log, "InitD3DDevice: -D3DDebug = %ls -D3D12GPUValidation = off", bWithDebug ? TEXT("on") : TEXT("off"));
#endif
#endif

#if PLATFORM_WINDOWS
	
#if NV_AFTERMATH
	if (IsRHIDeviceNVIDIA())
	{
		UE::RHICore::Nvidia::Aftermath::InitializeBeforeDeviceCreation();
		UE::RHICore::Nvidia::Aftermath::SetLateShaderAssociateCallback(UE::RHICore::Nvidia::Aftermath::D3D12::CreateShaderAssociations);
	}
#endif // NV_AFTERMATH

	// Setup DRED if requested
	bool bDRED = false;
	bool bDREDMarkersOnly = false; // LightweightDRED
	bool bDREDContext = false;
	{		
		HMODULE d3d12DllHandle = (HMODULE)FPlatformProcess::GetDllHandle(TEXT("d3d12.dll"));
		typedef HRESULT(WINAPI* FD3D12GetInterface)(REFCLSID, REFIID, void**);

		if (d3d12DllHandle)
		{
			FD3D12GetInterface D3D12GetInterfaceFnPtr = (FD3D12GetInterface)(void*)(GetProcAddress(d3d12DllHandle, "D3D12GetInterface"));

			if (D3D12GetInterfaceFnPtr != nullptr)
			{
				if (D3D12_GetDredMode() == ED3D12DredMode::Full)
				{
					TRefCountPtr<ID3D12DeviceRemovedExtendedDataSettings> DredSettings;
					HRESULT hr = D3D12GetInterfaceFnPtr(CLSID_D3D12DeviceRemovedExtendedData, IID_PPV_ARGS(DredSettings.GetInitReference()));

					// Can fail if not on correct Windows Version - needs 1903 or newer
					if (SUCCEEDED(hr))
					{
						// Turn on AutoBreadcrumbs and Page Fault reporting
						DredSettings->SetAutoBreadcrumbsEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);
						DredSettings->SetPageFaultEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);

						bDRED = true;
						bDREDMarkersOnly = false;
						SetEmitDrawEvents(true);
						UE_LOGF(LogD3D12RHI, Log, "[DRED] DRED enabled");
					}
					else
					{
						UE_LOGF(LogD3D12RHI, Log, "[DRED] DRED requested but interface was not found, hresult: %x. DRED only works on Windows 10 1903+.", hr);
					}

#ifdef __ID3D12DeviceRemovedExtendedDataSettings1_INTERFACE_DEFINED__
					TRefCountPtr<ID3D12DeviceRemovedExtendedDataSettings1> DredSettings1;
					hr = D3D12GetInterfaceFnPtr(CLSID_D3D12DeviceRemovedExtendedData, IID_PPV_ARGS(DredSettings1.GetInitReference()));

					if (SUCCEEDED(hr))
					{
						DredSettings1->SetBreadcrumbContextEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);
						bDREDContext = true;
						UE_LOGF(LogD3D12RHI, Log, "[DRED] Dred breadcrumb context enabled");
					}
#endif // __ID3D12DeviceRemovedExtendedDataSettings1_INTERFACE_DEFINED__
				}
				else if (D3D12_GetDredMode() == ED3D12DredMode::Lightweight)
				{
#ifdef __ID3D12DeviceRemovedExtendedDataSettings2_INTERFACE_DEFINED__
					TRefCountPtr<ID3D12DeviceRemovedExtendedDataSettings2> DredSettings2;
					HRESULT hr = D3D12GetInterfaceFnPtr(CLSID_D3D12DeviceRemovedExtendedData, IID_PPV_ARGS(DredSettings2.GetInitReference()));

					if (SUCCEEDED(hr))
					{
						DredSettings2->SetBreadcrumbContextEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);
						bDREDContext = true;
						UE_LOGF(LogD3D12RHI, Log, "[DRED] Dred breadcrumb context enabled");

						// Turn on AutoBreadcrumbs and Page Fault reporting
						DredSettings2->SetAutoBreadcrumbsEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);
						DredSettings2->SetPageFaultEnablement(D3D12_DRED_ENABLEMENT_FORCED_ON);

						bDREDMarkersOnly = true;
						DredSettings2->UseMarkersOnlyAutoBreadcrumbs(true);
						SetEmitDrawEvents(true);
						UE_LOGF(LogD3D12RHI, Log, "[DRED] Using lightweight DRED.");
					}
					else
					{
						UE_LOGF(LogD3D12RHI, Log, "[DRED] Lightweight DRED requested but interface was not found, hresult: %x. DRED only works on Windows 10 1903+.", hr);
					}
#else
					UE_LOGF(LogD3D12RHI, Log, "[DRED] Lightweight DRED unsupported.");
#endif // __ID3D12DeviceRemovedExtendedDataSettings2_INTERFACE_DEFINED__
				}
			}
		}
	}

	FGenericCrashContext::SetEngineData(TEXT("RHI.DRED"), bDRED ? TEXT("true") : TEXT("false"));
	FGenericCrashContext::SetEngineData(TEXT("RHI.DREDMarkersOnly"), bDREDMarkersOnly ? TEXT("true") : TEXT("false"));
	FGenericCrashContext::SetEngineData(TEXT("RHI.DREDContext"), bDREDContext && bDRED ? TEXT("true") : TEXT("false"));

#endif // PLATFORM_WINDOWS

#if USE_PIX
	UE_LOGF(LogD3D12RHI, Log, "Emitting draw events for PIX profiling.");
	SetEmitDrawEvents(true);
#endif

	CreateDXGIFactory(bWithDebug);

	// QI for the Adapter
	TRefCountPtr<IDXGIAdapter> TempAdapter;

	EnumAdapters(TempAdapter.GetInitReference());

	VERIFYD3D12RESULT(TempAdapter->QueryInterface(IID_PPV_ARGS(DxgiAdapter.GetInitReference())));

	bool bDeviceCreated = false;
#if !PLATFORM_CPU_ARM_FAMILY && (PLATFORM_WINDOWS)
	if (IsRHIDeviceAMD() && FD3D12DynamicRHI::GetD3DRHI()->GetAmdAgsContext())
	{
		auto* CVarDisableEngineAndAppRegistration = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableEngineAndAppRegistration"));

		const bool bDisableEngineRegistration = IsShaderDevelopmentModeEnabled() ||
			(CVarDisableEngineAndAppRegistration && CVarDisableEngineAndAppRegistration->GetValueOnAnyThread() != 0);
		const bool bDisableAppRegistration = bDisableEngineRegistration || !FApp::HasProjectName();

		// Creating the Direct3D device with AGS registration and extensions.
		AGSDX12DeviceCreationParams AmdDeviceCreationParams = {
			GetAdapter(),											// IDXGIAdapter*               pAdapter;
			__uuidof(**(RootDevice.GetInitReference())),			// IID                         iid;
			GetFeatureLevel(),										// D3D_FEATURE_LEVEL           FeatureLevel;
		};

		AGSDX12ExtensionParams AmdExtensionParams;
		FMemory::Memzero(&AmdExtensionParams, sizeof(AmdExtensionParams));

		// Register the engine name with the AMD driver, e.g. "UnrealEngine4.19", unless disabled
		// (note: to specify nothing for pEngineName below, you need to pass an empty string, not a null pointer)
		FString EngineName = FApp::GetEpicProductIdentifier() + FEngineVersion::Current().ToString(EVersionComponent::Minor);
		AmdExtensionParams.pEngineName = bDisableEngineRegistration ? TEXT("") : *EngineName;
		AmdExtensionParams.engineVersion = AGS_UNSPECIFIED_VERSION;

		// Register the project name with the AMD driver, unless disabled or no project name
		// (note: to specify nothing for pAppName below, you need to pass an empty string, not a null pointer)
		AmdExtensionParams.pAppName = bDisableAppRegistration ? TEXT("") : FApp::GetProjectName();
		AmdExtensionParams.appVersion = AGS_UNSPECIFIED_VERSION;

		// From Shaders\Shared\ThirdParty\AMD\ags_shader_intrinsics_dx12.h, the default dummy UAV used
		// to access shader intrinsics is declared as below:
		// RWByteAddressBuffer AmdExtD3DShaderIntrinsicsUAV : register(u0, AmdExtD3DShaderIntrinsicsSpaceId);
		// So, use slot 0 here to match.
		AmdExtensionParams.uavSlot = 0;

		AGSDX12ReturnedParams DeviceCreationReturnedParams;
		FMemory::Memzero(&DeviceCreationReturnedParams, sizeof(DeviceCreationReturnedParams));
		AGSReturnCode DeviceCreation = agsDriverExtensionsDX12_CreateDevice(
			FD3D12DynamicRHI::GetD3DRHI()->GetAmdAgsContext(),
			&AmdDeviceCreationParams,
			&AmdExtensionParams,
			&DeviceCreationReturnedParams
		);

		if (DeviceCreation == AGS_SUCCESS)
		{
			RootDevice = DeviceCreationReturnedParams.pDevice;
			{
				static_assert(sizeof(AGSDX12ReturnedParams::ExtensionsSupported) == sizeof(uint32));
				uint32 AMDSupportedExtensionFlags;
				FMemory::Memcpy(&AMDSupportedExtensionFlags, &DeviceCreationReturnedParams.extensionsSupported, sizeof(uint32));
				FD3D12DynamicRHI::GetD3DRHI()->SetAmdSupportedExtensionFlags(AMDSupportedExtensionFlags);
			}
			bDeviceCreated = true;
		}
	}
#endif

#if INTEL_EXTENSIONS
	if (IsRHIDeviceIntel() && UE::RHICore::AllowVendorDevice())
	{
		ID3D12Device* Device = nullptr;
		// Create the device for communication with the extension
		VERIFYD3D12RESULT(D3D12CreateDevice(
			GetAdapter(),
			D3D_FEATURE_LEVEL_11_0,
			IID_PPV_ARGS(&Device)
		));

		INTCExtensionInfo INTCExtensionInfo{};
		if (INTCExtensionContext* IntelExtensionContext = CreateIntelExtensionsContext(Device, INTCExtensionInfo, GetD3DAdapterDesc().DeviceId))
		{
			EnableIntelAtomic64Support(IntelExtensionContext, INTCExtensionInfo);
			// Destroy the context to release all reference to ID3D12Device
			DestroyIntelExtensionsContext(IntelExtensionContext);
		}

		Device->Release();
	}
#endif

	if (!bDeviceCreated)
	{
#if INTEL_EXTENSIONS
		if (IsRHIDeviceIntel() && UE::RHICore::AllowVendorDevice())
		{
			// Enable Intel App Discovery
			EnableIntelAppDiscovery(GRHIDeviceId);

#if INTEL_GPU_CRASH_DUMPS
			UE::RHICore::Intel::GPUCrashDumps::InitializeBeforeDeviceCreation(GRHIDeviceId);
#endif
		}

#endif

		// Creating the Direct3D device.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(D3D12CreateDevice);
			VERIFYD3D12RESULT(D3D12CreateDevice(
				GetAdapter(),
				GetFeatureLevel(),
				IID_PPV_ARGS(RootDevice.GetInitReference())
			));
		}
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	if (!CVarResidencyManagement.GetValueOnAnyThread())
	{
		UE_LOGF(LogD3D12RHI, Log, "D3D12 resource residency management is disabled.");
		GEnableResidencyManagement = false;
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT

#if NV_AFTERMATH
	if (IsRHIDeviceNVIDIA())
	{
		UE::RHICore::Nvidia::Aftermath::D3D12::InitializeDevice(RootDevice);
	}
#endif

#if PLATFORM_WINDOWS
	if (bWithDebug)
	{
		// add vectored exception handler to write the debug device warning & error messages to the log
		ExceptionHandlerHandle = AddVectoredExceptionHandler(1, D3DVectoredExceptionHandler);
	}
#endif // PLATFORM_WINDOWS

#if D3D12_SUPPORTS_DXGI_DEBUG
	if (bWithDebug)
	{
		// Manually load dxgi debug if available
		DxgiDebugDllHandle = (HMODULE)FPlatformProcess::GetDllHandle(TEXT("dxgidebug.dll"));
		if (DxgiDebugDllHandle)
		{
			typedef HRESULT(WINAPI* FDXGIGetDebugInterface)(REFIID, void**);
			FDXGIGetDebugInterface DXGIGetDebugInterfaceFnPtr = (FDXGIGetDebugInterface)(void*)(GetProcAddress(DxgiDebugDllHandle, "DXGIGetDebugInterface"));
			if (DXGIGetDebugInterfaceFnPtr != nullptr)
			{
				DXGIGetDebugInterfaceFnPtr(IID_PPV_ARGS(DXGIDebug.GetInitReference()));
			}
		}
	}
#endif // D3D12_SUPPORTS_DXGI_DEBUG

#if UE_BUILD_DEBUG	&& D3D12_SUPPORTS_INFO_QUEUE
	//break on debug
	TRefCountPtr<ID3D12Debug> d3dDebug;
	if (SUCCEEDED(RootDevice->QueryInterface(__uuidof(ID3D12Debug), (void**)d3dDebug.GetInitReference())))
	{
		TRefCountPtr<ID3D12InfoQueue> d3dInfoQueue;
		if (SUCCEEDED(d3dDebug->QueryInterface(__uuidof(ID3D12InfoQueue), (void**)d3dInfoQueue.GetInitReference())))
		{
			d3dInfoQueue->SetBreakOnSeverity(D3D12_MESSAGE_SEVERITY_CORRUPTION, D3D12_ShouldBreakOnD3DDebugErrors());
			d3dInfoQueue->SetBreakOnSeverity(D3D12_MESSAGE_SEVERITY_ERROR, D3D12_ShouldBreakOnD3DDebugErrors());
			d3dInfoQueue->SetBreakOnSeverity(D3D12_MESSAGE_SEVERITY_WARNING, D3D12_ShouldBreakOnD3DDebugWarnings());
		}
	}
#endif

#if WITH_MGPU
	GNumExplicitGPUsForRendering = 1;
	if (Desc.NumDeviceNodes > 1)
	{
		GNumExplicitGPUsForRendering = Desc.NumDeviceNodes;
		UE_LOGF(LogD3D12RHI, Log, "Enabling multi-GPU with %d nodes", Desc.NumDeviceNodes);
	}
#endif

	if (RootDevice)
	{
		if (CVarD3D12RHIStablePowerState.GetValueOnAnyThread() == 2)
		{
			bool bWorked = SUCCEEDED(RootDevice->SetStablePowerState(true));
			// This will fail if windows developper mode is not enabled. Windows will remove the adapter on failure, so we can't really gracefully exit here.
			checkf(bWorked, TEXT("Enabling state power state requires Windows developer mode to be enabled."));
		}
	}
}

// Add some filter outs for known debug spew messages (that we don't care about)
void FD3D12Adapter::CreateD3DInfoQueue()
{
#if !(UE_BUILD_SHIPPING && WITH_EDITOR) && D3D12_SUPPORTS_INFO_QUEUE
	ID3D12InfoQueue* pd3dInfoQueue = nullptr;
	VERIFYD3D12RESULT(RootDevice->QueryInterface(__uuidof(ID3D12InfoQueue), (void**)&pd3dInfoQueue));
	if (pd3dInfoQueue)
	{
		D3D12_INFO_QUEUE_FILTER NewFilter;
		FMemory::Memzero(&NewFilter, sizeof(NewFilter));

		// Turn off info msgs as these get really spewy
		const bool bLogWarnings = D3D12_ShouldBreakOnD3DDebugWarnings() || D3D12_ShouldLogD3DDebugWarnings();
		D3D12_MESSAGE_SEVERITY DenySeverity[] = { D3D12_MESSAGE_SEVERITY_INFO, D3D12_MESSAGE_SEVERITY_WARNING };
		NewFilter.DenyList.NumSeverities = 1 + (bLogWarnings ? 0 : 1);
		NewFilter.DenyList.pSeverityList = DenySeverity;
		// Be sure to carefully comment the reason for any additions here!  Someone should be able to look at it later and get an idea of whether it is still necessary.
		TArray<D3D12_MESSAGE_ID, TInlineAllocator<16>> DenyIds = {

			// The Pixel Shader expects a Render Target View bound to slot 0, but the PSO indicates that none will be bound.
			// This typically happens when a non-depth-only pixel shader is used for depth-only rendering.
			D3D12_MESSAGE_ID_CREATEGRAPHICSPIPELINESTATE_RENDERTARGETVIEW_NOT_SET,

			// QUERY_BEGIN_ABANDONING_PREVIOUS_RESULTS - The RHI exposes the interface to make and issue queries and a separate interface to use that data.
			//		Currently there is a situation where queries are issued and the results may be ignored on purpose.  Filtering out this message so it doesn't
			//		swarm the debug spew and mask other important warnings
			//D3D12_MESSAGE_ID_QUERY_BEGIN_ABANDONING_PREVIOUS_RESULTS,
			//D3D12_MESSAGE_ID_QUERY_END_ABANDONING_PREVIOUS_RESULTS,

			// D3D12_MESSAGE_ID_CREATEINPUTLAYOUT_EMPTY_LAYOUT - This is a warning that gets triggered if you use a null vertex declaration,
			//       which we want to do when the vertex shader is generating vertices based on ID.
			D3D12_MESSAGE_ID_CREATEINPUTLAYOUT_EMPTY_LAYOUT,

			// D3D12_MESSAGE_ID_COMMAND_LIST_DRAW_INDEX_BUFFER_TOO_SMALL - This warning gets triggered by Slate draws which are actually using a valid index range.
			//		The invalid warning seems to only happen when VS 2012 is installed.  Reported to MS.  
			//		There is now an assert in DrawIndexedPrimitive to catch any valid errors reading from the index buffer outside of range.
			//D3D12_MESSAGE_ID_COMMAND_LIST_DRAW_INDEX_BUFFER_TOO_SMALL,

			// D3D12_MESSAGE_ID_DEVICE_DRAW_RENDERTARGETVIEW_NOT_SET - This warning gets triggered by shadow depth rendering because the shader outputs
			//		a color but we don't bind a color render target. That is safe as writes to unbound render targets are discarded.
			//		Also, batched elements triggers it when rendering outside of scene rendering as it outputs to the GBuffer containing normals which is not bound.
			//(D3D12_MESSAGE_ID)3146081, // D3D12_MESSAGE_ID_DEVICE_DRAW_RENDERTARGETVIEW_NOT_SET,
			// BUGBUG: There is a D3D12_MESSAGE_ID_DEVICE_DRAW_DEPTHSTENCILVIEW_NOT_SET, why not one for RT?

			// D3D12_MESSAGE_ID_CLEARDEPTHSTENCILVIEW_MISMATCHINGCLEARVALUE/D3D12_MESSAGE_ID_CLEARRENDERTARGETVIEW_MISMATCHINGCLEARVALUE - 
			//      This warning gets triggered by ClearDepthStencilView/ClearRenderTargetView because when the resource was created
			//      it wasn't passed an optimized clear color (see CreateCommitedResource). This shows up a lot and is very noisy.
			D3D12_MESSAGE_ID_CLEARRENDERTARGETVIEW_MISMATCHINGCLEARVALUE,
			D3D12_MESSAGE_ID_CLEARDEPTHSTENCILVIEW_MISMATCHINGCLEARVALUE,

			// D3D12_MESSAGE_ID_EXECUTECOMMANDLISTS_GPU_WRITTEN_READBACK_RESOURCE_MAPPED - This warning gets triggered by ExecuteCommandLists.
			//		if it contains a readback resource that still has mapped subresources when executing a command list that performs a copy operation to the resource.
			//		This may be ok if any data read from the readback resources was flushed by calling Unmap() after the resourcecopy operation completed.
			//		We intentionally keep the readback resources persistently mapped.
			D3D12_MESSAGE_ID_EXECUTECOMMANDLISTS_GPU_WRITTEN_READBACK_RESOURCE_MAPPED,

			// This error gets generated on the first run when you install a new driver. The code handles this error properly and resets the PipelineLibrary,
			// so we can safely ignore this message. It could possibly be avoided by adding driver version to the PSO cache filename, but an average user is unlikely
			// to be interested in keeping PSO caches associated with old drivers around on disk, so it's better to just reset.
			D3D12_MESSAGE_ID_CREATEPIPELINELIBRARY_DRIVERVERSIONMISMATCH,

			// D3D complain about overlapping GPU addresses when aliasing DataBuffers in the same command list when using the Transient Allocator - it looks like
			// it ignored the aliasing barriers to validate, and probably can't check them when called from IASetVertexBuffers because it only has GPU Virtual Addresses then
			D3D12_MESSAGE_ID_HEAP_ADDRESS_RANGE_INTERSECTS_MULTIPLE_BUFFERS,

			// Ignore draw vertex buffer not set or too small - these are warnings and if the shader doesn't read from it it's fine. This happens because vertex
			// buffers are not removed from the cache, but only get removed when another buffer is set at the same slot or when the buffer gets destroyed.
			D3D12_MESSAGE_ID_COMMAND_LIST_DRAW_VERTEX_BUFFER_NOT_SET,
			D3D12_MESSAGE_ID_COMMAND_LIST_DRAW_VERTEX_BUFFER_TOO_SMALL,

			// D3D12 complains when a buffer is created with a specific initial resource state while all buffers are currently created in COMMON state. The 
			// next transition is then done use state promotion. It's just a warning and we need to keep track of the correct initial state as well for upcoming
			// internal transitions.
			D3D12_MESSAGE_ID_CREATERESOURCE_STATE_IGNORED,

#if ENABLE_RESIDENCY_MANAGEMENT
			// TODO: Remove this when the debug layers work for executions which are guarded by a fence
			D3D12_MESSAGE_ID_INVALID_USE_OF_NON_RESIDENT_RESOURCE,
#endif

			// When optimizing the graph DirectML tries various configurations to check if meta command can be used.
			// If a particular configuration is not supported the debug log message will be displayed.
			// For large DirectML graphs there can be dozens of messages like this.
			// This is safe to ignore
			D3D12_MESSAGE_ID_META_COMMAND_UNSUPPORTED_PARAMS,

			// D3D Agility SDK bug (version 614) where scratch allocation memory for BuildRaytracingAccelerationStructure is always computed using build size
			// while operation is update and needs less memory (verified by MS and will be fixed in the next Agility SDK)
			// See: UE-222685 to remove again from Deny list after Agility SDK upgrade
			D3D12_MESSAGE_ID_HEAP_ADDRESS_RANGE_HAS_NO_RESOURCE,

#if D3D12RHI_SUPPORTS_LEGACY_BARRIERS
			// This happens a lot because when we perform copies we'll transition a resource into the COPY_DEST or COPY_SRC state temporarily and then
			// transition it back into whatever state it was in before. Often there's no work done between that transition out of COPY_[DEST|SRC] and the next
			// transition. We already try to minimize this by identifying back-to-back barriers that can be combined, or nullify each other, but it still
			// happens when there are other barriers in-between. Short of tracking barriers by resource in the batcher, I don't see a way to avoid it given 
			// current architecture. The warning is quite noisy so turning it off should help us recognize more actionable and important warnings.
			D3D12_MESSAGE_ID_RESOURCE_BARRIER_DUPLICATE_SUBRESOURCE_TRANSITIONS,
#endif
		};

#if PLATFORM_DESKTOP
		if (!FWindowsPlatformMisc::VerifyWindowsVersion(10, 0, 18363))
		{
			// Ignore a known false positive error due to a bug in validation layer in certain older Windows versions
			DenyIds.Add(D3D12_MESSAGE_ID_COPY_DESCRIPTORS_INVALID_RANGES);
		}
#endif // PLATFORM_DESKTOP

		GIgnoreD3D12MessageArray.Add(MessageFilter(D3D12_MESSAGE_ID_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INVALID, TEXT("exceeds end of the virtual address range of Heap")));

		if (GSupportsEfficientAsyncCompute)
		{
			// This is not optimal to have it disable but this should be absolutely removed when DX12 can use EB 
 			// The reason why this is here is because when async compute is on we can receive from RDG a transition UAV -> SRVMask on the Async Pipe
 			// DX12 cannot transition to Gfx states on the Async Pipe therefore we do: UAV->SRVCompute on Async, Fence Async -> Gfx, SRVCompute->SRVMask on Gfx
 			// Since the debug layer is on the CPU and the submission is happening in parallel threads the validation layer cannot known that the order on the GPU will be UAV->SRVCompute->SRVMask
			GIgnoreD3D12MessageArray.Add(MessageFilter(D3D12_MESSAGE_ID_RESOURCE_BARRIER_BEFORE_AFTER_MISMATCH, TEXT("does not match with the state (0x40: D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE) specified in preceding")));
			GIgnoreD3D12MessageArray.Add(MessageFilter(D3D12_MESSAGE_ID_RESOURCE_BARRIER_BEFORE_AFTER_MISMATCH, TEXT("does not match with the state (0x8: D3D12_RESOURCE_STATE_UNORDERED_ACCESS) specified in preceding")));
		}

		// Adding all DenyIds as well with no sub string
		for (int i = 0; i < DenyIds.Num(); i++)
		{
			GIgnoreD3D12MessageArray.Add(MessageFilter(DenyIds[i], TEXT("")));
		}

		NewFilter.DenyList.NumIDs = DenyIds.Num();
		NewFilter.DenyList.pIDList = DenyIds.GetData();

		pd3dInfoQueue->PushStorageFilter(&NewFilter);

		// Break on D3D debug errors.
		pd3dInfoQueue->SetBreakOnSeverity(D3D12_MESSAGE_SEVERITY_ERROR, D3D12_ShouldBreakOnD3DDebugErrors());

		// Enable this to break on a specific id in order to quickly get a callstack
		//pd3dInfoQueue->SetBreakOnID(D3D12_MESSAGE_ID_DEVICE_DRAW_CONSTANT_BUFFER_TOO_SMALL, true);

		// Break on D3D warnings if warning log or warning breakpoint is enabled
		if (bLogWarnings)
		{
			pd3dInfoQueue->SetBreakOnSeverity(D3D12_MESSAGE_SEVERITY_WARNING, D3D12_ShouldBreakOnD3DDebugWarnings());
		}
		
		// Try to use RegisterMessageCallback in the new ID3D12InfoQueue1 (available since Windows 10 Release Preview build 20236)
		{
			ID3D12InfoQueue1* pd3dInfoQueue1 = nullptr;
			RootDevice->QueryInterface(__uuidof(ID3D12InfoQueue1), (void**)&pd3dInfoQueue1);
			if (pd3dInfoQueue1)
			{
				void* ParamContext = this;
				
				// To control exactly what we want to print out we need to disale the Debug output and ignore all filters if any
				pd3dInfoQueue1->RegisterMessageCallback(D3D12MessageCallBack, D3D12_MESSAGE_CALLBACK_IGNORE_FILTERS, ParamContext, &GD3D12CallBackCookie);
				pd3dInfoQueue1->SetMuteDebugOutput(true);

				// Set the minimum log severity
				GDebugLayerMinimumSeverity = (bLogWarnings) ? D3D12_MESSAGE_SEVERITY_WARNING : D3D12_MESSAGE_SEVERITY_ERROR;

				if (GD3D12CallBackCookie == 0)
				{
					UE_LOGF(LogD3D12RHI, Log, "ID3D12InfoQueue1::RegisterMessageCallback failed.");
				}

				pd3dInfoQueue1->Release();
			}
		}

		pd3dInfoQueue->Release();
	}
#endif
}

FD3D12TransientHeapCache& FD3D12Adapter::GetOrCreateTransientHeapCache()
{
	if (!TransientMemoryCache)
	{
		TransientMemoryCache = FD3D12TransientHeapCache::Create(this);
	}

	return static_cast<FD3D12TransientHeapCache&>(*TransientMemoryCache);
}

void FD3D12Adapter::InitializeDevices()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12Adapter::InitializeDevices);

	check(IsInGameThread());

	// Wait for the rendering thread to go idle.
	FlushRenderingCommands();

	// Use a debug device if specified on the command line.
	bool bWithD3DDebug = GRHIGlobals.IsDebugLayerEnabled;

	// If we don't have a device yet, either because this is the first viewport, or the old device was removed, create a device.
	if (!RootDevice)
	{
		CreateRootDevice(bWithD3DDebug);

		// See if we can get any newer device interfaces (to use newer D3D12 features).
		if (D3D12RHI_ShouldForceCompatibility())
		{
			UE_LOGF(LogD3D12RHI, Log, "Forcing D3D12 compatibility.");
		}
		else
		{
#if D3D12_MAX_DEVICE_INTERFACE >= 1
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice1.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device1 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 2
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice2.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device2 is supported.");
			}

			if (RootDevice1 == nullptr || RootDevice2 == nullptr)
			{
				// Note: we require Windows 1703 in FD3D12DynamicRHIModule::IsSupported()
				// If we still lack support, the user's drivers could be out of date.
				UE_LOGF(LogD3D12RHI, Fatal, "Missing full support for Direct3D 12. Please update to the latest drivers.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 3
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice3.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device3 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 4
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice4.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device4 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 5
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice5.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device5 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 6
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice6.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device6 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 7
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice7.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device7 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 8
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice8.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device8 is supported.");

				// D3D12_HEAP_FLAG_CREATE_NOT_ZEROED is supported
				bHeapNotZeroedSupported = true;

				// D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT requires a version of D3D12 which understands the flag.
				bHeapCreateNotResidentSupported = true;
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 9
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice9.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device9 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 10
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice10.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device10 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 11
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice11.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device11 is supported.");
			}
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 12
			if (SUCCEEDED(RootDevice->QueryInterface(IID_PPV_ARGS(RootDevice12.GetInitReference()))))
			{
				UE_LOGF(LogD3D12RHI, Log, "ID3D12Device12 is supported.");
			}
#endif

#if ENABLE_RESIDENCY_MANAGEMENT
			// If the OS does not understand D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT (pre-Windows 10 20H1) resources cannot be created evicted. 
			if (!bHeapCreateNotResidentSupported)
			{
				UE_LOGF(LogD3D12RHI, Log, "D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT is not supported (requires ID3D12Device8); forcing resources to start resident.");
				GD3D12StartResourceResident = true;
			}
#endif // ENABLE_RESIDENCY_MANAGEMENT

#if D3D12_MAX_FEATURE_OPTIONS >= 19
			D3D12_FEATURE_DATA_D3D12_OPTIONS19 Features19{};
			if (SUCCEEDED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS19, &Features19, sizeof(Features19))))
			{
				MaxNonSamplerDescriptors = Features19.MaxViewDescriptorHeapSize;
				MaxSamplerDescriptors = Features19.MaxSamplerDescriptorHeapSizeWithStaticSamplers;
			}
			else
#endif
			if (GetResourceBindingTier() == D3D12_RESOURCE_BINDING_TIER_1)
			{
				MaxNonSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_1;
				MaxSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;
			}
			else if (GetResourceBindingTier() == D3D12_RESOURCE_BINDING_TIER_2)
			{
				MaxNonSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_2;
				MaxSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;
			}
			else
			{
				MaxNonSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_2;
				MaxSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;
			}
			
			if (GetResourceBindingTier() == D3D12_RESOURCE_BINDING_TIER_1)
			{
				GRHIGlobals.MaxSimultaneousUAVs = Desc.MaxSupportedFeatureLevel > D3D_FEATURE_LEVEL_11_0 ? MAX_UAVS : 8;
			}
			else
			{
				GRHIGlobals.MaxSimultaneousUAVs = MAX_UAVS;
			}

			const bool bRenderDocPresent = D3D12RHI_IsRenderDocPresent(RootDevice);

			// From: https://microsoft.github.io/DirectX-Specs/d3d/HLSL_SM_6_6_DynamicResources.html
			//     ResourceDescriptorHeap/SamplerDescriptorHeap must be supported on devices that support both D3D12_RESOURCE_BINDING_TIER_3 and D3D_SHADER_MODEL_6_6
			if (GetHighestShaderModel() >= D3D_SHADER_MODEL_6_6 && GetResourceBindingTier() >= D3D12_RESOURCE_BINDING_TIER_3)
			{
				GRHIGlobals.bSupportsBindless = (GMaxRHIFeatureLevel > ERHIFeatureLevel::SM5);
				GRHIGlobals.bSupportsDescriptorRange = GRHIGlobals.bSupportsBindless;
				UE_LOGF(LogD3D12RHI, Log, "Bindless is supported. MaxNonSamplerDescriptors: %d, MaxSamplerDescriptors: %d", MaxNonSamplerDescriptors, MaxSamplerDescriptors);
			}

			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS Features{};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS, &Features, sizeof(Features));

				GRHISupportsStencilRefFromPixelShader = (Features.PSSpecifiedStencilRefSupported != 0);
				GRHISupportsRasterOrderViews = (Features.ROVsSupported != 0);

				UE_LOGF(LogD3D12RHI, Log, "Stencil ref from pixel shader is %ls", GRHISupportsStencilRefFromPixelShader ? TEXT("supported") : TEXT("not supported"));
				UE_LOGF(LogD3D12RHI, Log, "Raster order views are %ls", GRHISupportsRasterOrderViews ? TEXT("supported") : TEXT("not supported"));
			}

			// Detect availability of shader model 6.0 wave operations
			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS1 Features{};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS1, &Features, sizeof(Features));
				GRHISupportsWaveOperations = Features.WaveOps;
				GRHIMinimumWaveSize = Features.WaveLaneCountMin;
				GRHIMaximumWaveSize = Features.WaveLaneCountMax;
				if (GRHISupportsWaveOperations)
				{
					UE_LOGF(LogD3D12RHI, Log, "Wave Operations are supported (wave size: min=%d max=%d).", GRHIMinimumWaveSize, GRHIMaximumWaveSize);
					int32 MaximumWaveSizeLimit;
					if (FParse::Value(FCommandLine::Get(), TEXT("-wavesizelimit="), MaximumWaveSizeLimit))
					{
						if (MaximumWaveSizeLimit > GRHIMinimumWaveSize)
						{
							if (MaximumWaveSizeLimit < GRHIMaximumWaveSize)
							{
								UE_LOGF(LogD3D12RHI, Log, "Wave Operations limited from %u to %d by -wavesizelimit", Features.WaveLaneCountMax, MaximumWaveSizeLimit);
								GRHIMaximumWaveSize = MaximumWaveSizeLimit;
							}
							else
							{
								UE_LOGF(LogD3D12RHI, Warning, "Device WaveLaneCountMax %u is less than or equal to -wavesizelimit of %d so the limit was not applied.", Features.WaveLaneCountMax, MaximumWaveSizeLimit);
							}
						}
						else
						{
							UE_LOGF(LogD3D12RHI, Warning, "Device WaveLaneCountMin %u is greater than or equal to -wavesizelimit of %d so the limit was not applied.", Features.WaveLaneCountMin, MaximumWaveSizeLimit);
						}
					}
				}
			}

			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS3 Features{};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS3, &Features, sizeof(Features));
				GRHIGlobals.SupportsBarycentricsSemantic = Features.BarycentricsSupported;
			}

			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS4 Features{};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS4, &Features, sizeof(Features));
				GRHIGlobals.SupportsNative16BitOps = Features.Native16BitShaderOpsSupported;
			}

			bool bEnhancedBarriersSupported = false;
#if D3D12RHI_SUPPORTS_ENHANCED_BARRIERS
			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS12 options12 = {};
				if (SUCCEEDED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS12, &options12, sizeof(options12))))
				{
					bEnhancedBarriersSupported = options12.EnhancedBarriersSupported;
				}
			}
#endif
			PreferredBarrierImplType =
				[bEnhancedBarriersSupported]()
				{
					const ED3D12BarrierImplementationType CVarImplType =
						GetImplementationTypeFromCVarValue(GPreferredBarrierImplementation);
					switch (CVarImplType)
					{
					case ED3D12BarrierImplementationType::Invalid:
						return bEnhancedBarriersSupported ?
								ED3D12BarrierImplementationType::Enhanced :
								ED3D12BarrierImplementationType::Legacy;
					default:
						return CVarImplType;
					}
				}();

			const auto BoolToYesNo = [](bool bValue) { return bValue ? TEXT("Yes") : TEXT("No"); };
			UE_LOGF(LogD3D12RHI, Log, "     Legacy Barriers Present in Build: %ls", BoolToYesNo(!!D3D12RHI_SUPPORTS_LEGACY_BARRIERS));
			UE_LOGF(LogD3D12RHI, Log, "   Enhanced Barriers Present in Build: %ls", BoolToYesNo(!!D3D12RHI_SUPPORTS_ENHANCED_BARRIERS));
			UE_LOGF(LogD3D12RHI, Log, "     Enhanced Barriers Driver Support: %ls", BoolToYesNo(bEnhancedBarriersSupported));
			UE_LOGF(LogD3D12RHI, Log, "Preferred Barrier Implementation Type: %ls", GetImplementationTypeString(PreferredBarrierImplType));

			check(!Barriers);
			Barriers = TUniquePtr<FD3D12BarriersFactory::BarriersForAdapterType>(FD3D12BarriersFactory::CreateBarriersForAdapter(PreferredBarrierImplType));
			check(Barriers);
			if (!Barriers)
			{
				UE_LOGF(LogD3D12RHI, Fatal, "Unable to find a D3D12 barrier implementation!");
			}
			UE_LOGF(LogD3D12RHI, Log, "        Chosen Barrier Implementation: %ls", Barriers->GetImplementationName());

			GRHITransitionPrivateData_SizeInBytes = Barriers->GetTransitionDataSizeBytes();
			GRHITransitionPrivateData_AlignInBytes = Barriers->GetTransitionDataAlignmentBytes();
			Barriers->ConfigureDevice(RootDevice, bWithD3DDebug);

#if D3D12_RHI_RAYTRACING
#if PLATFORM_WINDOWS
			D3D12_FEATURE_DATA_D3D12_OPTIONS5 D3D12Caps5 = {};
			if (SUCCEEDED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS5, &D3D12Caps5, sizeof(D3D12Caps5))))
			{
				const bool bRayTracingAllowedOnCurrentShaderPlatform = (GMaxRHIShaderPlatform == SP_PCD3D_SM6);

				if (D3D12Caps5.RaytracingTier >= D3D12_RAYTRACING_TIER_1_0
					&& GetResourceBindingTier() >= D3D12_RESOURCE_BINDING_TIER_2
					&& RootDevice5
					&& FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(GMaxRHIShaderPlatform)
					&& !FParse::Param(FCommandLine::Get(), TEXT("noraytracing")))
				{
					if (D3D12Caps5.RaytracingTier >= D3D12_RAYTRACING_TIER_1_1
						&& GRHIGlobals.bSupportsBindless
						&& RootDevice7)
					{
						if (bRayTracingAllowedOnCurrentShaderPlatform)
						{
							UE_LOGF(LogD3D12RHI, Log, "D3D12 ray tracing tier 1.1 and bindless resources are supported.");

							GRHISupportsRayTracing = RHISupportsRayTracing(GMaxRHIShaderPlatform);

							// inline raytracing requires bindless on d3d12
							const ERHIBindlessConfiguration BindlessConfiguration = UE::RHICore::GetBindlessConfigurationOnStartup(GMaxRHIShaderPlatform);
							GRHISupportsInlineRayTracing = GRHISupportsRayTracing && RHISupportsInlineRayTracing(GMaxRHIShaderPlatform) && (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6) && !IsBindlessDisabled(BindlessConfiguration);
														
							// Inline RayTracing SBT is required
							GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT = GRHISupportsInlineRayTracing;
														
							GRHISupportsRayTracingPSOAdditions = true;
							GRHISupportsRayTracingShaders = GRHISupportsRayTracing && RHISupportsRayTracingShaders(GMaxRHIShaderPlatform) && UE::RHICore::AreRayTracingShadersEnabled();
						}
						else
						{
 							UE_LOGF(LogD3D12RHI, Log, "Ray tracing is disabled because SM6 shader platform is required.");
						}
					}
					else if (GRHIGlobals.bSupportsBindless)
					{
						UE_LOGF(LogD3D12RHI, Log, "Ray tracing is disabled because bindless resources are not supported (Shader Model 6.6 and Resource Binding Tier 3 are required).");
					}
					else
					{
						UE_LOGF(LogD3D12RHI, Log, "Ray tracing is disabled because D3D12 ray tracing tier 1.1 is required but only tier 1.0 is supported.");
					}
				}
				else if (D3D12Caps5.RaytracingTier == D3D12_RAYTRACING_TIER_NOT_SUPPORTED 
					&& bRenderDocPresent
					&& !FParse::Param(FCommandLine::Get(), TEXT("noraytracing")))
				{
					UE_LOGF(LogD3D12RHI, Warning, "Ray Tracing is disabled because the RenderDoc plugin is currently not compatible with D3D12 ray tracing.");
				}
			}
#endif // PLATFORM_WINDOWS

			static auto CVarRayTracingAllowCompaction = IConsoleManager::Get().FindConsoleVariable(TEXT("r.D3D12.RayTracing.AllowCompaction"));
			GRHIGlobals.RayTracing.SupportsAccelerationStructureCompaction = CVarRayTracingAllowCompaction->GetInt() != 0;

			GRHIRayTracingAccelerationStructureAlignment = uint32(D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BYTE_ALIGNMENT);
			GRHIRayTracingScratchBufferAlignment = uint32(D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BYTE_ALIGNMENT);
			GRHIRayTracingShaderTableAlignment = uint32(D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);
			GRHIRayTracingInstanceDescriptorSize = uint32(sizeof(D3D12_RAYTRACING_INSTANCE_DESC));

#endif // D3D12_RHI_RAYTRACING

#if PLATFORM_WINDOWS
			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS7 D3D12Caps7 = {};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS7, &D3D12Caps7, sizeof(D3D12Caps7));

				D3D12_FEATURE_DATA_D3D12_OPTIONS9 D3D12Caps9 = {};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS9, &D3D12Caps9, sizeof(D3D12Caps9));

				D3D12_FEATURE_DATA_D3D12_OPTIONS11 D3D12Caps11 = {};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS11, &D3D12Caps11, sizeof(D3D12Caps11));

				if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6)
				{
					GRHISupportsMeshShadersTier0 = GRHISupportsMeshShadersTier1 = (D3D12Caps7.MeshShaderTier >= D3D12_MESH_SHADER_TIER_1);
				}

				// If SM 6.0 is supported, DXIL is the preferred shader code format for preview platforms. Multi-patform cooks can still share their shader code betweeen D3D11 and D3D12 from the DDC.
				if (Desc.MaxSupportedShaderModel >= D3D_SHADER_MODEL_6_0)
				{
					GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("DXIL");
				}
				else
				{
					// If SM 6.0 is not supported, we cannot use DXIL and therefore have to use the same code format as D3D11, which is DXBC.
					// In this case, preview shaders can potentially be shared between D3D11 and D3D12 just like they can be shared in a cook for standard (i.e. non-preview) shaders.
					GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("DXBC");
				}

				if (D3D12Caps7.MeshShaderTier >= D3D12_MESH_SHADER_TIER_1)
				{
					UE_LOGF(LogD3D12RHI, Log, "Mesh shader tier 1.0 is supported");
				}

				if (D3D12Caps9.AtomicInt64OnTypedResourceSupported)
				{
					UE_LOGF(LogD3D12RHI, Log, "AtomicInt64OnTypedResource is supported");
					GRHISupportsAtomicUInt64 = true;
				}

				if (D3D12Caps9.AtomicInt64OnGroupSharedSupported)
				{
					UE_LOGF(LogD3D12RHI, Log, "AtomicInt64OnGroupShared is supported");
				}

				if (D3D12Caps11.AtomicInt64OnDescriptorHeapResourceSupported)
				{
					UE_LOGF(LogD3D12RHI, Log, "AtomicInt64OnDescriptorHeapResource is supported");
				}

				if (D3D12Caps9.AtomicInt64OnTypedResourceSupported && D3D12Caps11.AtomicInt64OnDescriptorHeapResourceSupported)
				{
					GRHISupportsDX12AtomicUInt64 = true;
				}

				if (GRHISupportsDX12AtomicUInt64)
				{
					UE_LOGF(LogD3D12RHI, Log, "Shader Model 6.6 atomic64 is supported");
				}
				else
				{
					UE_LOGF(LogD3D12RHI, Log, "Shader Model 6.6 atomic64 is not supported");
				}

#if D3D12_MAX_FEATURE_OPTIONS >= 21
				D3D12_FEATURE_DATA_D3D12_OPTIONS21 D3D12Caps21 = {};
				RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS21, &D3D12Caps21, sizeof(D3D12Caps21));

#if D3D12_RHI_WORKGRAPHS
				if (D3D12Caps21.WorkGraphsTier != D3D12_WORK_GRAPHS_TIER_NOT_SUPPORTED)
				{
					UE_LOGF(LogD3D12RHI, Log, "Work Graphs are supported");
					GRHIGlobals.SupportsShaderWorkGraphsTier1 = true;
				}
#endif // D3D12_RHI_WORKGRAPHS
#if D3D12_RHI_WORKGRAPHS_GRAPHICS
				if (D3D12Caps21.WorkGraphsTier >= D3D12_WORK_GRAPHS_TIER_1_1)
				{
					UE_LOGF(LogD3D12RHI, Log, "Work Graphs Tier 1.1 is supported");
					GRHIGlobals.SupportsShaderWorkGraphsTier1_1 = true;
				}
#endif // D3D12_RHI_WORKGRAPHS_GRAPHICS
#endif // D3D12_MAX_FEATURE_OPTIONS

				GRHIGlobals.SupportsShaderTimestamp = IsRHIDeviceNVIDIA();
			}
#endif // PLATFORM_WINDOWS
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("DisableAsyncCompute")))
		{
			GSupportsEfficientAsyncCompute = false;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("ForceAsyncCompute")))
		{
			GSupportsEfficientAsyncCompute = true;
		}
		else if (!GSupportsEfficientAsyncCompute && GAllowAsyncCompute && GRHISupportsParallelRHIExecute)
		{
			if (IsRHIDeviceAMD())
			{
				GSupportsEfficientAsyncCompute = true;
			}
#if INTEL_EXTENSIONS
			else if (IsRHIDeviceIntel())
			{
				bool bOutSupported{};
				if (TryCheckIntelAsyncCompute(RootDevice, GRHIDeviceId, bOutSupported) && bOutSupported)
				{
					GSupportsEfficientAsyncCompute = true;
				}
			}
#endif
#if PLATFORM_WINDOWS
			else if (IsRHIDeviceNVIDIA())
			{
				D3D12_FEATURE_DATA_D3D12_OPTIONS6 D3D12Caps6{};
				HRESULT Options6HR = RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS6, &D3D12Caps6, sizeof(D3D12Caps6));

				// Allow async compute by default on NVIDIA cards which support PerPrimitiveShadingRateSupportedWithViewportIndexing
				// this should be a good metric according to NVIDIA itself (this is set for Ampere and newer cards)
				if (Options6HR == S_OK && D3D12Caps6.PerPrimitiveShadingRateSupportedWithViewportIndexing)
				{
					GSupportsEfficientAsyncCompute = true;
				}
			}
#endif
		}

		if (bWithD3DDebug)
		{
			CreateD3DInfoQueue();
		}

		if (!FPlatformMemory::SupportsFastVRAMMemory() && GSupportsEfficientAsyncCompute)
		{
			GRHIGlobals.SupportsAsyncComputeTransientAliasing = true;
		}

#if PLATFORM_WINDOWS
		D3D12_FEATURE_DATA_D3D12_OPTIONS2 D3D12Caps2 = {};
		if (FAILED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS2, &D3D12Caps2, sizeof(D3D12Caps2))))
		{
			D3D12Caps2.DepthBoundsTestSupported = false;
			D3D12Caps2.ProgrammableSamplePositionsTier = D3D12_PROGRAMMABLE_SAMPLE_POSITIONS_TIER_NOT_SUPPORTED;
		}
		bDepthBoundsTestSupported = !!D3D12Caps2.DepthBoundsTestSupported;
#endif

		D3D12_FEATURE_DATA_ROOT_SIGNATURE D3D12RootSignatureCaps = {};
		D3D12RootSignatureCaps.HighestVersion = D3D_ROOT_SIGNATURE_VERSION_1_1;	// This is the highest version we currently support. If CheckFeatureSupport succeeds, the HighestVersion returned will not be greater than this.
		if (FAILED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_ROOT_SIGNATURE, &D3D12RootSignatureCaps, sizeof(D3D12RootSignatureCaps))))
		{
			D3D12RootSignatureCaps.HighestVersion = D3D_ROOT_SIGNATURE_VERSION_1_0;
		}
		RootSignatureVersion = D3D12RootSignatureCaps.HighestVersion;

		{
			D3D12_FEATURE_DATA_D3D12_OPTIONS3 D3D12Caps3 = {};
			if (FAILED(RootDevice->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS3, &D3D12Caps3, sizeof(D3D12Caps3))))
			{
				D3D12Caps3.CopyQueueTimestampQueriesSupported = false;
			}

			// @todo - fix copy-queue timestamps. 
			// ResolveQueryData() can only be called on graphics/compute command lists, since it internally dispatches a compute shader to do the resolve work, but the submission thread is calling this on a copy command list.
			// This leads to D3DDebug errors claiming SetComputeRootSignature was called on a copy command list.
			bCopyQueueTimestampQueriesSupported = false;//!!D3D12Caps3.CopyQueueTimestampQueriesSupported;
		}

#if TRACK_RESOURCE_ALLOCATIONS
		// Set flag if we want to track all allocations - comes with some overhead and only possible when Tier 2 is available
		// (because we will create placed buffers for texture allocation to retrieve the GPU virtual addresses)
		const bool bTraceMemAlloc = UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel);
		bTrackAllAllocation = (GD3D12TrackAllAllocations || UE::RHI::UseGPUCrashDebugging() || bTraceMemAlloc) && (GetResourceHeapTier() == D3D12_RESOURCE_HEAP_TIER_2);
#endif

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		ERHIBindlessConfiguration BindlessConfig = ERHIBindlessConfiguration::Disabled;

		if (Desc.MaxSupportedFeatureLevel >= D3D_FEATURE_LEVEL_12_0 && Desc.MaxSupportedShaderModel >= D3D_SHADER_MODEL_6_6 && Desc.ResourceBindingTier >= D3D12_RESOURCE_BINDING_TIER_3)
		{
			// Needs to happen before device creation below
			BindlessDescriptorAllocator.Init();

			BindlessConfig = BindlessDescriptorAllocator.GetConfiguration();

			GRHIGlobals.ShaderBundles.RequiresSharedBindlessParameters = (BindlessConfig > ERHIBindlessConfiguration::RayTracing);
		}
#endif

		// Context redirectors allow RHI commands to be executed on multiple GPUs at the
		// same time in a multi-GPU system. Redirectors have a physical mask for the GPUs
		// they can support and an active mask which restricts commands to operate on a
		// subset of the physical GPUs. The default context redirectors used by the
		// immediate command list can support all physical GPUs, whereas context containers
		// used by the parallel command lists might only support a subset of GPUs in the
		// system.
		DefaultContextRedirector.SetPhysicalGPUMask(FRHIGPUMask::All());

		// Create all of the FD3D12Devices.
		for (uint32 GPUIndex : FRHIGPUMask::All())
		{
			Devices[GPUIndex] = new FD3D12Device(FRHIGPUMask::FromIndex(GPUIndex), this);
			Devices[GPUIndex]->SetupAfterDeviceCreation();

			// The redirectors allow to broadcast to any GPU set
			DefaultContextRedirector.SetPhysicalContext(&Devices[GPUIndex]->GetDefaultCommandContext());
		}

		FrameFence = MakeUnique<FD3D12ManualFence>(this);

		const FString Name(L"Upload Buffer Allocator");
		for (uint32 GPUIndex : FRHIGPUMask::All())
		{
			// Safe to init as we have a device;
			UploadHeapAllocator[GPUIndex] = new FD3D12UploadHeapAllocator(this,	Devices[GPUIndex], Name);
			UploadHeapAllocator[GPUIndex]->Init();
		}


		// ID3D12Device1::CreatePipelineLibrary() requires each blob to be specific to the given adapter. To do this we create a unique file name with from the adpater desc. 
		// Note that : "The uniqueness of an LUID is guaranteed only until the system is restarted" according to windows doc and thus can not be reused.
		const FString UniqueDeviceCachePath = FString::Printf(TEXT("V%d_D%d_S%d_R%d.ushaderprecache"), Desc.Desc.VendorId, Desc.Desc.DeviceId, Desc.Desc.SubSysId, Desc.Desc.Revision);
		FString GraphicsCacheFile = PIPELINE_STATE_FILE_LOCATION / FString::Printf(TEXT("D3DGraphics_%s"), *UniqueDeviceCachePath);
	    FString ComputeCacheFile = PIPELINE_STATE_FILE_LOCATION / FString::Printf(TEXT("D3DCompute_%s"), *UniqueDeviceCachePath);
		FString DriverBlobFilename = PIPELINE_STATE_FILE_LOCATION / FString::Printf(TEXT("D3DDriverByteCodeBlob_%s"), *UniqueDeviceCachePath);

		PipelineStateCache.Init(GraphicsCacheFile, ComputeCacheFile, DriverBlobFilename);
		PipelineStateCache.RebuildFromDiskCache();

#if USE_STATIC_ROOT_SIGNATURE
		EShaderBindingLayoutFlags GraphicsFlags{};
#if PLATFORM_SUPPORTS_MESH_SHADERS
		EnumAddFlags(GraphicsFlags, EShaderBindingLayoutFlags::AllowMeshShaders);
#endif

		StaticGraphicsRootSignature.InitStaticGraphicsRootSignature(GraphicsFlags);
		StaticGraphicsWithConstantsRootSignature.InitStaticGraphicsRootSignature(GraphicsFlags | EShaderBindingLayoutFlags::RootConstants);
		StaticComputeRootSignature.InitStaticComputeRootSignatureDesc(GraphicsFlags);
		StaticComputeWithConstantsRootSignature.InitStaticComputeRootSignatureDesc(GraphicsFlags | EShaderBindingLayoutFlags::RootConstants);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (BindlessConfig > ERHIBindlessConfiguration::RayTracing)
		{
			EnumAddFlags(GraphicsFlags, EShaderBindingLayoutFlags::BindlessResources);
			EnumAddFlags(GraphicsFlags, EShaderBindingLayoutFlags::BindlessSamplers);

			StaticBindlessGraphicsRootSignature.InitStaticGraphicsRootSignature(GraphicsFlags);
			StaticBindlessGraphicsWithConstantsRootSignature.InitStaticGraphicsRootSignature(GraphicsFlags | EShaderBindingLayoutFlags::RootConstants);
			StaticBindlessComputeRootSignature.InitStaticComputeRootSignatureDesc(GraphicsFlags);
			StaticBindlessComputeWithConstantsRootSignature.InitStaticComputeRootSignatureDesc(GraphicsFlags | EShaderBindingLayoutFlags::RootConstants);
		}
#endif

#if D3D12_RHI_RAYTRACING
		EShaderBindingLayoutFlags RayTracingFlags{};

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (BindlessConfig != ERHIBindlessConfiguration::Disabled)
		{
			EnumAddFlags(RayTracingFlags, EShaderBindingLayoutFlags::BindlessResources);
			EnumAddFlags(RayTracingFlags, EShaderBindingLayoutFlags::BindlessSamplers);
		}
#endif

		if (RHIGetStaticShaderBindingLayoutSupport(GMaxRHIShaderPlatform) != ERHIStaticShaderBindingLayoutSupport::Unsupported)
		{
			check(EnumHasAllFlags(RayTracingFlags, EShaderBindingLayoutFlags::BindlessResources | EShaderBindingLayoutFlags::BindlessSamplers))
			EnumAddFlags(RayTracingFlags, EShaderBindingLayoutFlags::ShaderBindingLayoutUsed);
		}

		StaticRayTracingGlobalRootSignature.InitStaticRayTracingGlobalRootSignatureDesc(RayTracingFlags);
		StaticRayTracingLocalRootSignature.InitStaticRayTracingLocalRootSignatureDesc(RayTracingFlags);
#endif
#endif // USE_STATIC_ROOT_SIGNATURE

		// Creating command signatures relies on static ray tracing root signatures.
		CreateCommandSignatures();
	}
}
void FD3D12Adapter::InitializeExplicitDescriptorHeap()
{
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		Devices[GPUIndex]->InitExplicitDescriptorHeap();
	}
}

void FD3D12Adapter::InitializeRayTracing()
{
#if D3D12_RHI_RAYTRACING
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		if (Devices[GPUIndex]->GetParentAdapter()->GetD3DDevice5())
		{
			Devices[GPUIndex]->InitRayTracing();
		}
	}
#endif // D3D12_RHI_RAYTRACING
}

void FD3D12Adapter::CreateCommandSignatures()
{
	ID3D12Device* Device = GetD3DDevice();

	// ExecuteIndirect command signatures
	D3D12_COMMAND_SIGNATURE_DESC commandSignatureDesc = {};
	commandSignatureDesc.NumArgumentDescs = 1;
	commandSignatureDesc.ByteStride = 20;
	commandSignatureDesc.NodeMask = FRHIGPUMask::All().GetNative();

	D3D12_INDIRECT_ARGUMENT_DESC indirectParameterDesc[1] = {};
	commandSignatureDesc.pArgumentDescs = indirectParameterDesc;

	indirectParameterDesc[0].Type = D3D12_INDIRECT_ARGUMENT_TYPE_DRAW;
	commandSignatureDesc.ByteStride = sizeof(D3D12_DRAW_ARGUMENTS);
	VERIFYD3D12RESULT(Device->CreateCommandSignature(&commandSignatureDesc, nullptr, IID_PPV_ARGS(DrawIndirectCommandSignature.GetInitReference())));

	indirectParameterDesc[0].Type = D3D12_INDIRECT_ARGUMENT_TYPE_DRAW_INDEXED;
	commandSignatureDesc.ByteStride = sizeof(D3D12_DRAW_INDEXED_ARGUMENTS);
	VERIFYD3D12RESULT(Device->CreateCommandSignature(&commandSignatureDesc, nullptr, IID_PPV_ARGS(DrawIndexedIndirectCommandSignature.GetInitReference())));

	checkf(DispatchIndirectGraphicsCommandSignature.IsValid(), TEXT("Indirect graphics dispatch command signature is expected to be created by platform-specific D3D12 adapter implementation."))
	checkf(DispatchIndirectComputeCommandSignature.IsValid(), TEXT("Indirect compute dispatch command signature is expected to be created by platform-specific D3D12 adapter implementation."))
}

void FD3D12Adapter::CleanupResources()
{
	for (auto& Viewport : Viewports)
	{
		Viewport->IssueFrameEvent();
		Viewport->WaitForFrameEventCompletion();
	}

	TransientMemoryCache = nullptr;

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		Devices[GPUIndex]->CleanupResources();
	}
}

FD3D12Adapter::~FD3D12Adapter()
{
	// Release allocation data of all thread local transient uniform buffer allocators
	for (FTransientUniformBufferAllocator* Allocator : TransientUniformBufferAllocators)
	{
		Allocator->Cleanup();
	}
	TransientUniformBufferAllocators.Empty();

	Viewports.Empty();

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		if (UploadHeapAllocator[GPUIndex])
		{
			UploadHeapAllocator[GPUIndex]->Destroy();
			delete(UploadHeapAllocator[GPUIndex]);
			UploadHeapAllocator[GPUIndex] = nullptr;
		}
	}

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		delete(Devices[GPUIndex]);
		Devices[GPUIndex] = nullptr;
	}

	FrameFence = {};

	TransientMemoryCache.Reset();

	if (RootDevice)
	{
		PipelineStateCache.Close();
	}
	RootSignatureManager.Destroy();

	DrawIndirectCommandSignature.SafeRelease();
	DrawIndexedIndirectCommandSignature.SafeRelease();
	DispatchIndirectGraphicsCommandSignature.SafeRelease();
	DispatchIndirectComputeCommandSignature.SafeRelease();
	DispatchRaysIndirectCommandSignature.SafeRelease();

#if D3D12_SUPPORTS_DXGI_DEBUG
	// trace all leak D3D resource
	if (DXGIDebug != nullptr)
	{
		DXGIDebug->ReportLiveObjects(
			GUID{ 0xe48ae283, 0xda80, 0x490b, { 0x87, 0xe6, 0x43, 0xe9, 0xa9, 0xcf, 0xda, 0x8 } }, // DXGI_DEBUG_ALL
			DXGI_DEBUG_RLO_FLAGS(DXGI_DEBUG_RLO_DETAIL | DXGI_DEBUG_RLO_IGNORE_INTERNAL));
		DXGIDebug.SafeRelease();

#if D3D12_SUPPORTS_INFO_QUEUE
		CheckD3DStoredMessages(RootDevice);

		if (GD3D12CallBackCookie != 0)
		{
			ID3D12InfoQueue1* pd3dInfoQueue1 = nullptr;
			VERIFYD3D12RESULT(RootDevice->QueryInterface(__uuidof(ID3D12InfoQueue1), (void**)&pd3dInfoQueue1));
			if (pd3dInfoQueue1)
			{
				pd3dInfoQueue1->UnregisterMessageCallback(GD3D12CallBackCookie);
				GD3D12CallBackCookie = 0;
			}
		}
#endif
	}
#endif

#if PLATFORM_WINDOWS
	if (GetD3DDevice() && GRHIGlobals.IsDebugLayerEnabled)
	{
		TRefCountPtr<ID3D12DebugDevice> Debug;
		if (SUCCEEDED(GetD3DDevice()->QueryInterface(IID_PPV_ARGS(Debug.GetInitReference()))))
		{
			D3D12_RLDO_FLAGS rldoFlags = D3D12_RLDO_DETAIL;
			Debug->ReportLiveDeviceObjects(rldoFlags);
		}
	}

	if (ExceptionHandlerHandle != INVALID_HANDLE_VALUE)
	{
		RemoveVectoredExceptionHandler(ExceptionHandlerHandle);
	}
#endif

#if D3D12_SUPPORTS_DXGI_DEBUG
	FPlatformProcess::FreeDllHandle(DxgiDebugDllHandle);
#endif
}

void FD3D12Adapter::CreateDXGIFactory(TRefCountPtr<IDXGIFactory2>& DxgiFactory2, bool bWithDebug)
{
#if PLATFORM_WINDOWS
	const uint32 Flags = bWithDebug ? DXGI_CREATE_FACTORY_DEBUG : 0;
	VERIFYD3D12RESULT(CreateDXGIFactory2(Flags, IID_PPV_ARGS(DxgiFactory2.GetInitReference())));
#endif // PLATFORM_WINDOWS
}

void FD3D12Adapter::CreateDXGIFactory(bool bWithDebug)
{
#if PLATFORM_WINDOWS
	CreateDXGIFactory(DxgiFactory2, bWithDebug);

	InitDXGIFactoryVariants(DxgiFactory2);
#endif // PLATFORM_WINDOWS
}

void FD3D12Adapter::InitDXGIFactoryVariants(IDXGIFactory2* InDxgiFactory2)
{
#if DXGI_MAX_FACTORY_INTERFACE >= 3
	InDxgiFactory2->QueryInterface(IID_PPV_ARGS(DxgiFactory3.GetInitReference()));
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 4
	InDxgiFactory2->QueryInterface(IID_PPV_ARGS(DxgiFactory4.GetInitReference()));
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 5
	InDxgiFactory2->QueryInterface(IID_PPV_ARGS(DxgiFactory5.GetInitReference()));
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 6
	InDxgiFactory2->QueryInterface(IID_PPV_ARGS(DxgiFactory6.GetInitReference()));
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 7
	InDxgiFactory2->QueryInterface(IID_PPV_ARGS(DxgiFactory7.GetInitReference()));
#endif
}

HRESULT FD3D12Adapter::EnumAdapters(IDXGIAdapter** TempAdapter) const
{
#if DXGI_MAX_FACTORY_INTERFACE >= 6
	return FD3D12AdapterDesc::EnumAdapters(Desc.AdapterIndex, Desc.GpuPreference, DxgiFactory2, DxgiFactory6, TempAdapter);
#else
	return DxgiFactory2->EnumAdapters(Desc.AdapterIndex, TempAdapter);
#endif
}

void FD3D12Adapter::EndFrame()
{
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		uint64 FrameLag = 20;
		GetUploadHeapAllocator(GPUIndex).CleanUpAllocations(FrameLag);
	}

	if (TransientMemoryCache)
	{
		TransientMemoryCache->GarbageCollect();
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		GetDevice(GPUIndex)->GetBindlessDescriptorManager().GarbageCollect();
	}
#endif

#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS); 

	// remove tracked released resources older than n amount of frames
	int32 ReleaseCount = 0;
	uint64 CurrentFrameID = GetFrameFence().GetNextFenceToSignal();
	for (; ReleaseCount < ReleasedAllocationData.Num(); ++ReleaseCount)
	{
		if (ReleasedAllocationData[ReleaseCount].ReleasedFrameID + GTrackedReleasedAllocationFrameRetention > CurrentFrameID)
		{
			break;
		}
	}
	if (ReleaseCount > 0)
	{
		ReleasedAllocationData.RemoveAt(0, ReleaseCount, EAllowShrinking::No);
	}
#endif
}

FD3D12FastConstantAllocator& FD3D12Adapter::GetTransientUniformBufferAllocator()
{
	// Multi-GPU support : is using device 0 always appropriate here?
	return FTransientUniformBufferAllocator::Get([this]() -> FTransientUniformBufferAllocator*
	{
		FTransientUniformBufferAllocator* Alloc = new FTransientUniformBufferAllocator(this, Devices[0], FRHIGPUMask::All());

		// Register so the underlying resource location can be freed during adapter cleanup instead of when thread local allocation is destroyed
		{
			FScopeLock Lock(&TransientUniformBufferAllocatorsCS);
			TransientUniformBufferAllocators.Add(Alloc);
		}

		return Alloc;
	});
}

void FD3D12Adapter::ReleaseTransientUniformBufferAllocator(FTransientUniformBufferAllocator* InAllocator)
{
	FScopeLock Lock(&TransientUniformBufferAllocatorsCS);
	verify(TransientUniformBufferAllocators.Remove(InAllocator) == 1);
}

const FRHIMemoryStats& FD3D12Adapter::CollectMemoryStats()
{
#if PLATFORM_WINDOWS
	const uint64 UpdateFrame = FrameFence != nullptr ? FrameFence->GetNextFenceToSignal() : 0;

	// Avoid spurious query calls if we have already captured stats this frame.
	if (MemoryStatsUpdateFrame == UpdateFrame)
	{
		return MemoryStats;
	}
	MemoryStatsUpdateFrame = UpdateFrame;

	if (FAILED(UE::DXGIUtilities::GetD3DMemoryStats(GetAdapter(), MemoryStats)))
	{
		return MemoryStats;
	}

	// Update global RHI state (for warning output, etc.)
	GDemotedLocalMemorySize = MemoryStats.DemotedLocal;

#if ENABLE_RESIDENCY_MANAGEMENT
	// D3D12 residency manager will only evict resources when the resident set is larger than DXGI reported budget.
	// However, the DXGI reports high budget even when multiple applications use large amounts of VRAM.
	// This causes VidMm to automatically page out allocations out of VRAM based on its own heuristics, which can cause significant 
	// performance degradation when paged-out resources are used for rendering before VidMm pages them back in (which can take a long time).
	// By overriding the budget to 0 and stopping rendering at the high-level, we can immediately free VRAM and avoid VidMm paging.
	const bool bEvictResidentResources = GEnableResidencyManagement && GD3D12EvictAllResidentResourcesInBackground && !FApp::HasFocus();
	uint64 LocalMemoryBudgetLimit = bEvictResidentResources ? 0 : MemoryStats.BudgetLocal;

	// Debug override: force a specific budget to stress-test residency management.
	if (GD3D12ResidencyDebugBudgetMB >= 0)
	{
		LocalMemoryBudgetLimit = (uint64)GD3D12ResidencyDebugBudgetMB * 1024 * 1024;
	}

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		GetDevice(GPUIndex)->GetResidencyManager().SetLocalMemoryBudgetLimit(LocalMemoryBudgetLimit);
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT
#endif // PLATFORM_WINDOWS
	return MemoryStats;
}

void FD3D12Adapter::BlockUntilIdle()
{
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		GetDevice(GPUIndex)->BlockUntilIdle();
	}
}


void FD3D12Adapter::TrackAllocationData(FD3D12ResourceLocation* InAllocation, uint64 InAllocationSize, bool bCollectCallstack)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FTrackedAllocationData AllocationData;
	AllocationData.ResourceAllocation = InAllocation;
	AllocationData.AllocationSize = InAllocationSize;
	if (bCollectCallstack)
	{
		AllocationData.StackDepth = FPlatformStackWalk::CaptureStackBackTrace(&AllocationData.Stack[0], FTrackedAllocationData::MaxStackDepth);
	}
	else 
	{
		AllocationData.StackDepth = 0;
	}

	FScopeLock Lock(&TrackedAllocationDataCS);
	check(!TrackedAllocationData.Contains(InAllocation));
	TrackedAllocationData.Add(InAllocation, AllocationData);
#endif
}

void FD3D12Adapter::ReleaseTrackedAllocationData(FD3D12ResourceLocation* InAllocation, bool bDefragFree)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);

	D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = InAllocation->GetGPUVirtualAddress();
	if (GPUAddress != 0)
	{
		FReleasedAllocationData ReleasedData;
		ReleasedData.GPUVirtualAddress = GPUAddress;
		ReleasedData.AllocationSize = InAllocation->GetSize();
		ReleasedData.ResourceName = InAllocation->GetResource()->GetName();
		ReleasedData.ResourceDesc = InAllocation->GetResource()->GetDesc();
		ReleasedData.ReleasedFrameID = GetFrameFence().GetNextFenceToSignal();
		ReleasedData.bDefragFree = bDefragFree;
		ReleasedData.bBackBuffer = InAllocation->GetResource()->IsBackBuffer();
		ReleasedData.bTransient = InAllocation->IsTransient();
		ReleasedAllocationData.Add(ReleasedData);
	}

	verify(TrackedAllocationData.Remove(InAllocation) == 1);
#endif
}


void FD3D12Adapter::TrackHeapAllocation(FD3D12Heap* InHeap)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);
	check(!TrackedHeaps.Contains(InHeap));
	TrackedHeaps.Add(InHeap);
#endif
}

void FD3D12Adapter::ReleaseTrackedHeap(FD3D12Heap* InHeap)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);

	D3D12_GPU_VIRTUAL_ADDRESS GPUVirtualAddress = InHeap->GetGPUVirtualAddress();
	if (GPUVirtualAddress != 0 || IsTrackingAllAllocations())
	{
		FReleasedAllocationData ReleasedData;
		ReleasedData.GPUVirtualAddress	= GPUVirtualAddress;
		ReleasedData.AllocationSize		= InHeap->GetHeapDesc().SizeInBytes;
		ReleasedData.ResourceName		= InHeap->GetName();
		ReleasedData.ReleasedFrameID	= GetFrameFence().GetNextFenceToSignal();
		ReleasedData.bHeap				= true;
		ReleasedAllocationData.Add(ReleasedData);
	}

	verify(TrackedHeaps.Remove(InHeap) == 1);
#endif
}

void FD3D12Adapter::FindResourcesNearGPUAddress(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, uint64 InRange, TArray<FAllocatedResourceResult>& OutResources)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);

	TArray<FTrackedAllocationData> Allocations;
	TrackedAllocationData.GenerateValueArray(Allocations);
	FInt64Range TrackRange((int64)(InGPUVirtualAddress - InRange), (int64)(InGPUVirtualAddress + InRange));
	for (FTrackedAllocationData& AllocationData : Allocations)
	{
		D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = AllocationData.ResourceAllocation->GetResource()->GetGPUVirtualAddress();
		FInt64Range AllocationRange((int64)GPUAddress, (int64)(GPUAddress + AllocationData.AllocationSize));
		if (TrackRange.Overlaps(AllocationRange))
		{
			bool bContainsAllocation = AllocationRange.Contains(InGPUVirtualAddress);
			int64 Distance = bContainsAllocation ? 0 : ((InGPUVirtualAddress < GPUAddress) ? GPUAddress - InGPUVirtualAddress : InGPUVirtualAddress - AllocationRange.GetUpperBoundValue());
			check(Distance >= 0);

			FAllocatedResourceResult Result;
			Result.Allocation = AllocationData.ResourceAllocation;
			Result.Distance = Distance;
			OutResources.Add(Result);
		}
	}

	// Sort the resources on distance from the requested address
	Algo::Sort(OutResources, [InGPUVirtualAddress](const FAllocatedResourceResult& InLHS, const FAllocatedResourceResult& InRHS)
		{
			return InLHS.Distance < InRHS.Distance;
		});
#endif
}

void FD3D12Adapter::FindHeapsContainingGPUAddress(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, TArray<FD3D12Heap*>& OutHeaps)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);

	for (FD3D12Heap* AllocatedHeap : TrackedHeaps)
	{
		D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = AllocatedHeap->GetGPUVirtualAddress();
		FInt64Range HeapRange((int64)GPUAddress, (int64)(GPUAddress + AllocatedHeap->GetHeapDesc().SizeInBytes));
		if (HeapRange.Contains(InGPUVirtualAddress))
		{			
			OutHeaps.Add(AllocatedHeap);
		}
	}
#endif
}

void FD3D12Adapter::FindReleasedAllocationData(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, TArray<FReleasedAllocationData>& OutAllocationData)
{
#if TRACK_RESOURCE_ALLOCATIONS
	FScopeLock Lock(&TrackedAllocationDataCS);

	for (FReleasedAllocationData& AllocationData : ReleasedAllocationData)
	{
		if (InGPUVirtualAddress >= AllocationData.GPUVirtualAddress && InGPUVirtualAddress < (AllocationData.GPUVirtualAddress + AllocationData.AllocationSize))
		{
			// Add in reverse order, so last released resources at first in the array
			OutAllocationData.EmplaceAt(0, AllocationData);
		}
	}
#endif
}

#if TRACK_RESOURCE_ALLOCATIONS

static FAutoConsoleCommandWithOutputDevice GDumpTrackedD3D12AllocationsCmd(
	TEXT("D3D12.DumpTrackedAllocations"),
	TEXT("Dump all tracked d3d12 resource allocations."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
	{
		FD3D12DynamicRHI::GetD3DRHI()->GetAdapter().DumpTrackedAllocationData(OutputDevice, false, false);
	})
);

static FAutoConsoleCommandWithOutputDevice GDumpTrackedD3D12AllocationCallstacksCmd(
	TEXT("D3D12.DumpTrackedAllocationCallstacks"),
	TEXT("Dump all tracked d3d12 resource allocation callstacks."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
	{
		FD3D12DynamicRHI::GetD3DRHI()->GetAdapter().DumpTrackedAllocationData(OutputDevice, false, true);
	})
);

static FAutoConsoleCommandWithOutputDevice GDumpTrackedD3D12ResidentAllocationsCmd(
	TEXT("D3D12.DumpTrackedResidentAllocations"),
	TEXT("Dump all tracked resisdent d3d12 resource allocations."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
		{
			FD3D12DynamicRHI::GetD3DRHI()->GetAdapter().DumpTrackedAllocationData(OutputDevice, true, false);
		})
);

static FAutoConsoleCommandWithOutputDevice GDumpTrackedD3D12ResidentAllocationCallstacksCmd(
	TEXT("D3D12.DumpTrackedResidentAllocationCallstacks"),
	TEXT("Dump all tracked resident d3d12 resource allocation callstacks."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
		{
			FD3D12DynamicRHI::GetD3DRHI()->GetAdapter().DumpTrackedAllocationData(OutputDevice, true, true);
		})
);

void FD3D12Adapter::DumpTrackedAllocationData(FOutputDevice& OutputDevice, bool bResidentOnly, bool bWithCallstack)
{
	FScopeLock Lock(&TrackedAllocationDataCS);

	TArray<FTrackedAllocationData> Allocations;
	TrackedAllocationData.GenerateValueArray(Allocations);
	Allocations.Sort([](const FTrackedAllocationData& InLHS, const FTrackedAllocationData& InRHS)
		{
			return InLHS.AllocationSize > InRHS.AllocationSize;
		});

	TArray<FTrackedAllocationData> BufferAllocations;	
	TArray<FTrackedAllocationData> TextureAllocations;
	uint64 TotalAllocatedBufferSize = 0;
	uint64 TotalResidentBufferSize = 0;
	uint64 TotalAllocatedTextureSize = 0;
	uint64 TotalResidentTextureSize = 0;
	for (FTrackedAllocationData& AllocationData : Allocations)
	{
		D3D12_RESOURCE_DESC ResourceDesc = AllocationData.ResourceAllocation->GetResource()->GetDesc();
		if (ResourceDesc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
		{
			BufferAllocations.Add(AllocationData);
			TotalAllocatedBufferSize += AllocationData.AllocationSize;
			// TODO: accurately account for partially-resident resources
			TotalResidentBufferSize += (AllocationData.ResourceAllocation->GetResource()->IsResident()) ? AllocationData.AllocationSize : 0;
		}
		else
		{
			TextureAllocations.Add(AllocationData);
			TotalAllocatedTextureSize += AllocationData.AllocationSize;
			// TODO: accurately account for partially-resident resources
			TotalResidentTextureSize += (AllocationData.ResourceAllocation->GetResource()->IsResident()) ? AllocationData.AllocationSize : 0;
		}
	}

	const size_t STRING_SIZE = 16 * 1024;
	ANSICHAR StackTrace[STRING_SIZE];

	FString OutputData;
	OutputData += FString::Printf(TEXT("\n%d Tracked Texture Allocations (Total size: %4.3fMB - Resident: %4.3fMB):\n"), TextureAllocations.Num(), TotalAllocatedTextureSize / (1024.0f * 1024), TotalResidentTextureSize / (1024.0f * 1024));
	for (const FTrackedAllocationData& AllocationData : TextureAllocations)
	{
		D3D12_RESOURCE_DESC ResourceDesc = AllocationData.ResourceAllocation->GetResource()->GetDesc();

		bool bResident = true;
#if ENABLE_RESIDENCY_MANAGEMENT
		bResident = AllocationData.ResourceAllocation->GetResource()->IsResident();
#endif 
		if (!bResident && bResidentOnly)
		{
			continue;
		}

		FString Flags;
		if (EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET))
		{
			Flags += "RT";
		}
		else if (EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL))
		{
			Flags += "DS";
		}
		if (EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS))
		{
			Flags += EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL) ? "|UAV" : "UAV";
		}

		OutputData += FString::Printf(TEXT("\tName: %s - Size: %3.3fMB - Width: %" UINT64_FMT " - Height: %d - DepthOrArraySize: %d - MipLevels: %d - Flags: %s - Resident: %s\n"), 
			*AllocationData.ResourceAllocation->GetResource()->GetName().ToString(), 
			AllocationData.AllocationSize / (1024.0f * 1024),
			ResourceDesc.Width, ResourceDesc.Height, ResourceDesc.DepthOrArraySize, ResourceDesc.MipLevels,
			Flags.IsEmpty() ? TEXT("None") : *Flags,
			bResident ? TEXT("Yes") : TEXT("No"));

		if (bWithCallstack)
		{
			static uint32 EntriesToSkip = 3;
			for (uint32 Index = EntriesToSkip; Index < AllocationData.StackDepth; ++Index)
			{
				StackTrace[0] = 0;
				FPlatformStackWalk::ProgramCounterToHumanReadableString(Index, AllocationData.Stack[Index], StackTrace, STRING_SIZE, nullptr);
				OutputData += FString::Printf(TEXT("\t\t%d %s\n"), Index - EntriesToSkip, ANSI_TO_TCHAR(StackTrace));
			}
		}
	}

	OutputData += FString::Printf(TEXT("\n\n%d Tracked Buffer Allocations (Total size: %4.3fMB - Resident: %4.3fMB):\n"), BufferAllocations.Num(), TotalAllocatedBufferSize / (1024.0f * 1024), TotalResidentBufferSize / (1024.0f * 1024));
	for (const FTrackedAllocationData& AllocationData : BufferAllocations)
	{
		D3D12_RESOURCE_DESC ResourceDesc = AllocationData.ResourceAllocation->GetResource()->GetDesc();

		bool bResident = true;
#if ENABLE_RESIDENCY_MANAGEMENT
		bResident = AllocationData.ResourceAllocation->GetResource()->IsResident();
#endif 
		if (!bResident && bResidentOnly)
		{
			continue;
		}

		OutputData += FString::Printf(TEXT("\tName: %s - Size: %3.3fMB - Width: %" UINT64_FMT " - UAV: %s - Resident: %s\n"), 
			*AllocationData.ResourceAllocation->GetResource()->GetName().ToString(), 
			AllocationData.AllocationSize / (1024.0f * 1024),
			ResourceDesc.Width,
			EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) ? TEXT("Yes") : TEXT("No"),
			bResident ? TEXT("Yes") : TEXT("No"));

		if (bWithCallstack)
		{
			static uint32 EntriesToSkip = 3;
			for (uint32 Index = EntriesToSkip; Index < AllocationData.StackDepth; ++Index)
			{
				StackTrace[0] = 0;
				FPlatformStackWalk::ProgramCounterToHumanReadableString(Index, AllocationData.Stack[Index], StackTrace, STRING_SIZE, nullptr);
				OutputData += FString::Printf(TEXT("\t\t%d %s\n"), Index - EntriesToSkip, ANSI_TO_TCHAR(StackTrace));
			}
		}
	}

	OutputDevice.Log(OutputData);
}

#endif // TRACK_RESOURCE_ALLOCATIONS

void FD3D12Adapter::SetResidencyPriority(ID3D12Pageable* Pageable, D3D12_RESIDENCY_PRIORITY HeapPriority, uint32 GPUIndex, TOptional<uint64> ResourceSize)
{
#if D3D12_MAX_DEVICE_INTERFACE >= 5

	// GUID for our custom private data that holds the current priority
	checkf(GPUIndex <= 0xFF, TEXT("GPUIndex value must fit into a byte, as it's packed into private data ID"));

	FD3D12Device* NodeDevice = GetDevice(GPUIndex);
	ID3D12Device5* D3DDevice5 = NodeDevice->GetParentAdapter()->GetD3DDevice5();
	if (!D3DDevice5)
	{
		return;
	}

	const GUID DataGuid = GUID{ 0xB365961D, 0xA209, 0x4475, { 0x84, 0x9B, 0xDA, 0xFF, 0x90, 0x91, 0x2E, uint8(GPUIndex) } };

	const uint32 DataSize = uint32(sizeof(HeapPriority));
	uint32 ExistingDataSize = DataSize;
	D3D12_RESIDENCY_PRIORITY ExistingPriority = (D3D12_RESIDENCY_PRIORITY)0;

	const HRESULT Result = Pageable->GetPrivateData(DataGuid, &ExistingDataSize, &ExistingPriority);
	const bool bExistingDataValid = SUCCEEDED(Result) && ExistingDataSize == DataSize;

	uint32 SizeGroup = 0;

	if (ResourceSize.IsSet())
	{
		// To further prioritize among resources within the same coarse bucket,
		// the lowest 16-bits of the priority are set to the size of the resource divided by 10MB
		// (saturating to 0xFFFF for very large heaps). This additional prioritization
		// favors larger heaps and resources. This matches the prioritization done by default
		// for high priority buckets by windows, except here it is done for all buckets (not just high priority).
		// See https://learn.microsoft.com/en-us/windows/win32/direct3d12/residency#default-priority-algorithm
		SizeGroup = uint32(FMath::Min<uint64>((*ResourceSize >> 20u) / 10u, 0xFFFFu));
	}
	else if (bExistingDataValid)
	{
		// Keep the previously set resource size bits if not explicitly provided
		SizeGroup = uint32(ExistingPriority) & 0xFFFFu;
	}

	HeapPriority = D3D12_RESIDENCY_PRIORITY((uint32(HeapPriority) & 0xFFFF0000u) | SizeGroup);
	
	if (!bExistingDataValid || ExistingPriority != HeapPriority)
	{
		D3DDevice5->SetResidencyPriority(1, &Pageable, &HeapPriority);
		Pageable->SetPrivateData(DataGuid, DataSize, &HeapPriority);
	}
#endif // D3D12_MAX_DEVICE_INTERFACE >= 5
}

void FD3D12Adapter::CreateTransition(
	FRHITransition* Transition,
	const FRHITransitionCreateInfo& CreateInfo)
{
	check(Barriers);
	return Barriers->CreateTransition(Transition, CreateInfo);
}

void FD3D12Adapter::ReleaseTransition(FRHITransition* Transition)
{
	check(Barriers);
	return Barriers->ReleaseTransition(Transition);
}

TUniquePtr<FD3D12BarriersFactory::BarriersForContextType> FD3D12Adapter::CreateBarriersForContext()
{
	return TUniquePtr<FD3D12BarriersFactory::BarriersForContextType>(
		FD3D12BarriersFactory::CreateBarriersForContext(PreferredBarrierImplType));
}
