// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Adapter.h: D3D12 Adapter Interfaces

	The D3D12 RHI is laid out in the following structure. 

		[Engine]--
				|
				|-[RHI]--
						|
						|-[Adapter]-- (LDA)
						|			|
						|			|- [Device]
						|			|
						|			|- [Device]
						|
						|-[Adapter]--
									|
									|- [Device]--
												|
												|-[Queue]
												|
												|-[Queue]

	Under this scheme an FD3D12Device represents 1 node belonging to 1 physical adapter.

	This structure allows a single RHI to control several different hardware setups. Some example arrangements:
		- Single-GPU systems (the common case)
		- Multi-GPU systems i.e. LDA (Crossfire/SLI)
		- Asymmetric Multi-GPU systems i.e. Discrete/Integrated GPU cooperation
=============================================================================*/

#pragma once

#include "D3D12ThirdParty.h"
#include "D3D12CommandContext.h"
#include "D3D12RootSignature.h"
#include "D3D12BindlessDescriptors.h"

#include COMPILED_PLATFORM_HEADER(D3D12BarriersFactory.h)

class FD3D12TransientHeapCache;
class IRHITransientMemoryCache;

struct FD3D12DeviceBasicInfo
{
	D3D_FEATURE_LEVEL           MaxFeatureLevel;
	D3D_SHADER_MODEL            MaxShaderModel;
	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier;
	D3D12_RESOURCE_HEAP_TIER    ResourceHeapTier;
	uint32                      NumDeviceNodes;
	bool                        bSupportsWaveOps;
	bool                        bSupportsAtomic64;
	bool                        bTileBasedRenderer;
	bool                        bUMA;
	bool                        bCacheCoherentUMA;
	bool                        bSupportsGPUUploadHeaps;
	ERHIFeatureLevel::Type      MaxRHIFeatureLevel;
};

struct FD3D12AdapterDesc
{
	FD3D12AdapterDesc();
	FD3D12AdapterDesc(const DXGI_ADAPTER_DESC& InDesc, int32 InAdapterIndex, const FD3D12DeviceBasicInfo& DeviceInfo);

	bool IsValid() const;

#if DXGI_MAX_FACTORY_INTERFACE >= 6
	static HRESULT EnumAdapters(int32 AdapterIndex, DXGI_GPU_PREFERENCE GpuPreference, IDXGIFactory2* DxgiFactory2, IDXGIFactory6* DxgiFactory6, IDXGIAdapter** TempAdapter);
	HRESULT EnumAdapters(IDXGIFactory2* DxgiFactory2, IDXGIFactory6* DxgiFactory6, IDXGIAdapter** TempAdapter) const;
#endif

	DXGI_ADAPTER_DESC Desc{};

	/** -1 if not supported or FindAdapter() wasn't called. Ideally we would store a pointer to IDXGIAdapter but it's unlikely the adpaters change during engine init. */
	int32 AdapterIndex = -1;

	/** The maximum D3D12 feature level supported. 0 if not supported or FindAdapter() wasn't called */
	D3D_FEATURE_LEVEL MaxSupportedFeatureLevel = (D3D_FEATURE_LEVEL)0;

	/** The maximum Shader Model supported. 0 if not supported or FindAdpater() wasn't called */
	D3D_SHADER_MODEL MaxSupportedShaderModel = (D3D_SHADER_MODEL)0;

	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier = D3D12_RESOURCE_BINDING_TIER_1;

	D3D12_RESOURCE_HEAP_TIER ResourceHeapTier = D3D12_RESOURCE_HEAP_TIER_1;

	ERHIFeatureLevel::Type MaxRHIFeatureLevel = ERHIFeatureLevel::Num;

	/** Number of device nodes (read: GPUs) */
	uint32 NumDeviceNodes = 1;

	/** Whether the GPU is tile-based. */
	bool bTileBasedRenderer = false;

	/** Whether the GPU has unified memory. This can be used to identify integrated GPU as well. */
	bool bUMA = false;

	/** Whether the UMA device has cache coherency between the GPU and CPU.  */
	bool bCacheCoherentUMA = false;

	/** Whether the device supports GPU Upload Heaps  */
	bool bSupportsGPUUploadHeaps = false;

	/** Whether SM6.0 wave ops are supported */
	bool bSupportsWaveOps = false;

	/** Whether SM6.6 atomic64 wave ops are supported */
	bool bSupportsAtomic64 = false;

#if DXGI_MAX_FACTORY_INTERFACE >= 6
	DXGI_GPU_PREFERENCE GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED;
#endif
};

struct FD3D12MemoryInfo
{
	DXGI_QUERY_VIDEO_MEMORY_INFO LocalMemoryInfo{};
	DXGI_QUERY_VIDEO_MEMORY_INFO NonLocalMemoryInfo{};

	uint32 UpdateFrameNumber = ~uint32(0);

	uint64 AvailableLocalMemory = 0;
	uint64 DemotedLocalMemory = 0;

	bool IsOverBudget() const
	{
		return DemotedLocalMemory > 0;
	}
};

class FTransientUniformBufferAllocator : public FD3D12FastConstantAllocator, public TThreadSingleton<FTransientUniformBufferAllocator>
{
public:
	FTransientUniformBufferAllocator(FD3D12Adapter* InAdapter, FD3D12Device* Parent, FRHIGPUMask VisibiltyMask) : FD3D12FastConstantAllocator(Parent, VisibiltyMask), Adapter(InAdapter) {}
	~FTransientUniformBufferAllocator();

	void Cleanup();
private:
	FD3D12Adapter* Adapter = nullptr;
};

// Represents a set of linked D3D12 device nodes (LDA i.e 1 or more identical GPUs). In most cases there will be only 1 node, however if the system supports
// SLI/Crossfire and the app enables it an Adapter will have 2 or more nodes. This class will own anything that can be shared
// across LDA including: System Pool Memory,.Pipeline State Objects, Root Signatures etc.
class FD3D12Adapter : public FNoncopyable
{
public:

	FD3D12Adapter(FD3D12AdapterDesc& DescIn);
	virtual ~FD3D12Adapter();

	void CleanupResources();

	void InitializeDevices();
	void InitializeExplicitDescriptorHeap();
	void InitializeRayTracing();

	// Getters
	FORCEINLINE const uint32 GetAdapterIndex() const { return Desc.AdapterIndex; }
	FORCEINLINE const D3D_FEATURE_LEVEL GetFeatureLevel() const { return Desc.MaxSupportedFeatureLevel; }
	FORCEINLINE D3D_SHADER_MODEL GetHighestShaderModel() const { return Desc.MaxSupportedShaderModel; }

	FORCEINLINE ID3D12Device* GetD3DDevice() const { return RootDevice; }
#if D3D12_MAX_DEVICE_INTERFACE >= 1
	FORCEINLINE ID3D12Device1* GetD3DDevice1() const { return RootDevice1; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 2
	FORCEINLINE ID3D12Device2* GetD3DDevice2() const { return RootDevice2; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 3
	FORCEINLINE ID3D12Device3* GetD3DDevice3() const { return RootDevice3; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 4
	FORCEINLINE ID3D12Device4* GetD3DDevice4() const { return RootDevice4; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 5
	FORCEINLINE ID3D12Device5* GetD3DDevice5() { return RootDevice5; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 6
	FORCEINLINE ID3D12Device6* GetD3DDevice6() { return RootDevice6; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 7
	FORCEINLINE ID3D12Device7* GetD3DDevice7() { return RootDevice7; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 8
	FORCEINLINE ID3D12Device8* GetD3DDevice8() { return RootDevice8; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 9
	FORCEINLINE ID3D12Device9* GetD3DDevice9() { return RootDevice9; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 10
	FORCEINLINE ID3D12Device10* GetD3DDevice10() { return RootDevice10; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 11
	FORCEINLINE ID3D12Device11* GetD3DDevice11() { return RootDevice11; }
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 12
	FORCEINLINE ID3D12Device12* GetD3DDevice12() { return RootDevice12; }
#endif

	FORCEINLINE IDXGIFactory2* GetDXGIFactory2() const { return DxgiFactory2; }
#if DXGI_MAX_FACTORY_INTERFACE >= 3
	FORCEINLINE IDXGIFactory3* GetDXGIFactory3() const { return DxgiFactory3; }
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 4
	FORCEINLINE IDXGIFactory4* GetDXGIFactory4() const { return DxgiFactory4; }
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 5
	FORCEINLINE IDXGIFactory5* GetDXGIFactory5() const { return DxgiFactory5; }
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 6
	FORCEINLINE IDXGIFactory6* GetDXGIFactory6() const { return DxgiFactory6; }
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 7
	FORCEINLINE IDXGIFactory7* GetDXGIFactory7() const { return DxgiFactory7; }
#endif

	FORCEINLINE const bool IsDebugDevice() const { return bDebugDevice; }

	FORCEINLINE const D3D12_RESOURCE_HEAP_TIER     GetResourceHeapTier      () const { return Desc.ResourceHeapTier; }
	FORCEINLINE const D3D12_RESOURCE_BINDING_TIER  GetResourceBindingTier   () const { return Desc.ResourceBindingTier; }
	FORCEINLINE const D3D_ROOT_SIGNATURE_VERSION   GetRootSignatureVersion  () const { return RootSignatureVersion; }

	FORCEINLINE const bool IsDepthBoundsTestSupported() const { return bDepthBoundsTestSupported; }
	FORCEINLINE const bool IsHeapNotZeroedSupported  () const { return bHeapNotZeroedSupported; }
	FORCEINLINE const bool IsHeapCreateNotResidentSupported() const { return bHeapCreateNotResidentSupported; }

	FORCEINLINE const bool AreCopyQueueTimestampQueriesSupported() const { return bCopyQueueTimestampQueriesSupported; }

	FORCEINLINE const DXGI_ADAPTER_DESC& GetD3DAdapterDesc() const { return Desc.Desc; }
	FORCEINLINE IDXGIAdapter* GetAdapter() { return DxgiAdapter; }

	FORCEINLINE const FD3D12AdapterDesc& GetDesc() const { return Desc; }

	FORCEINLINE TArray<FD3D12Viewport*>& GetViewports() { return Viewports; }

	FORCEINLINE int32 GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType InHeapType) { return InHeapType == ERHIDescriptorHeapType::Sampler ? MaxSamplerDescriptors : MaxNonSamplerDescriptors; }

	FORCEINLINE ID3D12CommandSignature* GetDrawIndirectCommandSignature            () { return DrawIndirectCommandSignature; }
	FORCEINLINE ID3D12CommandSignature* GetDrawIndexedIndirectCommandSignature     () { return DrawIndexedIndirectCommandSignature; }
	FORCEINLINE ID3D12CommandSignature* GetDispatchIndirectGraphicsCommandSignature() { return DispatchIndirectGraphicsCommandSignature; }
	FORCEINLINE ID3D12CommandSignature* GetDispatchIndirectComputeCommandSignature () { return DispatchIndirectComputeCommandSignature; }
#if PLATFORM_SUPPORTS_MESH_SHADERS
	FORCEINLINE ID3D12CommandSignature* GetDispatchIndirectMeshCommandSignature() { return DispatchIndirectMeshCommandSignature; }
#endif
	FORCEINLINE ID3D12CommandSignature* GetDispatchRaysIndirectCommandSignature() { return DispatchRaysIndirectCommandSignature; }

	FORCEINLINE FD3D12PipelineStateCache& GetPSOCache() { return PipelineStateCache; }

	const FD3D12RootSignature* GetRootSignature(const FBoundShaderStateInput& BoundShaderState);
	const FD3D12RootSignature* GetRootSignature(const class FD3D12ComputeShader* Shader);
	const FD3D12RootSignature* GetRootSignature(ERHIShaderBundleMode ShaderBundleMode, bool bBindless);
	
	const FD3D12RootSignature* GetRootSignature(const class FD3D12WorkGraphShader* Shader);
	const FD3D12RootSignature* GetGlobalWorkGraphRootSignature(const FRHIShaderBindingLayout& ShaderBindingLayout);
	const FD3D12RootSignature* GetWorkGraphGraphicsRootSignature(const FBoundShaderStateInput& BoundShaderState);

	const FD3D12RootSignature* GetLocalRootSignature(const class FD3D12RayTracingShader* Shader);
	const FD3D12RootSignature* GetGlobalRayTracingRootSignature(const FRHIShaderBindingLayout& ShaderBindingLayout);

	FORCEINLINE FD3D12RootSignatureManager* GetRootSignatureManager()
	{
		return &RootSignatureManager;
	}

	FORCEINLINE FD3D12ManualFence& GetFrameFence()  { check(FrameFence); return *FrameFence; }

	FORCEINLINE FD3D12Device* GetDevice(uint32 GPUIndex) const
	{
		check(GPUIndex < GNumExplicitGPUsForRendering);
		return Devices[GPUIndex];
	}

	TConstArrayView<FD3D12Device*> GetDevices() const
	{
		return MakeArrayView(Devices.GetData(), GNumExplicitGPUsForRendering);
	}

	FORCEINLINE uint32 GetVRSTileSize() const { return VRSTileSize; }

	void CreateDXGIFactory(bool bWithDebug);
	static void CreateDXGIFactory(TRefCountPtr<IDXGIFactory2>& DxgiFactory2, bool bWithDebug);
	void InitDXGIFactoryVariants(IDXGIFactory2* InDxgiFactory2);
	HRESULT EnumAdapters(IDXGIAdapter** TempAdapter) const;

	FORCEINLINE FD3D12UploadHeapAllocator& GetUploadHeapAllocator(uint32 GPUIndex)
	{ 
		return *(UploadHeapAllocator[GPUIndex]); 
	}

	FORCEINLINE uint32 GetDebugFlags() const { return DebugFlags; }

	void EndFrame();

	// Resource Creation
	HRESULT CreateCommittedResource(const FD3D12ResourceDesc& InDesc,
		FRHIGPUMask CreationNode,
		const D3D12_HEAP_PROPERTIES& HeapProps,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		bool bVerifyHResult = true)
	{
		return CreateCommittedResource(
			InDesc,
			CreationNode,
			HeapProps,
			InInitialD3D12Access,
			ED3D12ResourceStateMode::Default,
			ED3D12Access::Unknown,
			ClearValue,
			ppOutResource,
			Name,
			bVerifyHResult);
	}

	HRESULT CreateCommittedResource(const FD3D12ResourceDesc& Desc,
		FRHIGPUMask CreationNode,
		const D3D12_HEAP_PROPERTIES& HeapProps,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		bool bVerifyHResult = true);

	HRESULT CreateCommittedResourceRaw(
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource)
	{
		return Barriers->CreateCommittedResource(
			*this,
			InHeapProps,
			InHeapFlags,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
	}

	HRESULT CreatePlacedResource(const FD3D12ResourceDesc& InDesc,
		FD3D12Heap* BackingHeap,
		uint64 HeapOffset,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		bool bVerifyHResult = true)
	{
		return CreatePlacedResource(
			InDesc,
			BackingHeap,
			HeapOffset,
			InInitialD3D12Access,
			ED3D12ResourceStateMode::Default,
			InInitialD3D12Access,
			ClearValue,
			ppOutResource,
			Name,
			bVerifyHResult);
	}

	HRESULT CreatePlacedResource(const FD3D12ResourceDesc& Desc,
		FD3D12Heap* BackingHeap,
		uint64 HeapOffset,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		bool bVerifyHResult = true);

	HRESULT CreateReservedResource(const FD3D12ResourceDesc& Desc,
		FRHIGPUMask CreationNode,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		bool bVerifyHResult = true);

	HRESULT CreateBuffer(D3D12_HEAP_TYPE HeapType,
		FRHIGPUMask CreationNode,
		FRHIGPUMask VisibleNodes,
		uint64 HeapSize,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		D3D12_RESOURCE_FLAGS Flags = D3D12_RESOURCE_FLAG_NONE);

	HRESULT CreateBuffer(D3D12_HEAP_TYPE HeapType,
		FRHIGPUMask CreationNode,
		FRHIGPUMask VisibleNodes,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		uint64 HeapSize,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		D3D12_RESOURCE_FLAGS Flags = D3D12_RESOURCE_FLAG_NONE);

	HRESULT CreateBuffer(const D3D12_HEAP_PROPERTIES& HeapProps,
		FRHIGPUMask CreationNode,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		uint64 HeapSize,
		FD3D12Resource** ppOutResource,
		const TCHAR* Name,
		D3D12_RESOURCE_FLAGS Flags = D3D12_RESOURCE_FLAG_NONE);

	FD3D12Buffer* CreateRHIBuffer(
		const D3D12_RESOURCE_DESC& Desc,
		uint32 Alignment,
		const FRHIBufferCreateDesc& CreateDesc,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InCreateD3D12Access,
		bool bKeepUnlocked = false,
		ID3D12ResourceAllocator* ResourceAllocator = nullptr
	);

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	void CreateUAVAliasResourceDesc(const FD3D12ResourceLocation& Location);
#endif

	template <typename ObjectType, typename CreationCoreFunction>
	inline ObjectType* CreateLinkedObject(FRHIGPUMask GPUMask, const CreationCoreFunction& pfnCreationCore)
	{
		return FD3D12LinkedAdapterObject<typename ObjectType::LinkedObjectType>::template CreateLinkedObjects<ObjectType>(
			GPUMask,
			[this](uint32 GPUIndex) { return GetDevice(GPUIndex); },
			pfnCreationCore
		);
	}

	inline FD3D12CommandContextRedirector& GetDefaultContextRedirector() { return DefaultContextRedirector; }

	FD3D12TransientHeapCache& GetOrCreateTransientHeapCache();

	FD3D12FastConstantAllocator& GetTransientUniformBufferAllocator();
	void ReleaseTransientUniformBufferAllocator(FTransientUniformBufferAllocator* InAllocator);

	void BlockUntilIdle();

	const FRHIMemoryStats& CollectMemoryStats();
	FORCEINLINE const FRHIMemoryStats& GetMemoryStats() const { return MemoryStats; }
	FORCEINLINE uint64 GetMemoryStatsUpdateFrame() const { return MemoryStatsUpdateFrame; }

	bool IsTrackingAllAllocations() const { return bTrackAllAllocation; }
	void TrackAllocationData(FD3D12ResourceLocation* InAllocation, uint64 InAllocationSize, bool bCollectCallstack);
	void ReleaseTrackedAllocationData(FD3D12ResourceLocation* InAllocation, bool bDefragFree);
	void TrackHeapAllocation(FD3D12Heap* InHeap);
	void ReleaseTrackedHeap(FD3D12Heap* InHeap);
	void DumpTrackedAllocationData(FOutputDevice& OutputDevice, bool bResidentOnly, bool bWithCallstack);

	struct FAllocatedResourceResult
	{
		FD3D12ResourceLocation* Allocation = nullptr;
		uint64 Distance = 0;
	};
	void FindResourcesNearGPUAddress(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, uint64 InRange, TArray<FAllocatedResourceResult>& OutResources);
	void FindHeapsContainingGPUAddress(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, TArray<FD3D12Heap*>& OutHeaps);
	
	struct FReleasedAllocationData
	{
		D3D12_GPU_VIRTUAL_ADDRESS GPUVirtualAddress = 0;
		uint64 AllocationSize = 0;
		FName ResourceName;
		D3D12_RESOURCE_DESC ResourceDesc = {};
		uint64 ReleasedFrameID = 0;
		bool bDefragFree = false;
		bool bBackBuffer = false;
		bool bTransient = false;
		bool bHeap = false;
	};
	void FindReleasedAllocationData(D3D12_GPU_VIRTUAL_ADDRESS InGPUVirtualAddress, TArray<FReleasedAllocationData>& OutAllocationData);

	/**
	 * Controls the heuristics used by Windows Video Memory Manager to demote resources from GPU physical memory when it is
	 * overcommitted or fragmented. The D3D12 RHI performs access tracking and manual residency management, however it is
	 * still possible to overcommit under some conditions on low-VRAM GPUs.
	 * Render targets, depth buffers and UAVs are assigned high residency priority by default. 
	 * To differentiate between high priority resources, the size is packed into low 16 bits of the priority value (in units of 10MB).
	 * If ResourceSize parameter is specified, the final priority value will be packed in a mannner consistent with the
	 * default priority assignment. Current priority value is stored in the Pageable private data. The size bits will be
	 * preserved if priority changes but the size is not provided in subsequent calls to this function.
	 * See https://learn.microsoft.com/en-us/windows/win32/direct3d12/residency.
	*/
	void SetResidencyPriority(ID3D12Pageable* Pageable, D3D12_RESIDENCY_PRIORITY HeapPriority,
							  uint32 GPUIndex, TOptional<uint64> ResourceSize = {});

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorAllocator& GetBindlessDescriptorAllocator() { return BindlessDescriptorAllocator; }
#endif

	void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo);
	void ReleaseTransition(FRHITransition* Transition);

	[[nodiscard]]
	static TUniquePtr<FD3D12BarriersFactory::BarriersForContextType> CreateBarriersForContext();

protected:

	virtual void CreateRootDevice(bool bWithDebug);

	void CreateD3DInfoQueue();

	virtual void AllocateBuffer(FD3D12Device* Device,
		const D3D12_RESOURCE_DESC& Desc,
		uint32 Size,
		EBufferUsageFlags InUsage,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InCreatED3D12Access,
		uint32 Alignment,
		FD3D12Buffer* Buffer,
		FD3D12ResourceLocation& ResourceLocation,
		ID3D12ResourceAllocator* ResourceAllocator,
		const TCHAR* InDebugName,
		const FName& OwnerName);

	static ED3D12BarrierImplementationType PreferredBarrierImplType;
	static TUniquePtr<const FD3D12BarriersFactory::BarriersForAdapterType> Barriers;

	// Creates default root and execute indirect signatures
	virtual void CreateCommandSignatures();

	// LDA setups have one ID3D12Device
	TRefCountPtr<ID3D12Device> RootDevice;
#if D3D12_MAX_DEVICE_INTERFACE >= 1
	TRefCountPtr<ID3D12Device1> RootDevice1;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 2
	TRefCountPtr<ID3D12Device2> RootDevice2;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 3
	TRefCountPtr<ID3D12Device3> RootDevice3;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 4
	TRefCountPtr<ID3D12Device4> RootDevice4;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 5
	TRefCountPtr<ID3D12Device5> RootDevice5;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 6
	TRefCountPtr<ID3D12Device6> RootDevice6;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 7
	TRefCountPtr<ID3D12Device7> RootDevice7;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 8
	TRefCountPtr<ID3D12Device8> RootDevice8;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 9
	TRefCountPtr<ID3D12Device9> RootDevice9;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 10
	TRefCountPtr<ID3D12Device10> RootDevice10;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 11
	TRefCountPtr<ID3D12Device11> RootDevice11;
#endif
#if D3D12_MAX_DEVICE_INTERFACE >= 12
	TRefCountPtr<ID3D12Device12> RootDevice12;
#endif

	TRefCountPtr<IDXGIFactory2> DxgiFactory2;
#if DXGI_MAX_FACTORY_INTERFACE >= 3
	TRefCountPtr<IDXGIFactory3> DxgiFactory3;
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 4
	TRefCountPtr<IDXGIFactory4> DxgiFactory4;
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 5
	TRefCountPtr<IDXGIFactory5> DxgiFactory5;
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 6
	TRefCountPtr<IDXGIFactory6> DxgiFactory6;
#endif
#if DXGI_MAX_FACTORY_INTERFACE >= 7
	TRefCountPtr<IDXGIFactory7> DxgiFactory7;
#endif

#if D3D12_SUPPORTS_DXGI_DEBUG
	HMODULE DxgiDebugDllHandle{};
	TRefCountPtr<IDXGIDebug> DXGIDebug;
	HANDLE ExceptionHandlerHandle = INVALID_HANDLE_VALUE;
#endif

	D3D_ROOT_SIGNATURE_VERSION RootSignatureVersion;
	bool bDepthBoundsTestSupported = false;
	bool bCopyQueueTimestampQueriesSupported = false;
	bool bHeapNotZeroedSupported = false;
	bool bHeapCreateNotResidentSupported = false;

	int32 MaxNonSamplerDescriptors = 0;
	int32 MaxSamplerDescriptors = 0;

	uint32 VRSTileSize = 0;

	/** Running with debug device */
	bool bDebugDevice = false;

	FD3D12AdapterDesc Desc;

	TRefCountPtr<IDXGIAdapter> DxgiAdapter;

	FD3D12RootSignatureManager RootSignatureManager;

	FD3D12PipelineStateCache PipelineStateCache;

	TRefCountPtr<ID3D12CommandSignature> DrawIndirectCommandSignature;
	TRefCountPtr<ID3D12CommandSignature> DrawIndexedIndirectCommandSignature;
	TRefCountPtr<ID3D12CommandSignature> DispatchIndirectGraphicsCommandSignature;
	TRefCountPtr<ID3D12CommandSignature> DispatchIndirectComputeCommandSignature;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	TRefCountPtr<ID3D12CommandSignature> DispatchIndirectMeshCommandSignature;
#endif
	TRefCountPtr<ID3D12CommandSignature> DispatchRaysIndirectCommandSignature;

	FD3D12UploadHeapAllocator*	UploadHeapAllocator[MAX_NUM_GPUS];

	/** A list of all viewport RHIs that have been created. */
	TArray<FD3D12Viewport*> Viewports;

	/** A Fence whose value increases every frame*/
	TUniquePtr<FD3D12ManualFence> FrameFence;

	FD3D12CommandContextRedirector DefaultContextRedirector;

	bool bTrackAllAllocation = false;

	/** Information about an allocated resource. */
	struct FTrackedAllocationData
	{
		static const int32 MaxStackDepth = 30;

		FD3D12ResourceLocation* ResourceAllocation;
		uint64 AllocationSize;
		uint32 StackDepth;
		uint64 Stack[MaxStackDepth];
	};

	/** Tracked resource information. */
	TMap<FD3D12ResourceLocation*, FTrackedAllocationData> TrackedAllocationData;
	TArray<FD3D12Heap*> TrackedHeaps;
	TArray<FReleasedAllocationData> ReleasedAllocationData;
	FCriticalSection TrackedAllocationDataCS;

	FRHIMemoryStats MemoryStats;
	uint64 MemoryStatsUpdateFrame;

	TArray<FTransientUniformBufferAllocator*> TransientUniformBufferAllocators;
	FCriticalSection TransientUniformBufferAllocatorsCS;

	TUniquePtr<IRHITransientMemoryCache> TransientMemoryCache;

	// Each of these devices represents a physical GPU 'Node'
	TStaticArray<FD3D12Device*, MAX_NUM_GPUS> Devices;

	uint32 DebugFlags = 0;

#if USE_STATIC_ROOT_SIGNATURE
	FD3D12RootSignature StaticGraphicsRootSignature;
	FD3D12RootSignature StaticGraphicsWithConstantsRootSignature;
	FD3D12RootSignature StaticComputeRootSignature;
	FD3D12RootSignature StaticComputeWithConstantsRootSignature;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12RootSignature StaticBindlessGraphicsRootSignature;
	FD3D12RootSignature StaticBindlessGraphicsWithConstantsRootSignature;
	FD3D12RootSignature StaticBindlessComputeRootSignature;
	FD3D12RootSignature StaticBindlessComputeWithConstantsRootSignature;
#endif

	FD3D12RootSignature StaticRayTracingGlobalRootSignature;
	FD3D12RootSignature StaticRayTracingLocalRootSignature;
#endif

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorAllocator BindlessDescriptorAllocator;
#endif

private:
	// Insight memory trace helper
	void TraceMemoryAllocation(FD3D12Resource* Resource);
};
