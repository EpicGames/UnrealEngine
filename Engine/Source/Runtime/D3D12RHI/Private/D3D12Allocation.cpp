// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Memory Allocation Strategies

//-----------------------------------------------------------------------------
//	Include Files
//-----------------------------------------------------------------------------
#include "D3D12Allocation.h"
#include "D3D12RHIPrivate.h"
#include "D3D12Util.h"
#include "Misc/BufferedOutputDevice.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "Async/Async.h"

// Fix for random GPU crashes on draw indirects on multiple IHVs. Force all indirect arg buffers as committed resources (see UE-115982)
static int32 GD3D12AllowPoolAllocateIndirectArgBuffers = 1;
static FAutoConsoleVariableRef CVarD3D12AllowPoolAllocateIndirectArgBuffers(
	TEXT("d3d12.AllowPoolAllocateIndirectArgBuffers"),
	GD3D12AllowPoolAllocateIndirectArgBuffers,
	TEXT("Allow indirect args to be pool allocated (otherwise they will be committed resources) (default: 0)"),
	ECVF_ReadOnly);

#if USE_BUFFER_POOL_ALLOCATOR

#if !defined(BUFFER_POOL_DEFRAG_MAX_COPY_SIZE_PER_FRAME)
#define BUFFER_POOL_DEFRAG_MAX_COPY_SIZE_PER_FRAME				32*1024*1024
#endif

#if !defined(BUFFER_POOL_DEFAULT_POOL_SIZE)
#define BUFFER_POOL_DEFAULT_POOL_SIZE							32*1024*1024
#endif

#if !defined(BUFFER_POOL_DEFAULT_POOL_MAX_ALLOC_SIZE)
#define BUFFER_POOL_DEFAULT_POOL_MAX_ALLOC_SIZE					16*1024*1024
#endif

#if !defined(BUFFER_POOL_RT_ACCELERATION_STRUCTURE_POOL_SIZE)
#define BUFFER_POOL_RT_ACCELERATION_STRUCTURE_POOL_SIZE			32*1024*1024
#endif

#if !defined(BUFFER_POOL_RT_ACCELERATION_STRUCTURE_MAX_ALLOC_SIZE)
#define BUFFER_POOL_RT_ACCELERATION_STRUCTURE_MAX_ALLOC_SIZE	16*1024*1024
#endif

static int32 GD3D12VRAMBufferPoolDefrag = 1;
static FAutoConsoleVariableRef CVarD3D12VRAMBufferPoolDefrag(
	TEXT("d3d12.VRAMBufferPoolDefrag"),
	GD3D12VRAMBufferPoolDefrag,
	TEXT("Defrag the VRAM buffer pool"),
	ECVF_ReadOnly);

static int32 GD3D12VRAMBufferPoolDefragMaxCopySizePerFrame = BUFFER_POOL_DEFRAG_MAX_COPY_SIZE_PER_FRAME;
static FAutoConsoleVariableRef CVarD3D12VRAMBufferPoolDefragMaxCopySizePerFrame(
	TEXT("d3d12.VRAMBufferPoolDefrag.MaxCopySizePerFrame"),
	GD3D12VRAMBufferPoolDefragMaxCopySizePerFrame,
	TEXT("Max amount of data to copy during defragmentation in a single frame (default 32MB)"),
	ECVF_ReadOnly);
#endif // USE_BUFFER_POOL_ALLOCATOR

#if USE_TEXTURE_POOL_ALLOCATOR
static int32 GD3D12PoolAllocatorReadOnlyTextureVRAMPoolSize = 64 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12PoolAllocatorReadOnlyTextureVRAMPoolSize(
	TEXT("d3d12.PoolAllocator.ReadOnlyTextureVRAMPoolSize"),
	GD3D12PoolAllocatorReadOnlyTextureVRAMPoolSize,
	TEXT("Pool size of a single VRAM ReadOnly Texture memory pool (default 64MB)"),
	ECVF_ReadOnly);

static int32 GD3D12PoolAllocatorReadOnlyTextureVRAMMaxAllocationSize = 64 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12PoolAllocatorReadOnlyTextureMaxAllocationSize(
	TEXT("d3d12.PoolAllocator.ReadOnlyTextureMaxAllocationSize"),
	GD3D12PoolAllocatorReadOnlyTextureVRAMMaxAllocationSize,
	TEXT("Maximum size of a single allocation in the VRAM ReadOnly Texture pool allocator (default 64MB)"),
	ECVF_ReadOnly);

static int32 GD3D12PoolAllocatorRTUAVTextureVRAMPoolSize = 0 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12PoolAllocatorRTUAVTextureVRAMPoolSize(
	TEXT("d3d12.PoolAllocator.RTUAVTextureVRAMPoolSize"),
	GD3D12PoolAllocatorRTUAVTextureVRAMPoolSize,
	TEXT("Pool size of a single VRAM RTUAV Texture memory pool (default 0MB - disabled)"),
	ECVF_ReadOnly);

static int32 GD3D12PoolAllocatorRTUAVTextureVRAMMaxAllocationSize = 0 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12PoolAllocatorRTUAVTextureMaxAllocationSize(
	TEXT("d3d12.PoolAllocator.RTUAVTextureMaxAllocationSize"),
	GD3D12PoolAllocatorRTUAVTextureVRAMMaxAllocationSize,
	TEXT("Maximum size of a single allocation in the VRAM RTUAV Texture pool allocator (default 0MB - disabled)"),
	ECVF_ReadOnly);

static int32 GD3D12VRAMTexturePoolDefrag = 1;
static FAutoConsoleVariableRef CVarD3D12VRAMTexturePoolDefrag(
	TEXT("d3d12.VRAMTexturePoolDefrag"),
	GD3D12VRAMTexturePoolDefrag,
	TEXT("Defrag the VRAM Texture pool (enabled by default)"),
	ECVF_ReadOnly);

static int32 GD3D12VRAMTexturePoolDefragMaxCopySizePerFrame = 32 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12VRAMTexturePoolDefragMaxCopySizePerFrame(
	TEXT("d3d12.VRAMTexturePoolDefrag.MaxCopySizePerFrame"),
	GD3D12VRAMTexturePoolDefragMaxCopySizePerFrame,
	TEXT("Max amount of data to copy during defragmentation in a single frame (default 32MB)"),
	ECVF_ReadOnly);
#endif // USE_TEXTURE_POOL_ALLOCATOR

static int32 GD3D12UploadHeapSmallBlockMaxAllocationSize = 64 * 1024;
static FAutoConsoleVariableRef CVarD3D12UploadHeapSmallBlockMaxAllocationSize(
	TEXT("d3d12.UploadHeap.SmallBlock.MaxAllocationSize"),
	GD3D12UploadHeapSmallBlockMaxAllocationSize,
	TEXT("Maximum allocation size on the small block allocator for upload memory"),
	ECVF_ReadOnly);

static int32 GD3D12UploadHeapSmallBlockPoolSize = 4 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12UploadHeapSmallBlockPoolSize(
	TEXT("d3d12.UploadHeap.SmallBlock.PoolSize"),
	GD3D12UploadHeapSmallBlockPoolSize,
	TEXT("Pool size for the upload memory small block allocator"),
	ECVF_ReadOnly);

static int32 GD3D12UploadHeapBigBlockMaxAllocationSize = 64 * 1024 * 1024;
static FAutoConsoleVariableRef CVarD3D12UploadHeapBigBlockMaxAllocationSize(
	TEXT("d3d12.UploadHeap.BigBlock.MaxAllocationSize"),
	GD3D12UploadHeapBigBlockMaxAllocationSize,
	TEXT("Maximum allocation size on the big block allocator for upload memory"),
	ECVF_ReadOnly);

static int32 GD3D12UploadHeapBigBlockPoolSize = 32 * 1024 * 1024; //On a game with Nanite disabled this value could be lowered
static FAutoConsoleVariableRef CVarD3D12UploadHeapBigBlockPoolSize(
	TEXT("d3d12.UploadHeap.BigBlock.PoolSize"),
	GD3D12UploadHeapBigBlockPoolSize,
	TEXT("Pool size for the upload memory big block allocator"),
	ECVF_ReadOnly);

#if D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY
static int32 GD3D12UploadHeapPoolAllocationStrategy = 0;
#else
static int32 GD3D12UploadHeapPoolAllocationStrategy = 1;
#endif

static FAutoConsoleVariableRef CVarD3D12UploadHeapPoolAllocationStrategy(
	TEXT("d3d12.UploadHeap.PoolAllocationStrategy"),
	GD3D12UploadHeapPoolAllocationStrategy,
	TEXT("Pool allocation strategy")
	TEXT("0 - Immediate: a new pool will be allocated on the spot when needed")
	TEXT("1 - Async: the system will try to pre allocate overflow pools in advance"),
	ECVF_ReadOnly);

static int32 GD3D12UploadHeapOverFlowPoolsCount = 1;
static FAutoConsoleVariableRef CVarD3D12UploadHeapOverFlowPoolsCount(
	TEXT("d3d12.UploadHeap.OverFlowPoolsCount"),
	GD3D12UploadHeapOverFlowPoolsCount,
	TEXT("Max OverFlow Pools count to pre-allocate"),
	ECVF_ReadOnly);

#if D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY
static int32 GD3D12DefaultBufferPoolAllocationStrategy = 0;
#else
static int32 GD3D12DefaultBufferPoolAllocationStrategy = 1;
#endif

static FAutoConsoleVariableRef CVarD3D12DefaultBufferPoolAllocationStrategy(
	TEXT("d3d12.DefaultBuffer.PoolAllocationStrategy"),
	GD3D12DefaultBufferPoolAllocationStrategy,
	TEXT("Pool allocation strategy")
	TEXT("0 - Immediate: a new pool will be allocated on the spot when needed")
	TEXT("1 - Async: the system will try to pre allocate overflow pools in advance"),
	ECVF_ReadOnly);

static int32 GD3D12DefaultBufferOverFlowPoolsCount = 1;
static FAutoConsoleVariableRef CVarD3D12DefaultBufferPoolOverFlowPoolsCount(
	TEXT("d3d12.DefaultBuffer.OverFlowPoolsCount"),
	GD3D12DefaultBufferOverFlowPoolsCount,
	TEXT("Max OverFlow Pools count to pre-allocate"),
	ECVF_ReadOnly);


static int32 GD3D12FastConstantAllocatorPageSize = 64 * 1024;
static FAutoConsoleVariableRef CVarD3D12FastConstantAllocatorPageSize(
	TEXT("d3d12.FastConstantAllocatorPageSize"),
	GD3D12FastConstantAllocatorPageSize,
	TEXT("Page size for the fast constant allocator"),
	ECVF_ReadOnly);

static int32 GD3D12FastAllocatorMinPagesToRetain = 5;
static FAutoConsoleVariableRef CVarD3D12FastAllocatorMinPagesToRetain(
	TEXT("d3d12.FastAllocator.MinPagesToRetain"),
	GD3D12FastAllocatorMinPagesToRetain,
	TEXT("Minimum number of pages to retain. Pages below this limit will never be released. Pages above can be released after being unused for a certain number of frames."),
	ECVF_Default);

DECLARE_LLM_MEMORY_STAT(TEXT("D3D12AllocatorUnused"), STAT_D3D12AllocatorUnusedLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(D3D12AllocatorUnused, NAME_None, NAME_None, GET_STATFNAME(STAT_D3D12AllocatorUnusedLLM), GET_STATFNAME(STAT_EngineSummaryLLM));
DECLARE_LLM_MEMORY_STAT(TEXT("D3D12AllocatorWasted"), STAT_D3D12AllocatorWastedLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(D3D12AllocatorWasted, NAME_None, NAME_None, GET_STATFNAME(STAT_D3D12AllocatorWastedLLM), GET_STATFNAME(STAT_EngineSummaryLLM));

namespace ED3D12AllocatorID
{
	enum Type
	{
		DefaultBufferAllocator,
		DynamicHeapAllocator,
		TextureAllocator,
		DefaultBufferAllocatorFullResources
	};
};

static ERHIPoolAllocationStrategy GetPoolAllocationStrategy(uint32 AllocationStrategyIndex)
{
	// 0 = ERHIPoolAllocationStrategy::Immediate
	// 1 = ERHIPoolAllocationStrategy::Async
	return (AllocationStrategyIndex == 0) ? ERHIPoolAllocationStrategy::Immediate : ERHIPoolAllocationStrategy::Async;
}

//-----------------------------------------------------------------------------
//	Allocator Base
//-----------------------------------------------------------------------------
FD3D12ResourceAllocator::FD3D12ResourceAllocator(FD3D12Device* ParentDevice,
	FRHIGPUMask VisibleNodes,
	const FD3D12ResourceInitConfig& InInitConfig,
	const FString& Name,
	uint32 MaxSizeForPooling)
	: FD3D12DeviceChild(ParentDevice)
	, FD3D12MultiNodeGPUObject(ParentDevice->GetGPUMask(), VisibleNodes)
	, InitConfig(InInitConfig)
	, DebugName(Name)
	, Initialized(false)
	, MaximumAllocationSizeForPooling(MaxSizeForPooling)
#if D3D12RHI_TRACK_DETAILED_STATS
	, SpaceAlignedUsed(0)
	, SpaceActualUsed(0)
	, NumBlocksInDeferredDeletionQueue(0)
	, PeakUsage(0)
	, FailedAllocationSpace(0)
#endif
{
}

FD3D12ResourceAllocator::~FD3D12ResourceAllocator()
{
}

//-----------------------------------------------------------------------------
//	Buddy Allocator
//-----------------------------------------------------------------------------

FD3D12BuddyAllocator::FD3D12BuddyAllocator(FD3D12Device* ParentDevice,
	FRHIGPUMask VisibleNodes,
	const FD3D12ResourceInitConfig& InInitConfig,
	const FString& Name,
	EResourceAllocationStrategy InAllocationStrategy,
	uint32 MaxSizeForPooling,
	uint32 InMaxBlockSize,
	uint32 InMinBlockSize,
	HeapId InTraceParentHeapId)
	: FD3D12ResourceAllocator(ParentDevice, VisibleNodes, InInitConfig, Name, MaxSizeForPooling)
	, MaxBlockSize(InMaxBlockSize)
	, MinBlockSize(InMinBlockSize)
	, AllocationStrategy(InAllocationStrategy)
	, BackingHeap(nullptr)
	, LastUsedFrameFence(0)
	, TotalSizeUsed(0)
	, HeapFullMessageDisplayed(false)
{
#if UE_MEMORY_TRACE_ENABLED
	TraceHeapId = MemoryTrace_HeapSpec(InTraceParentHeapId, AllocationStrategy == EResourceAllocationStrategy::kPlacedResource ? TEXT("BuddyAllocator (PlacedResource)") : TEXT("BuddyAllocator (ManualSubAllocation)"));
#endif
	// maxBlockSize should be evenly dividable by MinBlockSize and  
	// maxBlockSize / MinBlockSize should be a power of two  
	check((MaxBlockSize / MinBlockSize) * MinBlockSize == MaxBlockSize); // Evenly dividable  
	check(0 == ((MaxBlockSize / MinBlockSize) & ((MaxBlockSize / MinBlockSize) - 1))); // Power of two  
	check(0 == (MinBlockSize % 16)); // Always 16 byte aligned
	MaxOrder = UnitSizeToOrder(SizeToUnitSize(MaxBlockSize));

	Reset();
}

void FD3D12BuddyAllocator::Initialize()
{
	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	if (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource)
	{
		D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InitConfig.HeapType);
		HeapProps.CreationNodeMask = GetGPUMask().GetNative();
		HeapProps.VisibleNodeMask = GetVisibilityMask().GetNative();

		D3D12_HEAP_DESC Desc = {};
		Desc.SizeInBytes = MaxBlockSize;
		Desc.Properties = HeapProps;
		Desc.Alignment = 0;
		Desc.Flags = InitConfig.HeapFlags;
		if (Adapter->IsHeapNotZeroedSupported())
		{
			Desc.Flags |= FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED;
		}

		CA_SUPPRESS(6286);
		const bool bShouldTrackResource = (GNumExplicitGPUsForRendering == 1) || IsGPUOnly(HeapProps.Type);

#if ENABLE_RESIDENCY_MANAGEMENT
		if (!GD3D12StartResourceResident && bShouldTrackResource)
		{
			Desc.Flags |= D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT;
		}
#endif

		ID3D12Heap* Heap = nullptr;
		{
			LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

#if PLATFORM_WINDOWS
			// we are tracking allocations ourselves
			LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
			VERIFYD3D12RESULT(Adapter->GetD3DDevice()->CreateHeap(&Desc, IID_PPV_ARGS(&Heap)));
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, MaxBlockSize, ELLMTracker::Platform, ELLMAllocType::System);
#else
			LLM_SCOPE_BYTAG(D3D12AllocatorUnused);
			VERIFYD3D12RESULT(Adapter->GetD3DDevice()->CreateHeap(&Desc, IID_PPV_ARGS(&Heap)));
#endif
		}

		BackingHeap = new FD3D12Heap(GetParentDevice(), GetVisibilityMask(), TraceHeapId);
		BackingHeap->SetHeap(Heap, TEXT("BuddyAllocator Heap"));

		if (bShouldTrackResource)
		{
			BackingHeap->BeginTrackingResidency(Desc.SizeInBytes);
		}
	}
	else
	{
		{
			const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InitConfig.HeapType, GetGPUMask().GetNative(), GetVisibilityMask().GetNative());
#if PLATFORM_WINDOWS
			LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
			VERIFYD3D12RESULT(
				Adapter->CreateBuffer(
					HeapProps,
					GetGPUMask(),
					InitConfig.InitialD3D12Access,
					ED3D12ResourceStateMode::SingleState,
					InitConfig.InitialD3D12Access,
					MaxBlockSize,
					BackingResource.GetInitReference(),
					TEXT("Resource Allocator Underlying Buffer"),
					InitConfig.ResourceFlags));
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, MaxBlockSize, ELLMTracker::Platform, ELLMAllocType::System);
#else
			LLM_SCOPE_BYTAG(D3D12AllocatorUnused);
			VERIFYD3D12RESULT(
				Adapter->CreateBuffer(
					HeapProps,
					GetGPUMask(),
					InitConfig.InitialD3D12Access,
					ED3D12ResourceStateMode::SingleState,
					InitConfig.InitialD3D12Access,
					MaxBlockSize,
					BackingResource.GetInitReference(),
					TEXT("Resource Allocator Underlying Buffer"),
					InitConfig.ResourceFlags));
#endif
#if UE_MEMORY_TRACE_ENABLED
			MemoryTrace_MarkAllocAsHeap(BackingResource->GetGPUVirtualAddress(), TraceHeapId);
#endif
		}

		if (IsCPUAccessible(InitConfig.HeapType))
		{
			BackingResource->Map();
		}
	}
}

void FD3D12BuddyAllocator::Destroy()
{
	ReleaseAllResources();
}

uint32 FD3D12BuddyAllocator::AllocateBlock(uint32 order)
{
	uint32 offset;

	if (order > MaxOrder)
	{
		// Can't allocate a block that large
		check(false); 
		// Crash to avoid infinite recursivity
		UE_LOGF(LogD3D12RHI, Fatal, "Buddy Allocator cant allocate a block that large (order %d)", order);
	}

	if (FreeBlocks[order].Num() == 0)
	{
		// No free nodes in the requested pool.  Try to find a higher-order block and split it.  
		uint32 left = AllocateBlock(order + 1);

		uint32 size = OrderToUnitSize(order);

		uint32 right = left + size;

		FreeBlocks[order].Add(right); // Add the right block to the free pool  

		offset = left; // Return the left block  
	}

	else
	{
		TSet<uint32>::TConstIterator it(FreeBlocks[order]);
		offset = *it;

		// Remove the block from the free list
		FreeBlocks[order].Remove(*it);
	}

	return offset;
}

void FD3D12BuddyAllocator::DeallocateBlock(uint32 offset, uint32 order)
{
	// See if the buddy block is free  
	uint32 size = OrderToUnitSize(order);

	uint32 buddy = GetBuddyOffset(offset, size);

	uint32* it = FreeBlocks[order].Find(buddy);

	if (it != nullptr)
	{
		// Deallocate merged blocks
		DeallocateBlock(FMath::Min(offset, buddy), order + 1);
		// Remove the buddy from the free list  
		FreeBlocks[order].Remove(*it);
	}
	else
	{
		// Add the block to the free list
		FreeBlocks[order].Add(offset);
	}
}

void FD3D12BuddyAllocator::Allocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation)
{
	FScopeLock Lock(&CS);

	if (Initialized == false)
	{
		Initialize();
		Initialized = true;
	}

	uint32 SizeToAllocate = SizeInBytes;

	// Need to always align to a multiple of 16
	Alignment = Align(FMath::Max(Alignment, 16u), 16u);

	// If the alignment doesn't match the block size
	if (MinBlockSize % Alignment != 0)
	{
		SizeToAllocate = SizeInBytes + Alignment;
	}

	// Work out what size block is needed and allocate one
	const uint32 UnitSize = SizeToUnitSize(SizeToAllocate);
	const uint32 Order = UnitSizeToOrder(UnitSize);
	const uint32 Offset = AllocateBlock(Order); // This is the offset in MinBlockSize units

	const uint32 AllocSize = uint32(OrderToUnitSize(Order) * MinBlockSize);
	const uint32 AllocationBlockOffset = uint32(Offset * MinBlockSize);
	uint32 Padding = 0;

	if (AllocationBlockOffset % Alignment != 0)
	{
		uint32 AlignedBlockOffset = AlignArbitrary(AllocationBlockOffset, Alignment);
		Padding = AlignedBlockOffset - AllocationBlockOffset;

		check((Padding + SizeInBytes) <= AllocSize)
	}

	INCREASE_ALLOC_COUNTER(SpaceAlignedUsed, AllocSize);
	INCREASE_ALLOC_COUNTER(SpaceActualUsed, SizeInBytes);
	
#if PLATFORM_WINDOWS
	// Decrease only texture size so wasted amount stays in D3D12AllocatorUnused
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - int64(SizeInBytes), ELLMTracker::Platform, ELLMAllocType::System);
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorWasted, int64(AllocSize - SizeInBytes), ELLMTracker::Platform, ELLMAllocType::System);
#endif

	TotalSizeUsed += AllocSize;

#if D3D12RHI_TRACK_DETAILED_STATS
	if (SpaceActualUsed > PeakUsage)
	{
		PeakUsage = SpaceActualUsed;
	}
#endif
	const uint32 AlignedOffsetFromResourceBase = AllocationBlockOffset + Padding;
	check((AlignedOffsetFromResourceBase % Alignment) == 0);

	// Setup the info that this allocator
	FD3D12BuddyAllocatorPrivateData& PrivateData = ResourceLocation.GetBuddyAllocatorPrivateData();
	PrivateData.Order = Order;
	PrivateData.Offset = Offset;

	ResourceLocation.SetType(FD3D12ResourceLocation::ResourceLocationType::eSubAllocation);
	ResourceLocation.SetAllocator((FD3D12BaseAllocatorType*)this);
	ResourceLocation.SetSize(SizeInBytes);

	if (AllocationStrategy == EResourceAllocationStrategy::kManualSubAllocation)
	{
		ResourceLocation.SetOffsetFromBaseOfResource(AlignedOffsetFromResourceBase);
		ResourceLocation.SetResource(BackingResource);
		ResourceLocation.SetGPUVirtualAddress(BackingResource->GetGPUVirtualAddress() + AlignedOffsetFromResourceBase);

		if (IsCPUAccessible(InitConfig.HeapType))
		{
			ResourceLocation.SetMappedBaseAddress((uint8*)BackingResource->GetResourceBaseAddress() + AlignedOffsetFromResourceBase);
		}
	}
	else
	{
		// Place resources are intialized elsewhere
	}

	// track the allocation
#if !PLATFORM_WINDOWS
	{
		LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - int64(AllocSize), ELLMTracker::Default, ELLMAllocType::System);
	}
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, ResourceLocation.GetAddressForLLMTracking(), AllocSize));
	// Note: Disabling this LLM hook for Windows is due to a work-around in the way that d3d12 buffers are tracked
	// by LLM. LLM tracks buffer data in the UpdateBufferStats function because that is the easiest place to ensure that LLM
	// can be updated whenever a buffer is created or released. Unfortunately, some buffers allocate from this allocator
	// which means that the memory would be counted twice. Because of this the tracking had to be disabled here.
	// This does mean that non-buffer memory that goes through this allocator won't be tracked, so this does need a better solution.
	// see UpdateBufferStats for a more detailed explanation.
#endif
}

bool FD3D12BuddyAllocator::TryAllocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation)
{
	FScopeLock Lock(&CS);

	if (CanAllocate(SizeInBytes, Alignment))
	{
		Allocate(SizeInBytes, Alignment, ResourceLocation);
		return true;
	}
	else
	{
		INCREASE_ALLOC_COUNTER(FailedAllocationSpace, SizeInBytes);
		return false;
	}
}

void FD3D12BuddyAllocator::Deallocate(FD3D12ResourceLocation& ResourceLocation)
{
	check(IsOwner(ResourceLocation));
	// Blocks are cleaned up async so need a lock
	FScopeLock Lock(&CS);

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	RetiredBlock& Block = DeferredDeletionQueue.Emplace_GetRef();
	Block.FrameFence = FrameFence.GetNextFenceToSignal();

	FD3D12BuddyAllocatorPrivateData& PrivateData = ResourceLocation.GetBuddyAllocatorPrivateData();
	Block.Data.Order = PrivateData.Order;
	Block.Data.Offset = PrivateData.Offset;
	Block.AllocationSize = ResourceLocation.GetSize();

	// update the last used framce fence used during garbage collection
	LastUsedFrameFence = FMath::Max(LastUsedFrameFence, Block.FrameFence);

	if (ResourceLocation.GetResource()->IsPlacedResource())
	{
		Block.PlacedResource = ResourceLocation.GetResource();
	}

	INCREASE_ALLOC_COUNTER(NumBlocksInDeferredDeletionQueue, 1);

	// track the allocation
#if !PLATFORM_WINDOWS
	// Note: Disabling this LLM hook for Windows is due to a work-around in the way that d3d12 buffers are tracked
	// by LLM. LLM tracks buffer data in the UpdateBufferStats function because that is the easiest place to ensure that LLM
	// can be updated whenever a buffer is created or released. Unfortunately, some buffers allocate from this allocator
	// which means that the memory would be counted twice. Because of this the tracking had to be disabled here.
	// This does mean that non-buffer memory that goes through this allocator won't be tracked, so this does need a better solution.
	// see UpdateBufferStats for a more detailed explanation.
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, ResourceLocation.GetAddressForLLMTracking()));
	{
		const uint32 AllocSize = uint32(OrderToUnitSize(Block.Data.Order) * MinBlockSize);
		LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, int64(AllocSize), ELLMTracker::Default, ELLMAllocType::System);
	}
#endif
}

void FD3D12BuddyAllocator::DeallocateInternal(RetiredBlock& Block)
{
	DeallocateBlock(Block.Data.Offset, Block.Data.Order);

	const uint32 Size = uint32(OrderToUnitSize(Block.Data.Order) * MinBlockSize);
	DECREASE_ALLOC_COUNTER(SpaceAlignedUsed, Size);
	DECREASE_ALLOC_COUNTER(SpaceActualUsed, Block.AllocationSize);
#if PLATFORM_WINDOWS
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, int64(Block.AllocationSize), ELLMTracker::Platform, ELLMAllocType::System);
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorWasted, 0 - int64(Size - Block.AllocationSize), ELLMTracker::Platform, ELLMAllocType::System);
#endif
	
	TotalSizeUsed -= Size;

	if (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource)
	{
		// Release the resource
		check(Block.PlacedResource != nullptr);
		Block.PlacedResource->Release();
		Block.PlacedResource = nullptr;
	}
};

void FD3D12BuddyAllocator::CleanUpAllocations()
{
	FScopeLock Lock(&CS);

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	uint32 PopCount = 0;
	for (int32 i = 0; i < DeferredDeletionQueue.Num(); i++)
	{
		RetiredBlock& Block = DeferredDeletionQueue[i];

		if (FrameFence.IsFenceComplete(Block.FrameFence, /* bUpdateCachedFenceValue */ false))
		{
			DeallocateInternal(Block);
			DECREASE_ALLOC_COUNTER(NumBlocksInDeferredDeletionQueue, 1);
			PopCount = i + 1;
		}
		else
		{
			break;
		}
	}

	if (PopCount)
	{
		// clear out all of the released blocks, don't allow the array to shrink
		DeferredDeletionQueue.RemoveAt(0, PopCount, EAllowShrinking::No);
	}
}

void FD3D12BuddyAllocator::ReleaseAllResources()
{
#if PLATFORM_WINDOWS
	LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - int64(MaxBlockSize), ELLMTracker::Platform, ELLMAllocType::System);
#endif

#if UE_MEMORY_TRACE_ENABLED
	if (AllocationStrategy != EResourceAllocationStrategy::kPlacedResource)
	{
		// Free memory & heap to match alloc operations
		D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = BackingResource ? BackingResource->GetGPUVirtualAddress() : 0;
		if (GPUAddress > 0)
		{
			MemoryTrace_UnmarkAllocAsHeap(GPUAddress, TraceHeapId);
			MemoryTrace_Free(GPUAddress, EMemoryTraceRootHeap::VideoMemory);
		}
	}

#endif

	for (RetiredBlock& Block : DeferredDeletionQueue)
	{
		DeallocateInternal(Block);
		DECREASE_ALLOC_COUNTER(NumBlocksInDeferredDeletionQueue, 1);
	}

	DeferredDeletionQueue.Empty();

	if (BackingResource)
	{
		ensure(BackingResource->GetRefCount() == 1 || GNumExplicitGPUsForRendering > 1);
		BackingResource = nullptr;
	}
}

void FD3D12BuddyAllocator::DumpAllocatorStats(class FOutputDevice& Ar)
{
#if defined(UE_BUILD_DEBUG)
	FBufferedOutputDevice BufferedOutput;
	{
		// This is the memory tracked inside individual allocation pools
		FD3D12DynamicRHI* D3DRHI = FD3D12DynamicRHI::GetD3DRHI();
		FName categoryName(&DebugName.GetCharArray()[0]);

		BufferedOutput.CategorizedLogf(categoryName, ELogVerbosity::Log, TEXT(""));
		BufferedOutput.CategorizedLogf(categoryName, ELogVerbosity::Log, TEXT("Heap Size | MinBlock Size | Space Used | Peak Usage | Unpooled Allocations | Internal Fragmentation | Blocks in Deferred Delete Queue "));
		BufferedOutput.CategorizedLogf(categoryName, ELogVerbosity::Log, TEXT("----------"));

		uint64 InternalFragmentation = SpaceAlignedUsed - SpaceActualUsed;
		BufferedOutput.CategorizedLogf(categoryName, ELogVerbosity::Log, TEXT("% 10i % 10i % 16i % 12i % 13i % 8i % 10I"),
			MaxBlockSize,
			MinBlockSize,
			SpaceAlignedUsed,
			PeakUsage,
			FailedAllocationSpace,
			InternalFragmentation,
			NumBlocksInDeferredDeletionQueue);
	}

	BufferedOutput.RedirectTo(Ar);
#endif
}


void FD3D12BuddyAllocator::UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount)
{
#if D3D12RHI_TRACK_DETAILED_STATS
	IOMemoryAllocated += MaxBlockSize;
	IOMemoryUsed += SpaceActualUsed;
	IOMemoryFree += (MaxBlockSize - SpaceAlignedUsed);
	IOAlignmentWaste += SpaceAlignedUsed - SpaceActualUsed;
	IOAllocatedPageCount++;
	if (MaxBlockSize == SpaceAlignedUsed)
		IOFullPageCount++;
#endif
}


bool FD3D12BuddyAllocator::CanAllocate(uint32 size, uint32 alignment)
{
	if (TotalSizeUsed == MaxBlockSize)
	{
		return false;
	}

	uint32 sizeToAllocate = size;
	// If the alignment doesn't match the block size
	if (alignment != 0 && MinBlockSize % alignment != 0)
	{
		sizeToAllocate = size + alignment;
	}

	uint32 blockSize = MaxBlockSize;

	for (int32 i = FreeBlocks.Num() - 1; i >= 0; i--)
	{
		if (FreeBlocks[i].Num() && blockSize >= sizeToAllocate)
		{
			return true;
		}

		// Halve the block size;
		blockSize = blockSize >> 1;

		if (blockSize < sizeToAllocate) return false;
	}
	return false;
}

void FD3D12BuddyAllocator::Reset()
{
	// Clear the free blocks collection
	FreeBlocks.Empty();

	// Initialize the pool with a free inner block of max inner block size
	FreeBlocks.SetNum(MaxOrder + 1);
	FreeBlocks[MaxOrder].Add((uint32)0);
}

//-----------------------------------------------------------------------------
//	Multi-Buddy Allocator
//-----------------------------------------------------------------------------

FD3D12MultiBuddyAllocator::FD3D12MultiBuddyAllocator(FD3D12Device* ParentDevice,
	FRHIGPUMask VisibleNodes,
	const FD3D12ResourceInitConfig& InInitConfig,
	const FString& Name,
	EResourceAllocationStrategy InAllocationStrategy,
	uint32 InMaxAllocationSize,
	uint32 InDefaultPoolSize,
	uint32 InMinBlockSize,
	HeapId InTraceParentHeapId,
	ERHIPoolAllocationStrategy InPoolAllocationStrategy,
	uint16 InOverFlowPoolsCount)
	: FD3D12ResourceAllocator(ParentDevice, VisibleNodes, InInitConfig, Name, InMaxAllocationSize)
	, AllocationStrategy(InAllocationStrategy)
	, MinBlockSize(InMinBlockSize)
	, DefaultPoolSize(InDefaultPoolSize)
	, PoolAllocationStrategy(InPoolAllocationStrategy)
	, OverFlowPoolsCount(InOverFlowPoolsCount)
{
#if UE_MEMORY_TRACE_ENABLED
	TraceHeapId = MemoryTrace_HeapSpec(InTraceParentHeapId, *Name);
#endif
}

FD3D12MultiBuddyAllocator::~FD3D12MultiBuddyAllocator()
{
	Destroy();
}

bool FD3D12MultiBuddyAllocator::TryAllocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation)
{
	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		CS.Lock();

		// No Pools, needs initialization
		if (Allocators.IsEmpty())
		{
			Initialize();
		}

		// Iterate over Pool and find one where we can allocate from 
		bool AllocationSuccess = false;

		for (int32 i = 0; i < Allocators.Num(); i++)
		{
			if (Allocators[i]->TryAllocate(SizeInBytes, Alignment, ResourceLocation))
			{
				AllocationSuccess = true;
				break;
			}
		}
		
		// Handle a failed allocation
		if (!AllocationSuccess)
		{
			// Allocation failed, grab a ready OverFlow pool and try allocating from it
			if (OverFlowAllocators.Num())
			{
				Allocators.Add(OverFlowAllocators.Pop());
				AllocationSuccess = Allocators.Last()->TryAllocate(SizeInBytes, Alignment, ResourceLocation);
			}

			// Allocation failed and either no OverFlow pool is ready or allocation failed from it as well, we need to allocate a new Pool immediately
			// we want to avoid this case as much as possible, consider increasing OverFlowPoolsCount if this happens
			if (!AllocationSuccess)
			{
				Allocators.Add(CreateNewAllocator(SizeInBytes));
				AllocationSuccess = Allocators.Last()->TryAllocate(SizeInBytes, Alignment, ResourceLocation);
			}
		}
			
		// Check if we needs to start a new async Pool Allocation if not in progress
		if ((OverFlowAllocators.Num() < OverFlowPoolsCount) && (PoolAllocationStatus == ERHIPoolAllocationStatus::Idle))
		{
			PoolAllocationStatus = ERHIPoolAllocationStatus::Allocating;

			CS.Unlock();

			AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [this]()
			{
				NewAllocatorAsyncAllocate();
			});
		}
		else
		{
			CS.Unlock();
		}

		return AllocationSuccess;
	}
	else
	{
		FScopeLock Lock(&CS);

		for (int32 i = 0; i < Allocators.Num(); i++)
		{
			if (Allocators[i]->TryAllocate(SizeInBytes, Alignment, ResourceLocation))
			{
				return true;
			}
		}

		Allocators.Add(CreateNewAllocator(SizeInBytes));
		return Allocators.Last()->TryAllocate(SizeInBytes, Alignment, ResourceLocation);

	}
}

void FD3D12MultiBuddyAllocator::Deallocate(FD3D12ResourceLocation& ResourceLocation)
{
	//The sub-allocators should handle the deallocation
	check(false);
}

FD3D12BuddyAllocator* FD3D12MultiBuddyAllocator::CreateNewAllocator(uint32 InMinSizeInBytes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12MultiBuddyAllocator::CreateNewAllocator);

	check(InMinSizeInBytes <= MaximumAllocationSizeForPooling);
	uint32 AllocationSize = (InMinSizeInBytes > DefaultPoolSize) ? FMath::RoundUpToPowerOfTwo(InMinSizeInBytes) : DefaultPoolSize;

	return new FD3D12BuddyAllocator(GetParentDevice(),
		GetVisibilityMask(),
		InitConfig,
		DebugName,
		AllocationStrategy,
		AllocationSize,
		AllocationSize,
		MinBlockSize,
		TraceHeapId);
}

void FD3D12MultiBuddyAllocator::NewAllocatorAsyncAllocate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12MultiBuddyAllocator::NewAllocatorAsyncAllocate);

	FD3D12BuddyAllocator* NewBuddyAllocator = CreateNewAllocator(MaximumAllocationSizeForPooling);

	{
		FScopeLock Lock(&CS);

		check(PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async);
		
		if (PoolAllocationStatus == ERHIPoolAllocationStatus::Allocating)
		{
			OverFlowAllocators.Add(NewBuddyAllocator);
		}
		else
		{
			delete NewBuddyAllocator;
		}

		PoolAllocationStatus = ERHIPoolAllocationStatus::Idle;
	}
}

void FD3D12MultiBuddyAllocator::Initialize()
{
	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		Allocators.Add(CreateNewAllocator(MaximumAllocationSizeForPooling));

		check(OverFlowPoolsCount > 0);

		PoolAllocationStatus = ERHIPoolAllocationStatus::Allocating;

		AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, [this]()
			{
				NewAllocatorAsyncAllocate();
			});
	}
	else
	{
		Allocators.Add(CreateNewAllocator(MaximumAllocationSizeForPooling));
	}
}

void FD3D12MultiBuddyAllocator::Destroy()
{
	ReleaseAllResources();
}

void FD3D12MultiBuddyAllocator::CleanUpAllocations(uint64 InFrameLag)
{
	FScopeLock Lock(&CS);

	for (auto*& Allocator : Allocators)
	{
		Allocator->CleanUpAllocations();
	}

	// Trim empty allocators if not used in last n frames
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	const uint64 CompletedFence = FrameFence.GetCompletedFenceValue(/* bUpdateCachedFenceValue */ true);

	for (int32 i = (Allocators.Num() - 1); i >= 0; i--)
	{
		if (Allocators[i]->IsEmpty() && (Allocators[i]->GetLastUsedFrameFence() + InFrameLag <= CompletedFence))
		{
			Allocators[i]->Destroy();
			delete(Allocators[i]);
			Allocators.RemoveAt(i);
		}
	}
}

void FD3D12MultiBuddyAllocator::DumpAllocatorStats(class FOutputDevice& Ar)
{
	//TODO
}

void FD3D12MultiBuddyAllocator::UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount)
{
#if D3D12RHI_TRACK_DETAILED_STATS
	FScopeLock Lock(&CS);

	for (FD3D12BuddyAllocator* Allocator : Allocators)
	{
		Allocator->UpdateMemoryStats(IOMemoryAllocated, IOMemoryUsed, IOMemoryFree, IOAlignmentWaste, IOAllocatedPageCount, IOFullPageCount);
	}
#endif
}

void FD3D12MultiBuddyAllocator::ReleaseAllResources()
{
	if (PoolAllocationStrategy == ERHIPoolAllocationStrategy::Async)
	{
		FScopeLock Lock(&CS);
		PoolAllocationStatus = ERHIPoolAllocationStatus::ShutDown;
	}

	for (int32 c = 0; c < OverFlowAllocators.Num(); c++)
	{
		if (OverFlowAllocators[c] != nullptr)
		{
			OverFlowAllocators[c]->Destroy();
			delete(OverFlowAllocators[c]);
		}
	}

	for (int32 i = (Allocators.Num() - 1); i >= 0; i--)
	{
		if (Allocators[i])
		{
			Allocators[i]->Destroy();
			delete(Allocators[i]);
		}
	}

	OverFlowAllocators.Empty();
	Allocators.Empty();
}

void FD3D12MultiBuddyAllocator::Reset()
{

}

//-----------------------------------------------------------------------------
//	Dynamic Buffer Allocator
//-----------------------------------------------------------------------------

FD3D12UploadHeapAllocator::FD3D12UploadHeapAllocator(FD3D12Adapter* InParent, FD3D12Device* InParentDevice, const FString& InName)
	: FD3D12AdapterChild(InParent)
	, FD3D12DeviceChild(InParentDevice)
	, FD3D12MultiNodeGPUObject(InParentDevice->GetGPUMask(), FRHIGPUMask::All()) // Upload memory, thus they can be trivially visibile to all GPUs
	, TraceHeapId(MemoryTrace_HeapSpec(EMemoryTraceRootHeap::VideoMemory, *FString(InName + TEXT(" (UploadHeapAllocator)"))))
	, SmallBlockAllocator(
		InParentDevice
		, GetVisibilityMask()
		, FD3D12ResourceInitConfig::CreateUpload()
		, TEXT("Small Block Multi Buddy Allocator")
		, EResourceAllocationStrategy::kManualSubAllocation
		, GD3D12UploadHeapSmallBlockMaxAllocationSize
		, GD3D12UploadHeapSmallBlockPoolSize
		, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT
		, TraceHeapId
		, GetPoolAllocationStrategy(GD3D12UploadHeapPoolAllocationStrategy)
		, GD3D12UploadHeapOverFlowPoolsCount)

	, BigBlockAllocator(
		InParentDevice
		, GetVisibilityMask()
		, FD3D12ResourceInitConfig::CreateUpload()
		, TEXT("Big Block Pool Allocator")
		, EResourceAllocationStrategy::kManualSubAllocation
		, GD3D12UploadHeapBigBlockPoolSize
		, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT
		, GD3D12UploadHeapBigBlockMaxAllocationSize
		, FRHIMemoryPool::EFreeListOrder::SortByOffset
		, false /*defrag*/
		, TraceHeapId
		, GetPoolAllocationStrategy(GD3D12UploadHeapPoolAllocationStrategy)
		, GD3D12UploadHeapOverFlowPoolsCount)

	, FastConstantPageAllocator(
		InParentDevice
		, GetVisibilityMask()
		, FD3D12ResourceInitConfig::CreateUpload()
		, TEXT("Fast Constant Page Multi Buddy Allocator")
		, EResourceAllocationStrategy::kManualSubAllocation
		, GD3D12FastConstantAllocatorPageSize * 64
		, GD3D12UploadHeapSmallBlockPoolSize
		, GD3D12FastConstantAllocatorPageSize
		, TraceHeapId
		, GetPoolAllocationStrategy(GD3D12UploadHeapPoolAllocationStrategy)
		, GD3D12UploadHeapOverFlowPoolsCount)
{
}

void FD3D12UploadHeapAllocator::Init()
{
	SmallBlockAllocator.Initialize();
	BigBlockAllocator.Initialize(ERHIPoolResourceTypes::Buffers);
}

void FD3D12UploadHeapAllocator::Destroy()
{
	SmallBlockAllocator.Destroy();
	BigBlockAllocator.Destroy();
	FastConstantPageAllocator.Destroy();
}


void* FD3D12UploadHeapAllocator::AllocUploadResource(uint32 InSize, uint32 InAlignment, FD3D12ResourceLocation& ResourceLocation)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12UploadHeapAllocator::AllocUploadResource);

	check(InSize > 0);
	ResourceLocation.Clear();

	// Fit in small block allocator?
	if (InSize <= SmallBlockAllocator.GetMaximumAllocationSizeForPooling())
	{
		verify(SmallBlockAllocator.TryAllocate(InSize, InAlignment, ResourceLocation));
	}
	else
	{
		FD3D12ScopeLock Lock(&BigBlockCS);

		// Forward to the big block allocator
		const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(InSize, D3D12_RESOURCE_FLAG_NONE);
		BigBlockAllocator.AllocateResource(
			GetParentDevice()->GetGPUIndex(),
			D3D12_HEAP_TYPE_UPLOAD,
			ResourceDesc,
			InSize,
			InAlignment,
			ED3D12Access::GenericRead,
			ED3D12ResourceStateMode::SingleState,
			ED3D12Access::GenericRead,
			nullptr,
			nullptr,
			ResourceLocation);
		ResourceLocation.UnlockPoolData();		
	}
	
	return ResourceLocation.GetMappedBaseAddress();
}


void* FD3D12UploadHeapAllocator::AllocFastConstantAllocationPage(uint32 InSize, uint32 InAlignment, FD3D12ResourceLocation& ResourceLocation)
{
	check(InSize > 0);
	check(InSize <= FastConstantPageAllocator.GetMaximumAllocationSizeForPooling());

	ResourceLocation.Clear();
	verify(FastConstantPageAllocator.TryAllocate(InSize, InAlignment, ResourceLocation));
	return ResourceLocation.GetMappedBaseAddress();
}


void FD3D12UploadHeapAllocator::CleanUpAllocations(uint64 InFrameLag)
{
	SmallBlockAllocator.CleanUpAllocations(InFrameLag);
	{
		FD3D12ScopeLock Lock(&BigBlockCS);
		BigBlockAllocator.CleanUpAllocations(InFrameLag);
	}
	FastConstantPageAllocator.CleanUpAllocations(InFrameLag);
}


void FD3D12UploadHeapAllocator::UpdateMemoryStats()
{
	uint32 MemoryAllocated = 0;
	uint32 MemoryUsed = 0;
	uint32 FreeMemory = 0;
	uint32 EndFreeMemory = 0;
	uint32 AlignmentWaste = 0;
	uint32 AllocatedPageCount = 0;
	uint32 FullPageCount = 0;

#if D3D12RHI_TRACK_DETAILED_STATS
	SmallBlockAllocator.UpdateMemoryStats(MemoryAllocated, MemoryUsed, FreeMemory, AlignmentWaste, AllocatedPageCount, FullPageCount);
	{
		FD3D12ScopeLock Lock(&BigBlockCS);
		BigBlockAllocator.UpdateMemoryStats(MemoryAllocated, MemoryUsed, FreeMemory, EndFreeMemory, AlignmentWaste, AllocatedPageCount, FullPageCount);
	}
	FastConstantPageAllocator.UpdateMemoryStats(MemoryAllocated, MemoryUsed, FreeMemory, AlignmentWaste, AllocatedPageCount, FullPageCount);
#endif

	SET_MEMORY_STAT(STAT_D3D12UploadPoolMemoryAllocated, MemoryAllocated);
	SET_MEMORY_STAT(STAT_D3D12UploadPoolMemoryUsed, MemoryUsed);
	SET_MEMORY_STAT(STAT_D3D12UploadPoolMemoryFree, FreeMemory);
	SET_MEMORY_STAT(STAT_D3D12UploadPoolAlignmentWaste, AlignmentWaste);
	SET_DWORD_STAT(STAT_D3D12UploadPoolPageCount, AllocatedPageCount);
	SET_DWORD_STAT(STAT_D3D12UploadPoolFullPages, FullPageCount);
}


//-----------------------------------------------------------------------------
//	Default Buffer Allocator
//-----------------------------------------------------------------------------

FD3D12ResourceInitConfig FD3D12DefaultBufferPool::GetResourceAllocatorInitConfig(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage)
{
	FD3D12ResourceInitConfig InitConfig;
	InitConfig.HeapType = InHeapType;
	InitConfig.ResourceFlags = InResourceFlags;

#if D3D12_RHI_RAYTRACING
	// Setup initial resource state depending on the requested buffer flags
	if (EnumHasAnyFlags(InBufferUsage, BUF_AccelerationStructure))
	{
		// TODO: Can we do better than READ + WRITE all the time?
		// should only have this flag and no other flags
		check(InBufferUsage == BUF_AccelerationStructure);
		check(EnumHasAnyFlags(InResourceFlags, D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE));
		InitConfig.InitialD3D12Access = ED3D12Access::BVHRead | ED3D12Access::BVHWrite;
	}
	else 
#endif // D3D12_RHI_RAYTRACING
	if (InitConfig.HeapType == D3D12_HEAP_TYPE_READBACK)
	{
		InitConfig.InitialD3D12Access = ED3D12Access::CopyDest;
	}
	else if (EnumHasAnyFlags(InBufferUsage, BUF_UnorderedAccess))
	{
		check(InResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
		InitConfig.InitialD3D12Access = ED3D12Access::UAVMask;
	}
	else
	{
		InitConfig.InitialD3D12Access = ED3D12Access::GenericRead;
	}

	InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS;
	if (EnumHasAnyFlags(InBufferUsage, BUF_DrawIndirect))
	{
		check(InResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
		InitConfig.HeapFlags |= D3D12RHI_HEAP_FLAG_ALLOW_INDIRECT_BUFFERS;
	}

	return InitConfig;
}


EResourceAllocationStrategy FD3D12DefaultBufferPool::GetResourceAllocationStrategy(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment)
{
	if (Alignment > kD3D12ManualSubAllocationAlignment)
	{
		return EResourceAllocationStrategy::kPlacedResource;
	}

	// Does the resource need state tracking and transitions
	ED3D12ResourceStateMode ResourceStateMode = InResourceStateMode;
	if (ResourceStateMode == ED3D12ResourceStateMode::Default)
	{
		ResourceStateMode = (InResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) ? ED3D12ResourceStateMode::MultiState : ED3D12ResourceStateMode::SingleState;
	}

	// multi state resource need to placed because each allocation can be in a different state
	return (ResourceStateMode == ED3D12ResourceStateMode::MultiState) ? EResourceAllocationStrategy::kPlacedResource : EResourceAllocationStrategy::kManualSubAllocation;
}


//-----------------------------------------------------------------------------
//	Default Buffer Pool
//-----------------------------------------------------------------------------


FD3D12DefaultBufferPool::FD3D12DefaultBufferPool(FD3D12Device* InParent, FD3D12MultiBuddyAllocator* InAllocator)
	: FD3D12DeviceChild(InParent)
	, FD3D12MultiNodeGPUObject(InAllocator->GetGPUMask(), InAllocator->GetVisibilityMask())
	, Allocator(InAllocator)
{
}


bool FD3D12DefaultBufferPool::SupportsAllocation(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment) const
{
	FD3D12ResourceInitConfig InitConfig = GetResourceAllocatorInitConfig(InHeapType, InResourceFlags, InBufferUsage);
	EResourceAllocationStrategy AllocationStrategy = GetResourceAllocationStrategy(InResourceFlags, InResourceStateMode, Alignment);
	return (Allocator->GetInitConfig() == InitConfig && Allocator->GetAllocationStrategy() == AllocationStrategy);
}


void FD3D12DefaultBufferPool::CleanUpAllocations(uint64 FrameLag)
{
	Allocator->CleanUpAllocations(FrameLag);
}

// Grab a buffer from the available buffers or create a new buffer if none are available
void FD3D12DefaultBufferPool::AllocDefaultResource(
	D3D12_HEAP_TYPE InHeapType,
	const D3D12_RESOURCE_DESC& Desc,
	EBufferUsageFlags InUsage,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InCreateD3D12Access,
	uint32 Alignment,
	const TCHAR* Name,
	FD3D12ResourceLocation& ResourceLocation)
{
	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	// If the resource location owns a block, this will deallocate it.
	ResourceLocation.Clear();

	if (Desc.Width == 0)
	{
		return;
	}

#if DO_CHECK
	// Validate the create state
	if (InHeapType == D3D12_HEAP_TYPE_READBACK)
	{
		check(InCreateD3D12Access == ED3D12Access::CopyDest);
	}
	else if (InHeapType == D3D12_HEAP_TYPE_UPLOAD)
	{
		check(InCreateD3D12Access == ED3D12Access::GenericRead);
	}
#if D3D12_RHI_RAYTRACING
	else if (EnumHasAnyFlags(InUsage, BUF_AccelerationStructure))
	{
		// @TODO - This isn't true anymore when using enhanced barriers and when using legacy barriers, it's the 
		// implementations responsibility now to deal with the API restriction. This can be re-written to use
		// BVHRead and BVHWrite as appropriate so that barriers implementations that can do better, will.
		// RayTracing acceleration structures must be created in a particular state and may never transition out of it.
		check(InResourceStateMode == ED3D12ResourceStateMode::SingleState);
		check(EnumHasAnyFlags(InCreateD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite));
	}
#endif // D3D12_RHI_RAYTRACING
#endif  // DO_CHECK

	const bool PoolResource = Desc.Width < Allocator->GetMaximumAllocationSizeForPooling()/* && ((Desc.Width % (1024 * 64)) != 0)*/;

	if (PoolResource)
	{
		const bool bPlacedResource = (Allocator->GetAllocationStrategy() == EResourceAllocationStrategy::kPlacedResource);

		// Ensure we're allocating from the correct pool
		if (bPlacedResource)
		{
			// Writeable resources get separate ID3D12Resource* with their own resource state by using placed resources. Just make sure it's UAV, other flags are free to differ.
			check((Desc.Flags & Allocator->GetInitConfig().ResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) != 0 || InHeapType == D3D12_HEAP_TYPE_READBACK || Alignment > kD3D12ManualSubAllocationAlignment || InResourceStateMode == ED3D12ResourceStateMode::MultiState);
		}
		else
		{
			// Read-only resources get suballocated from big resources, thus share ID3D12Resource* and resource state with other resources. Ensure it's suballocated from a resource with identical flags.
			check(Desc.Flags == Allocator->GetInitConfig().ResourceFlags);
		}
	
		if (Allocator->TryAllocate(Desc.Width, Alignment, ResourceLocation))
		{
			if (bPlacedResource)
			{
				check(ResourceLocation.GetResource() == nullptr);

				FD3D12Heap* BackingHeap = ((FD3D12BuddyAllocator*) ResourceLocation.GetAllocator())->GetBackingHeap();
				uint64 HeapOffset = ResourceLocation.GetAllocator()->GetAllocationOffsetInBytes(ResourceLocation.GetBuddyAllocatorPrivateData());

				FD3D12Resource* NewResource = nullptr;
				VERIFYD3D12RESULT(Adapter->CreatePlacedResource(
					Desc,
					BackingHeap,
					HeapOffset,
					InCreateD3D12Access,
					// @TODO - Is this right? We're ignoring InResourceStateMode
					ED3D12ResourceStateMode::MultiState,
					ED3D12Access::Unknown,
					nullptr,
					&NewResource,
					Name));

				ResourceLocation.SetResource(NewResource);
			}
			else
			{
				// Nothing to do for suballocated resources
			}

			// Successfully sub-allocated
			return;
		}
	}

	// Allocate Standalone
	// Todo: track stand alone allocations and see how much memory we use by this and how many we have
	FD3D12Resource* NewResource = nullptr;
	VERIFYD3D12RESULT(Adapter->CreateBuffer(
		InHeapType,
		GetGPUMask(),
		GetVisibilityMask(),
		InCreateD3D12Access,
		InResourceStateMode,
		InCreateD3D12Access,
		Desc.Width,
		&NewResource,
		Name,
		Desc.Flags));

	ResourceLocation.AsStandAlone(NewResource, Desc.Width);
}



void FD3D12DefaultBufferPool::UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOMemoryEndFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount)
{
	Allocator->UpdateMemoryStats(IOMemoryAllocated, IOMemoryUsed, IOMemoryFree, IOAlignmentWaste, IOAllocatedPageCount, IOFullPageCount);
}


//-----------------------------------------------------------------------------
//	Default Buffer Allocator
//-----------------------------------------------------------------------------


FD3D12DefaultBufferAllocator::FD3D12DefaultBufferAllocator(FD3D12Device* InParent, FRHIGPUMask VisibleNodes)
	: FD3D12DeviceChild(InParent)
	, FD3D12MultiNodeGPUObject(InParent->GetGPUMask(), VisibleNodes)
	, TraceHeapId(MemoryTrace_HeapSpec(EMemoryTraceRootHeap::VideoMemory, TEXT("Default Buffer Allocator")))
{
	FMemory::Memset(DefaultBufferPools, 0);
}

FD3D12BufferPool* FD3D12DefaultBufferAllocator::CreateBufferPool(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment)
{
	FD3D12Device* Device = GetParentDevice();
	FD3D12ResourceInitConfig InitConfig = FD3D12BufferPool::GetResourceAllocatorInitConfig(InHeapType, InResourceFlags, InBufferUsage);

	// NNE resources must be in heaps visible on GPU0 only.  Required by DirectML.  Note that in single GPU mode, GetVisibilityMask() will
	// be the same as GPU0(), so no extra heap fragmentation occurs in that case.
	FRHIGPUMask VisibleNodes = EnumHasAnyFlags(InBufferUsage, EBufferUsageFlags::NNE) ? FRHIGPUMask::GPU0() : GetVisibilityMask();

#if USE_BUFFER_POOL_ALLOCATOR

	const FString Name(L"D3D12 Pool Allocator");
	EResourceAllocationStrategy AllocationStrategy = FD3D12PoolAllocator::GetResourceAllocationStrategy(InResourceFlags, InResourceStateMode, Alignment);
	uint64 PoolSize = InHeapType == D3D12_HEAP_TYPE_READBACK ? READBACK_BUFFER_POOL_DEFAULT_POOL_SIZE : BUFFER_POOL_DEFAULT_POOL_SIZE;
	uint64 PoolAlignment = (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource) ? MIN_PLACED_RESOURCE_SIZE : kD3D12ManualSubAllocationAlignment;
	uint64 MaxAllocationSize = InHeapType == D3D12_HEAP_TYPE_READBACK ? READBACK_BUFFER_POOL_MAX_ALLOC_SIZE : BUFFER_POOL_DEFAULT_POOL_MAX_ALLOC_SIZE;
	FRHIMemoryPool::EFreeListOrder FreeListOrder = FRHIMemoryPool::EFreeListOrder::SortBySize;

	// Disable defrag if not Default memory
	bool bDefragEnabled = (InitConfig.HeapType == D3D12_HEAP_TYPE_DEFAULT);

	ERHIPoolAllocationStrategy PoolAllocationStrategy = GetPoolAllocationStrategy(GD3D12DefaultBufferPoolAllocationStrategy);
	uint16 OverFlowPoolCount = GD3D12DefaultBufferOverFlowPoolsCount;

#if D3D12_RHI_RAYTRACING
	// Disable defrag on the RT Acceleration pool - #kenzo_todo
	// Acceleration structure buffers may be created, but not built immediately.
	// If CopyRaytracingAccelerationStructure is called on such buffer, the GPU will crash.
	if (EnumHasAnyFlags(InitConfig.InitialD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite))
	{
		bDefragEnabled = false;

		// Use custom pool and allocation size for RT structures because they don't defrag and will thus 'waste' more memory
		PoolSize = BUFFER_POOL_RT_ACCELERATION_STRUCTURE_POOL_SIZE;
		MaxAllocationSize = BUFFER_POOL_RT_ACCELERATION_STRUCTURE_MAX_ALLOC_SIZE;

		PoolAllocationStrategy = ERHIPoolAllocationStrategy::Immediate;
		OverFlowPoolCount = 0;
	}
#endif // D3D12_RHI_RAYTRACING

	FD3D12BufferPool* NewPool = new FD3D12PoolAllocator(Device, VisibleNodes, InitConfig, Name, AllocationStrategy, PoolSize, PoolAlignment, MaxAllocationSize, FreeListOrder, bDefragEnabled, TraceHeapId, PoolAllocationStrategy, OverFlowPoolCount);

#else // USE_BUFFER_POOL_ALLOCATOR

	EResourceAllocationStrategy AllocationStrategy = FD3D12DefaultBufferPool::GetResourceAllocationStrategy(InResourceFlags, InResourceStateMode, Alignment);

	// if placed then 64KB alignment required :(
	uint32 MinBlockSize = (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource) ? MIN_PLACED_RESOURCE_SIZE : 16;

	const FString Name(L"Default Buffer Multi Buddy Allocator");
	FD3D12MultiBuddyAllocator* Allocator = new FD3D12MultiBuddyAllocator(Device,
		VisibleNodes,
		InitConfig,
		Name,
		AllocationStrategy,
		InHeapType == D3D12_HEAP_TYPE_READBACK ? READBACK_BUFFER_POOL_MAX_ALLOC_SIZE : DEFAULT_BUFFER_POOL_MAX_ALLOC_SIZE,
		InHeapType == D3D12_HEAP_TYPE_READBACK ? READBACK_BUFFER_POOL_DEFAULT_POOL_SIZE : DEFAULT_BUFFER_POOL_DEFAULT_POOL_SIZE,
		MinBlockSize,
		TraceHeapId,
		ERHIPoolAllocationStrategy::Immediate,
		0
		);

	FD3D12DefaultBufferPool* NewPool = new FD3D12DefaultBufferPool(Device, Allocator);

#endif // USE_BUFFER_POOL_ALLOCATOR

	DefaultBufferPools.Add(NewPool);
	return NewPool;
}


bool FD3D12DefaultBufferAllocator::IsPlacedResource(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment)
{
	EResourceAllocationStrategy AllocationStrategy = FD3D12BufferPool::GetResourceAllocationStrategy(InResourceFlags, InResourceStateMode, Alignment);
	return (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource);
}


ED3D12Access FD3D12DefaultBufferAllocator::GetDefaultInitialD3D12Access(
	D3D12_HEAP_TYPE InHeapType,
	EBufferUsageFlags InBufferFlags,
	ED3D12ResourceStateMode InResourceStateMode)
{
	// Validate the create state
	if (InHeapType == D3D12_HEAP_TYPE_READBACK)
	{
		return ED3D12Access::CopyDest;
	}
	else if (InHeapType == D3D12_HEAP_TYPE_UPLOAD)
	{
		return ED3D12Access::GenericRead;
	}
	else if (InBufferFlags == BUF_UnorderedAccess && InResourceStateMode == ED3D12ResourceStateMode::SingleState)
	{
		check(InHeapType == D3D12_HEAP_TYPE_DEFAULT);
		return ED3D12Access::UAVMask;
	}
#if D3D12_RHI_RAYTRACING
	else if (EnumHasAnyFlags(InBufferFlags, BUF_AccelerationStructure))
	{
		check(InHeapType == D3D12_HEAP_TYPE_DEFAULT);
		return ED3D12Access::BVHRead | ED3D12Access::BVHWrite;
	}
#endif // D3D12_RHI_RAYTRACING
	else
	{
		return ED3D12Access::GenericRead;
	}
}


// Grab a buffer from the available buffers or create a new buffer if none are available
void FD3D12DefaultBufferAllocator::AllocDefaultResource(
	D3D12_HEAP_TYPE InHeapType,
	const D3D12_RESOURCE_DESC& InResourceDesc,
	EBufferUsageFlags InBufferUsage,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InCreateD3D12Access,
	FD3D12ResourceLocation& ResourceLocation,
	uint32 Alignment,
	const TCHAR* Name)
{
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();

	// Force indirect args to stand alone allocations instead of pooled
	if (!GD3D12AllowPoolAllocateIndirectArgBuffers && EnumHasAnyFlags(InBufferUsage, BUF_DrawIndirect))
	{
		ResourceLocation.Clear();

		// NNE resources must be in heaps visible on GPU0 only.  Required by DirectML.
		FD3D12Resource* NewResource = nullptr;
		const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InHeapType, GetGPUMask().GetNative(), EnumHasAnyFlags(InBufferUsage, EBufferUsageFlags::NNE) ? FRHIGPUMask::GPU0().GetNative() : GetVisibilityMask().GetNative());
		D3D12_RESOURCE_DESC Desc = InResourceDesc;
		Desc.Alignment = 0;
		VERIFYD3D12RESULT(Adapter->CreateCommittedResource(
			Desc,
			GetGPUMask(),
			HeapProps,
			InCreateD3D12Access,
			InResourceStateMode,
			InCreateD3D12Access,
			nullptr,
			&NewResource,
			Name,
			false));

		ResourceLocation.AsStandAlone(NewResource, InResourceDesc.Width);

		return;
	}

	if (EnumHasAnyFlags(InBufferUsage, BUF_ReservedResource))
	{
		ResourceLocation.Clear();

		FD3D12Resource* NewResource = nullptr;
		checkf(Alignment % GRHIGlobals.ReservedResources.TileSizeInBytes == 0,
			TEXT("Reserved buffer alignment is expected to be a multiple of the reserved resource tile size"));
		FD3D12ResourceDesc Desc = InResourceDesc;
		Desc.Alignment = Alignment;
		Desc.bReservedResource = true;
		VERIFYD3D12RESULT(Adapter->CreateReservedResource(
			Desc,
			GetGPUMask(),
			InCreateD3D12Access,
			InResourceStateMode,
			InCreateD3D12Access,
			nullptr,
			&NewResource,
			Name,
			false));

		ResourceLocation.AsStandAlone(NewResource, InResourceDesc.Width);

		return;
	}

	FScopeLock Lock(&CS);

	// Patch out deny shader resource because it doesn't add anything for buffers and allows more pool sharing
	// TODO: check if this is different on Xbox?
	D3D12_RESOURCE_DESC ResourceDesc = InResourceDesc;
	ResourceDesc.Flags = ResourceDesc.Flags & (~D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE);

	// Do we already have a default pool which support this allocation?
	FD3D12BufferPool* BufferPool = nullptr;
	for (FD3D12BufferPool* Pool : DefaultBufferPools)
	{
		if (Pool->SupportsAllocation(InHeapType, ResourceDesc.Flags, InBufferUsage, InResourceStateMode, Alignment))
		{
			BufferPool = Pool;
			break;
		}
	}

	// No pool yet, then create one
	if (BufferPool == nullptr)
	{
		BufferPool = CreateBufferPool(InHeapType, ResourceDesc.Flags, InBufferUsage, InResourceStateMode, Alignment);
	}

	// Perform actual allocation
	BufferPool->AllocDefaultResource(
		InHeapType,
		ResourceDesc,
		InBufferUsage,
		InResourceStateMode,
		InCreateD3D12Access,
		Alignment,
		Name,
		ResourceLocation);
}


void FD3D12DefaultBufferAllocator::FreeDefaultBufferPools()
{
	FScopeLock Lock(&CS);

	for (FD3D12BufferPool*& DefaultBufferPool : DefaultBufferPools)
	{
		if (DefaultBufferPool)
		{
			// No frame lag, delete all unused pages immediatly
			DefaultBufferPool->CleanUpAllocations(0);

			delete DefaultBufferPool;
			DefaultBufferPool = nullptr;
		}
	}
}


void FD3D12DefaultBufferAllocator::BeginFrame(FD3D12ContextArray const& Contexts)
{
#if USE_BUFFER_POOL_ALLOCATOR
	FScopeLock Lock(&CS);

	if (GD3D12VRAMBufferPoolDefrag > 0 && GD3D12VRAMBufferPoolDefragMaxCopySizePerFrame > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DefragBufferPool);

		uint32 MaxCopySize = GD3D12VRAMBufferPoolDefragMaxCopySizePerFrame;
		uint32 CopySize = 0;
		for (FD3D12BufferPool* DefaultBufferPool : DefaultBufferPools)
		{
			if (DefaultBufferPool)
			{
				DefaultBufferPool->Defrag(Contexts, MaxCopySize, CopySize);

				// break when we reach the max copy size
				if (CopySize >= MaxCopySize)
				{
					break;
				}
			}
		}
	}

	{
		// @todo dev-pr - explicit use of graphics context - nothing is synchronizing async compute - needs refactor
		FD3D12CommandContext& CommandContext = *Contexts[ERHIPipeline::Graphics]->GetSingleDeviceContext(GetParentDevice()->GetGPUIndex());

		TRACE_CPUPROFILER_EVENT_SCOPE(FlushPendingBufferCopyOps);
		RHI_BREADCRUMB_EVENT(CommandContext, "BufferPoolCopyOps");

		for (FD3D12BufferPool* DefaultBufferPool : DefaultBufferPools)
		{
			if (DefaultBufferPool)
			{
				DefaultBufferPool->FlushPendingCopyOps(CommandContext);
			}
		}
	}
#endif // USE_BUFFER_POOL_ALLOCATOR
}


void FD3D12DefaultBufferAllocator::CleanupFreeBlocks(uint64 InFrameLag)
{
	FScopeLock Lock(&CS);

	for (FD3D12BufferPool* DefaultBufferPool : DefaultBufferPools)
	{
		if (DefaultBufferPool)
		{
			DefaultBufferPool->CleanUpAllocations(InFrameLag);
		}
	}
}

void FD3D12DefaultBufferAllocator::UpdateMemoryStats()
{
	FScopeLock Lock(&CS);

	uint32 MemoryAllocated = 0;
	uint32 MemoryUsed = 0;
	uint32 FreeMemory = 0;
	uint32 EndFreeMemory = 0;
	uint32 AlignmentWaste = 0;
	uint32 AllocatedPageCount = 0;
	uint32 FullPageCount = 0;

#if D3D12RHI_TRACK_DETAILED_STATS
	for (FD3D12BufferPool* DefaultBufferPool : DefaultBufferPools)
	{
		if (DefaultBufferPool)
		{
			DefaultBufferPool->UpdateMemoryStats(MemoryAllocated, MemoryUsed, FreeMemory, EndFreeMemory, AlignmentWaste, AllocatedPageCount, FullPageCount);
		}
	}
#endif

	//check((MemoryUsed + AlignmentWaste + FreeMemory) == MemoryAllocated);

	// compute fragmentation percentage stats:
	uint32 Fragmentation = FreeMemory - EndFreeMemory;
	uint32 FragmentationUsed = MemoryUsed + AlignmentWaste + Fragmentation;
	float FragmentationPercentage = FragmentationUsed == 0 ? 0.0 : float(Fragmentation) / float(FragmentationUsed);

	SET_MEMORY_STAT(STAT_D3D12BufferPoolMemoryAllocated, MemoryAllocated);
	SET_MEMORY_STAT(STAT_D3D12BufferPoolMemoryUsed, MemoryUsed);
	SET_MEMORY_STAT(STAT_D3D12BufferPoolMemoryFree, FreeMemory);
	SET_MEMORY_STAT(STAT_D3D12BufferPoolAlignmentWaste, AlignmentWaste);
	SET_DWORD_STAT(STAT_D3D12BufferPoolPageCount, AllocatedPageCount);
	SET_DWORD_STAT(STAT_D3D12BufferPoolFullPages, FullPageCount);
	SET_MEMORY_STAT(STAT_D3D12BufferPoolFragmentation, Fragmentation);
	SET_FLOAT_STAT(STAT_D3D12BufferPoolFragmentationPercentage, FragmentationPercentage);
}

//-----------------------------------------------------------------------------
//	Texture Allocator
//-----------------------------------------------------------------------------

#if USE_TEXTURE_POOL_ALLOCATOR
FD3D12TextureAllocatorPool::FD3D12TextureAllocatorPool(FD3D12Device* Device, FRHIGPUMask VisibilityNode) :
	FD3D12DeviceChild(Device),
	FD3D12MultiNodeGPUObject(Device->GetGPUMask(), VisibilityNode),
	TraceHeapId(MemoryTrace_HeapSpec(EMemoryTraceRootHeap::VideoMemory, TEXT("Texture Allocator Pool")))
{	
	FD3D12ResourceInitConfig SharedInitConfig;

	// texture only interesting in VRAM for now
	SharedInitConfig.HeapType = D3D12_HEAP_TYPE_DEFAULT;

	// unused for textures because placed and not suballocated
	SharedInitConfig.ResourceFlags = D3D12_RESOURCE_FLAG_NONE;
	SharedInitConfig.InitialD3D12Access = ED3D12Access::Common;

	EResourceAllocationStrategy AllocationStrategy = EResourceAllocationStrategy::kPlacedResource;
	FRHIMemoryPool::EFreeListOrder FreeListOrder = FRHIMemoryPool::EFreeListOrder::SortBySize;

	{
		FD3D12ResourceInitConfig InitConfig = SharedInitConfig;
		InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;

		const FString Name(L"D3D12 ReadOnly4K Texture Pool Allocator");
		uint64 PoolSize = 4 * 1024 * 1024;
		uint64 MaxAllocationSize = PoolSize;
		bool bDefragEnabled = false; // Disable defrag on 4K pool because it shouldn't really fragment - all allocations are 4K or multiple of 4K and pretty small
		FD3D12PoolAllocator* ReadOnly4KPool = new FD3D12PoolAllocator(Device, GetVisibilityMask(), InitConfig, Name, AllocationStrategy, PoolSize, D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT, MaxAllocationSize, FreeListOrder, bDefragEnabled, TraceHeapId, ERHIPoolAllocationStrategy::Immediate, 0);
		ReadOnly4KPool->Initialize(ERHIPoolResourceTypes::NonRTDSTextures);

		PoolAllocators[(int)EPoolType::ReadOnly4K] = ReadOnly4KPool;
	}

	{
		FD3D12ResourceInitConfig InitConfig = SharedInitConfig;
		InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;

		const FString Name(L"D3D12 ReadOnly Texture Pool Allocator");
		uint64 PoolSize = GD3D12PoolAllocatorReadOnlyTextureVRAMPoolSize;
		uint64 MaxAllocationSize = GD3D12PoolAllocatorReadOnlyTextureVRAMMaxAllocationSize;
		bool bDefragEnabled = true;
		FD3D12PoolAllocator* ReadOnlyPool = new FD3D12PoolAllocator(Device, GetVisibilityMask(), InitConfig, Name, AllocationStrategy, PoolSize, D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, MaxAllocationSize, FreeListOrder, bDefragEnabled, TraceHeapId, ERHIPoolAllocationStrategy::Immediate, 0);
		ReadOnlyPool->Initialize(ERHIPoolResourceTypes::NonRTDSTextures);

		PoolAllocators[(int)EPoolType::ReadOnly] = ReadOnlyPool;
	}

	{
		FD3D12ResourceInitConfig InitConfig = SharedInitConfig;
		InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_RT_DS_TEXTURES;

		const FString Name(L"D3D12 RT Texture Pool Allocator");
		uint64 PoolSize = GD3D12PoolAllocatorRTUAVTextureVRAMPoolSize;
		uint64 MaxAllocationSize = GD3D12PoolAllocatorRTUAVTextureVRAMMaxAllocationSize;
		// FD3D12ResourceLocation::OnAllocationMoved doesn't correctly retrieve the clear value when recreating moved resources, so we need to disable defrag for this pool for the time being.
		bool bDefragEnabled = false;
		FD3D12PoolAllocator* RTPool = new FD3D12PoolAllocator(Device, GetVisibilityMask(), InitConfig, Name, AllocationStrategy, PoolSize, D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, MaxAllocationSize, FreeListOrder, bDefragEnabled, TraceHeapId, ERHIPoolAllocationStrategy::Immediate, 0);
		RTPool->Initialize(ERHIPoolResourceTypes::NonRTDSTextures);

		PoolAllocators[(int)EPoolType::RenderTarget] = RTPool;
	}

	{
		FD3D12ResourceInitConfig InitConfig = SharedInitConfig;
		InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;

		const FString Name(L"D3D12 UAV Texture Pool Allocator");
		uint64 PoolSize = GD3D12PoolAllocatorRTUAVTextureVRAMPoolSize;
		uint64 MaxAllocationSize = GD3D12PoolAllocatorRTUAVTextureVRAMMaxAllocationSize;
		// Defrag doesn't correctly handle resources which need the BCn/UINT UAV aliasing workaround, so we'll turn off defrag for this heap for now.
		bool bDefragEnabled = false;
		FD3D12PoolAllocator* UAVPool = new FD3D12PoolAllocator(Device, GetVisibilityMask(), InitConfig, Name, AllocationStrategy, PoolSize, D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, MaxAllocationSize, FreeListOrder, bDefragEnabled, TraceHeapId, ERHIPoolAllocationStrategy::Immediate, 0);
		UAVPool->Initialize(ERHIPoolResourceTypes::NonRTDSTextures);

		PoolAllocators[(int)EPoolType::UAV] = UAVPool;
	}
}

void FD3D12TextureAllocatorPool::Destroy()
{
	for (uint32 PoolIndex = 0; PoolIndex < (uint32)EPoolType::Count; ++PoolIndex)
	{
		PoolAllocators[PoolIndex]->CleanUpAllocations(0);
		delete PoolAllocators[PoolIndex];
		PoolAllocators[PoolIndex] = nullptr;
	}
}

HRESULT FD3D12TextureAllocatorPool::AllocateTexture(
	FD3D12ResourceDesc Desc,
	const D3D12_CLEAR_VALUE* ClearValue,
	EPixelFormat UEFormat,
	FD3D12ResourceLocation& TextureLocation,
	ED3D12Access InInitialD3D12Access,
	ED3D12Access InDefaultD3D12Access,
	const TCHAR* Name)
{
	// The top mip level must be less than 64 KB to use 4 KB alignment
	bool b4KAligment = FD3D12Texture::CanBe4KAligned(Desc, (EPixelFormat)UEFormat);
	Desc.Alignment = b4KAligment ?	D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT : (Desc.SampleDesc.Count > 1 ? D3D12_DEFAULT_MSAA_RESOURCE_PLACEMENT_ALIGNMENT : D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT);

	const D3D12_RESOURCE_ALLOCATION_INFO Info = GetParentDevice()->GetResourceAllocationInfoUncached(Desc);

	const bool bIsRenderTarget = EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);
	const bool bIsReadOnly = !bIsRenderTarget && !EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) && !Desc.NeedsUAVAliasWorkarounds();

	ED3D12ResourceStateMode ResourceStateMode = bIsReadOnly ? ED3D12ResourceStateMode::Default : ED3D12ResourceStateMode::MultiState;
	EPoolType PoolType = bIsReadOnly ? (b4KAligment ? EPoolType::ReadOnly4K : EPoolType::ReadOnly) : (bIsRenderTarget ? EPoolType::RenderTarget : EPoolType::UAV);

	PoolAllocators[(int)PoolType]->AllocateResource(
		GetParentDevice()->GetGPUIndex(),
		D3D12_HEAP_TYPE_DEFAULT,
		Desc,
		Info.SizeInBytes,
		Info.Alignment,
		InInitialD3D12Access,
		ResourceStateMode,
		InDefaultD3D12Access,
		ClearValue,
		Name,
		TextureLocation);

	return S_OK;
}


void FD3D12TextureAllocatorPool::BeginFrame(FD3D12ContextArray const& Contexts)
{
	if (GD3D12VRAMTexturePoolDefrag > 0 && GD3D12VRAMTexturePoolDefragMaxCopySizePerFrame > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DefragTexturePool);

		uint32 MaxCopySize = GD3D12VRAMTexturePoolDefragMaxCopySizePerFrame;
		uint32 CopySize = 0;
		for (uint32 PoolIndex = 0; PoolIndex < (uint32)EPoolType::Count; ++PoolIndex)
		{
			PoolAllocators[PoolIndex]->Defrag(Contexts, MaxCopySize, CopySize);
		}
	}

	{
		// @todo dev-pr - explicit use of graphics context - nothing is synchronizing async compute - needs refactor
		FD3D12CommandContext& CommandContext = *Contexts[ERHIPipeline::Graphics]->GetSingleDeviceContext(GetParentDevice()->GetGPUIndex());

		TRACE_CPUPROFILER_EVENT_SCOPE(FlushPendingTextureCopyOps);
		RHI_BREADCRUMB_EVENT(CommandContext, "TexturePoolCopyOps");
		
		for (uint32 PoolIndex = 0; PoolIndex < (uint32)EPoolType::Count; ++PoolIndex)
		{
			PoolAllocators[PoolIndex]->FlushPendingCopyOps(CommandContext);
		}
	}
}

void FD3D12TextureAllocatorPool::CleanUpAllocations()
{
	for (uint32 PoolIndex = 0; PoolIndex < (uint32)EPoolType::Count; ++PoolIndex)
	{
		PoolAllocators[PoolIndex]->CleanUpAllocations(20);
	}
}


bool FD3D12TextureAllocatorPool::GetMemoryStats(uint64& OutTotalAllocated, uint64& OutTotalUnused) const
{
	uint32 MemoryAllocated = 0;
	uint32 MemoryUsed = 0;
	uint32 FreeMemory = 0;
	uint32 EndFreeMemory = 0;
	uint32 AlignmentWaste = 0;
	uint32 AllocatedPageCount = 0;
	uint32 FullPageCount = 0;

	for (uint32 PoolIndex = 0; PoolIndex < (uint32)EPoolType::Count; ++PoolIndex)
	{
		PoolAllocators[PoolIndex]->UpdateMemoryStats(MemoryAllocated, MemoryUsed, FreeMemory, EndFreeMemory, AlignmentWaste, AllocatedPageCount, FullPageCount);
	}

	OutTotalAllocated = MemoryAllocated;
	OutTotalUnused = FreeMemory;

	return true;
}

#else

FD3D12TextureAllocator::FD3D12TextureAllocator(FD3D12Device* Device,
	FRHIGPUMask VisibleNodes,
	const FString& Name,
	uint32 HeapSize,
	D3D12_HEAP_FLAGS Flags,
	HeapId InTraceParentHeapId) :
	FD3D12MultiBuddyAllocator(Device,
		VisibleNodes,
		FD3D12ResourceInitConfig
		{
			D3D12_HEAP_TYPE_DEFAULT,
			Flags | D3D12_HEAP_FLAG_DENY_BUFFERS,
			D3D12_RESOURCE_FLAG_NONE,
			ED3D12Access::GenericRead,
		},
		Name,
		EResourceAllocationStrategy::kPlacedResource,
		D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT,
		HeapSize,
		D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT,
		InTraceParentHeapId,
		ERHIPoolAllocationStrategy::Immediate, 
		0)
{
}

FD3D12TextureAllocator::~FD3D12TextureAllocator()
{
}

HRESULT FD3D12TextureAllocator::AllocateTexture(
	FD3D12ResourceDesc Desc,
	const D3D12_CLEAR_VALUE* ClearValue,
	FD3D12ResourceLocation& TextureLocation,
	ED3D12Access InInitialD3D12Access,
	ED3D12Access InDefaultD3D12Access,
	const TCHAR* Name)
{
	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	HRESULT hr = S_OK;
	FD3D12Resource* NewResource = nullptr;

	TextureLocation.Clear();

	D3D12_RESOURCE_ALLOCATION_INFO Info = Device->GetResourceAllocationInfoUncached(Desc);

	if (Info.SizeInBytes < D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT)
	{
		if (TryAllocate(Info.SizeInBytes, Info.Alignment, TextureLocation))
		{
			FD3D12Heap* BackingHeap = ((FD3D12BuddyAllocator*)TextureLocation.GetAllocator())->GetBackingHeap();
			uint64 HeapOffset = TextureLocation.GetAllocator()->GetAllocationOffsetInBytes(TextureLocation.GetBuddyAllocatorPrivateData());

			hr = Adapter->CreatePlacedResource(
				Desc,
				BackingHeap,
				HeapOffset,
				InInitialD3D12Access,
				ED3D12ResourceStateMode::Default,
				InDefaultD3D12Access,
				ClearValue,
				&NewResource,
				Name,
				false);
	
			TextureLocation.SetType(FD3D12ResourceLocation::ResourceLocationType::eSubAllocation);
			TextureLocation.SetResource(NewResource);
	
			return hr;
		}
	}

	// Request default alignment for stand alone textures
	Desc.Alignment = 0;
	const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT, GetGPUMask().GetNative(), (uint32)GetVisibilityMask().GetNative());

	hr = Adapter->CreateCommittedResource(
		Desc,
		GetGPUMask(),
		HeapProps,
		InInitialD3D12Access,
		ClearValue,
		&NewResource,
		Name,
		false);

	TextureLocation.AsStandAlone(NewResource, Info.SizeInBytes);

	return hr;
}

FD3D12TextureAllocatorPool::FD3D12TextureAllocatorPool(FD3D12Device* Device, FRHIGPUMask VisibilityNode) :
	FD3D12DeviceChild(Device),
	FD3D12MultiNodeGPUObject(Device->GetGPUMask(), VisibilityNode),
	TraceHeapId(MemoryTrace_HeapSpec(EMemoryTraceRootHeap::VideoMemory, TEXT("Texture Allocator Pool"))),
	ReadOnlyTexturePool(Device, VisibilityNode, FString(L"Small Non-DS/RT Texture allocator"), TEXTURE_POOL_SIZE, D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES, TraceHeapId)
{};

HRESULT FD3D12TextureAllocatorPool::AllocateTexture(
	FD3D12ResourceDesc Desc,
	const D3D12_CLEAR_VALUE* ClearValue,
	EPixelFormat UEFormat,
	FD3D12ResourceLocation& TextureLocation, 
	ED3D12Access InInitialD3D12Access,
	ED3D12Access InDefaultD3D12Access,
	const TCHAR* Name)
{
	if (FD3D12Texture::CanBe4KAligned(Desc, (EPixelFormat)UEFormat))
	{
		Desc.Alignment = D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT; // request 4k alignment
			return ReadOnlyTexturePool.AllocateTexture(
				Desc,
				ClearValue,
				TextureLocation,
				InInitialD3D12Access,
				InDefaultD3D12Access,
				Name);
	}

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12Resource* Resource = nullptr;

	const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT, GetGPUMask().GetNative(), GetVisibilityMask().GetNative());
	const D3D12_RESOURCE_ALLOCATION_INFO Info = GetParentDevice()->GetResourceAllocationInfoUncached(Desc);

	HRESULT hr = Adapter->CreateCommittedResource(Desc, GetGPUMask(), HeapProps, InInitialD3D12Access, ClearValue, &Resource, Name, false);

	if (SUCCEEDED(hr))
	{
		TextureLocation.AsStandAlone(Resource, Info.SizeInBytes);
	}

	return hr;
}
#endif

//-----------------------------------------------------------------------------
//	Fast Allocation
//-----------------------------------------------------------------------------

FD3D12FastAllocator::FD3D12FastAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, D3D12_HEAP_TYPE InHeapType, uint32 PageSize)
	: FD3D12DeviceChild(Parent)
	, FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibiltyMask)
	, PagePool(Parent, VisibiltyMask, InHeapType, PageSize)
	, CurrentAllocatorPage(nullptr)
{}

FD3D12FastAllocator::FD3D12FastAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, const D3D12_HEAP_PROPERTIES& InHeapProperties, uint32 PageSize)
	: FD3D12DeviceChild(Parent)
	, FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibiltyMask)
	, PagePool(Parent, VisibiltyMask, InHeapProperties, PageSize)
	, CurrentAllocatorPage(nullptr)
{}

void* FD3D12FastAllocator::Allocate(uint32 Size, uint32 Alignment, class FD3D12ResourceLocation* ResourceLocation)
{
	// Check to make sure our assumption that we don't need a ResourceLocation->Clear() here is valid.
	checkf(!ResourceLocation->IsValid(), TEXT("The supplied resource location already has a valid resource. You should Clear() it first or it may leak."));

	if (Size > PagePool.GetPageSize())
	{
		FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();

		// If upload memory then fallback to the shader upload heap allocator which support dynamic sized allocation of bigger sizes
		if (PagePool.GetHeapType() == D3D12_HEAP_TYPE_UPLOAD)
		{
			return Adapter->GetUploadHeapAllocator(GetGPUMask().ToIndex()).AllocUploadResource(Size, Alignment, *ResourceLocation);
		}

		//Allocations are 64k aligned
		if (Alignment)
		{
			Alignment = (D3D_BUFFER_ALIGNMENT % Alignment) == 0 ? 0 : Alignment;
		}

		FD3D12Resource* Resource = nullptr;
		FString ResourceName;
#if NAME_OBJECTS
		static int64 ID = 0;
		const int64 UniqueID = FPlatformAtomics::InterlockedIncrement(&ID);
		ResourceName = FString::Printf(TEXT("Stand Alone Fast Allocation %lld"), UniqueID);
#endif
		VERIFYD3D12RESULT(Adapter->CreateBuffer(PagePool.GetHeapType(), GetGPUMask(), GetVisibilityMask(), Size + Alignment, &Resource, *ResourceName));
		ResourceLocation->AsStandAlone(Resource, Size + Alignment);

		return PagePool.IsCPUWritable() ? Resource->GetResourceBaseAddress() : nullptr;
	}
	else
	{
		FD3D12ScopeLock Lock(&CS);

		const uint32 Offset = (CurrentAllocatorPage) ? CurrentAllocatorPage->NextFastAllocOffset : 0;
		uint32 CurrentOffset = AlignArbitrary(Offset, Alignment);

		// See if there is room in the current pool
		if (CurrentAllocatorPage == nullptr || PagePool.GetPageSize() < CurrentOffset + Size)
		{
			if (CurrentAllocatorPage)
			{
				PagePool.ReturnFastAllocatorPage(CurrentAllocatorPage);
			}
			CurrentAllocatorPage = PagePool.RequestFastAllocatorPage();

			CurrentOffset = AlignArbitrary(CurrentAllocatorPage->NextFastAllocOffset, Alignment);
		}

		check(PagePool.GetPageSize() - Size >= CurrentOffset);

		// Create a FD3D12ResourceLocation representing a sub-section of the pool resource
		ResourceLocation->AsFastAllocation(CurrentAllocatorPage->FastAllocBuffer.GetReference(),
			Size,
			CurrentAllocatorPage->FastAllocBuffer->GetGPUVirtualAddress(),
			CurrentAllocatorPage->FastAllocData,
			0,
			CurrentOffset);

		CurrentAllocatorPage->NextFastAllocOffset = CurrentOffset + Size;
		CurrentAllocatorPage->UpdateFence();

		check(ResourceLocation->GetMappedBaseAddress());
		return ResourceLocation->GetMappedBaseAddress();
	}
}

void FD3D12FastAllocator::CleanupPages(uint64 FrameLag)
{
	FD3D12ScopeLock Lock(&CS);
	PagePool.CleanupPages(FrameLag);
}

void FD3D12FastAllocator::Destroy()
{
	FD3D12ScopeLock Lock(&CS);
	if (CurrentAllocatorPage)
	{
		PagePool.ReturnFastAllocatorPage(CurrentAllocatorPage);
		CurrentAllocatorPage = nullptr;
	}

	PagePool.Destroy();
}

FD3D12FastAllocatorPagePool::FD3D12FastAllocatorPagePool(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, D3D12_HEAP_TYPE InHeapType, uint32 Size)
	: FD3D12DeviceChild(Parent)
	, FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibiltyMask)
	, PageSize(Size)
	, HeapProperties(CD3DX12_HEAP_PROPERTIES(InHeapType, Parent->GetGPUMask().GetNative(), VisibiltyMask.GetNative()))
{};

FD3D12FastAllocatorPagePool::FD3D12FastAllocatorPagePool(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, const D3D12_HEAP_PROPERTIES& InHeapProperties, uint32 Size)
	: FD3D12DeviceChild(Parent)
	, FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibiltyMask)
	, PageSize(Size)
	, HeapProperties(InHeapProperties)
{};

FD3D12FastAllocatorPage* FD3D12FastAllocatorPagePool::RequestFastAllocatorPage()
{
	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();
	FD3D12ManualFence& Fence = Adapter->GetFrameFence();

	const uint64 CompletedFence = Fence.GetCompletedFenceValue(/* bUpdateCachedFenceValue */ true);

	for (int32 Index = 0; Index < Pool.Num(); Index++)
	{
		FD3D12FastAllocatorPage* Page = Pool[Index];

		//If the GPU is done with it and no-one has a lock on it
		if (Page->FastAllocBuffer->GetRefCount() == 1 &&
			Page->FrameFence <= CompletedFence)
		{
			Page->Reset();
			Pool.RemoveAt(Index);
			return Page;
		}
	}

	FD3D12FastAllocatorPage* Page = new FD3D12FastAllocatorPage(PageSize);

	const ED3D12Access InitialD3D12Access = DetermineInitialBufferD3D12Access(HeapProperties.Type, &HeapProperties);
	VERIFYD3D12RESULT(Adapter->CreateBuffer(
		HeapProperties,
		GetGPUMask(),
		InitialD3D12Access,
		ED3D12ResourceStateMode::SingleState,
		InitialD3D12Access,
		PageSize,
		Page->FastAllocBuffer.GetInitReference(),
		TEXT("Fast Allocator Page")));
	Page->FastAllocBuffer->DoNotDeferDelete();

	Page->FastAllocData = Page->FastAllocBuffer->Map();

	return Page;
}

FD3D12FastAllocatorPage::~FD3D12FastAllocatorPage()
{
#if UE_MEMORY_TRACE_ENABLED
	// Matches MemoryTrace_Alloc issued from CreateBuffer call inside RequestFastAllocatorPage() function
	MemoryTrace_Free(FastAllocBuffer->GetGPUVirtualAddress(), EMemoryTraceRootHeap::VideoMemory);
#endif
}

void FD3D12FastAllocatorPage::UpdateFence()
{
	// Fence value must be updated every time the page is used to service an allocation.
	// Max() is required as fast allocator may be used from Render or RHI thread,
	// which have different fence values. See FD3D12ManualFence::GetCurrentFence() implementation.
	FD3D12Adapter* Adapter = FastAllocBuffer->GetParentDevice()->GetParentAdapter();
	FrameFence = FMath::Max(FrameFence, Adapter->GetFrameFence().GetNextFenceToSignal());
}

void FD3D12FastAllocatorPagePool::ReturnFastAllocatorPage(FD3D12FastAllocatorPage* Page)
{
	// TODO:  AFR has been removed, but I don't understand the comment, so I'm leaving it.  What did this code have to do with AFR?
	// Extend the lifetime of these resources when in AFR as other nodes might be relying on this
	Page->UpdateFence();
	Pool.Add(Page);
}

void FD3D12FastAllocatorPagePool::CleanupPages(uint64 FrameLag)
{
	if (Pool.Num() <= GD3D12FastAllocatorMinPagesToRetain)
	{
		return;
	}

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	const uint64 CompletedFence = FrameFence.GetCompletedFenceValue(/* bUpdateCachedFenceValue */ true);

	// Pages get returned to end of list, so we'll look for pages to delete, starting from the LRU
	for (int32 Index = 0; Index < Pool.Num(); Index++)
	{
		FD3D12FastAllocatorPage* Page = Pool[Index];

		//If the GPU is done with it and no-one has a lock on it
		if (Page->FastAllocBuffer->GetRefCount() == 1 &&
			Page->FrameFence + FrameLag <= CompletedFence)
		{
			Pool.RemoveAt(Index);
			delete(Page);

			// Only release at most one page per frame			
			return;
		}
	}
}

void FD3D12FastAllocatorPagePool::Destroy()
{
	for (int32 i = 0; i < Pool.Num(); i++)
	{
		//check(Pool[i]->FastAllocBuffer->GetRefCount() == 1);
		{
			FD3D12FastAllocatorPage *Page = Pool[i];
			delete(Page);
			Page = nullptr;
		}
	}

	Pool.Empty();
}

FD3D12FastConstantAllocator::FD3D12FastConstantAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask)
	: FD3D12DeviceChild(Parent)
	, FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibiltyMask)
	, UnderlyingResource(Parent)
	, Offset(GD3D12FastConstantAllocatorPageSize) // Initial offset is at end of page so that first Allocate() call triggers a page allocation
	, PageSize(GD3D12FastConstantAllocatorPageSize)
{
	check(PageSize % D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT == 0);
}

void* FD3D12FastConstantAllocator::Allocate(uint32 Bytes, FD3D12ResourceLocation& OutLocation, FD3D12ConstantBufferView* OutCBView)
{
	check(Bytes <= PageSize);

	const uint32 AlignedSize = Align(Bytes, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT);

	if (Offset + AlignedSize > PageSize)
	{
		Offset = 0;

		FD3D12Device* Device = GetParentDevice();
		FD3D12Adapter* Adapter = Device->GetParentAdapter();

		FD3D12UploadHeapAllocator& Allocator = Adapter->GetUploadHeapAllocator(Device->GetGPUIndex());
		Allocator.AllocFastConstantAllocationPage(PageSize, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT, UnderlyingResource);
	}

	OutLocation.AsFastAllocation(UnderlyingResource.GetResource(),
		AlignedSize,
		UnderlyingResource.GetGPUVirtualAddress(),
		UnderlyingResource.GetMappedBaseAddress(),
		UnderlyingResource.GetOffsetFromBaseOfResource(), // AllocUploadResource returns a suballocated resource where we're suballocating (again) from
		Offset);

	if (OutCBView)
	{
		OutCBView->CreateView(&UnderlyingResource, Offset, AlignedSize);
	}

	Offset += AlignedSize;

	return OutLocation.GetMappedBaseAddress();
}
