// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Allocation.h: A Collection of allocators
=============================================================================*/

#pragma once

#include "D3D12Resources.h"
#include "D3D12PoolAllocator.h"
#include "Misc/ScopeRWLock.h"

class FD3D12ConstantBufferView;

#if !defined(USE_BUFFER_POOL_ALLOCATOR)
#error "USE_BUFFER_POOL_ALLOCATOR is not defined"
#endif

#if !defined(USE_TEXTURE_POOL_ALLOCATOR)
#error "USE_TEXTURE_POOL_ALLOCATOR is not defined"
#endif

#define D3D12RHI_TRACK_DETAILED_STATS (PLATFORM_WINDOWS && !(UE_BUILD_TEST || UE_BUILD_SHIPPING))

const uint32 kD3D12ManualSubAllocationAlignment = 256;

class FD3D12ResourceAllocator : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	
	FD3D12ResourceAllocator(FD3D12Device* ParentDevice,
		FRHIGPUMask VisibleNodes,
		const FD3D12ResourceInitConfig& InInitConfig,
		const FString& Name,
		uint32 MaxSizeForPooling);
	~FD3D12ResourceAllocator();

	const FD3D12ResourceInitConfig& GetInitConfig() const { return InitConfig; }
	const uint32 GetMaximumAllocationSizeForPooling() const { return MaximumAllocationSizeForPooling; }

protected:

	const FD3D12ResourceInitConfig InitConfig;
	const FString DebugName;
	bool Initialized;

	// Any allocation larger than this just gets straight up allocated (i.e. not pooled).
	// These large allocations should be infrequent so the CPU overhead should be minimal
	const uint32 MaximumAllocationSizeForPooling;

	FCriticalSection CS;

#if defined(D3D12RHI_TRACK_DETAILED_STATS)
	uint32 SpaceAlignedUsed;
	uint32 SpaceActualUsed;
	uint32 NumBlocksInDeferredDeletionQueue;
	uint32 PeakUsage;
	uint32 FailedAllocationSpace;
#endif
};

//-----------------------------------------------------------------------------
//	Buddy Allocator
//-----------------------------------------------------------------------------
// Allocates blocks from a fixed range using buddy allocation method.
// Buddy allocation allows reasonably fast allocation of arbitrary size blocks
// with minimal fragmentation and provides efficient reuse of freed ranges.
// When a block is de-allocated an attempt is made to merge it with it's 
// neighbour (buddy) if it is contiguous and free.
// Based on reference implementation by MSFT: billkris

// Unfortunately the api restricts the minimum size of a placed buffer resource to 64k
#define MIN_PLACED_RESOURCE_SIZE (64 * 1024)
#define D3D_BUFFER_ALIGNMENT (64 * 1024)

#if defined(D3D12RHI_TRACK_DETAILED_STATS)
#define INCREASE_ALLOC_COUNTER(A, B) (A = A + B);
#define DECREASE_ALLOC_COUNTER(A, B) (A = A - B);
#else
#define INCREASE_ALLOC_COUNTER(A, B)
#define DECREASE_ALLOC_COUNTER(A, B)
#endif


class FD3D12BuddyAllocator : public FD3D12ResourceAllocator
{
public:	

	FD3D12BuddyAllocator(FD3D12Device* ParentDevice,
		FRHIGPUMask VisibleNodes,
		const FD3D12ResourceInitConfig& InInitConfig,
		const FString& Name,
		EResourceAllocationStrategy InAllocationStrategy,
		uint32 MaxSizeForPooling,
		uint32 InMaxBlockSize,
		uint32 InMinBlockSize,
		HeapId InTraceParentHeapId);

	bool TryAllocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation);

	void Deallocate(FD3D12ResourceLocation& ResourceLocation);

	void Initialize();

	void Destroy();

	void CleanUpAllocations();

	void DumpAllocatorStats(class FOutputDevice& Ar);
	void UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount);

	void ReleaseAllResources();

	void Reset();

	inline bool IsEmpty()
	{
		return FreeBlocks[MaxOrder].Num() == 1;
	}
	inline uint64 GetLastUsedFrameFence() const { return LastUsedFrameFence; }

	inline uint32 GetTotalSizeUsed() const { return TotalSizeUsed; }
	inline uint64 GetAllocationOffsetInBytes(const FD3D12BuddyAllocatorPrivateData& AllocatorPrivateData) const { return uint64(AllocatorPrivateData.Offset) * MinBlockSize; }

	inline FD3D12Heap* GetBackingHeap() { check(AllocationStrategy == EResourceAllocationStrategy::kPlacedResource); return BackingHeap.GetReference(); }

	inline bool IsOwner(FD3D12ResourceLocation& ResourceLocation)
	{
		return ResourceLocation.GetAllocator() == (FD3D12BaseAllocatorType*)this;
	}

protected:

	const uint32 MaxBlockSize;
	const uint32 MinBlockSize;
	const EResourceAllocationStrategy AllocationStrategy;

	TRefCountPtr<FD3D12Resource> BackingResource;
	TRefCountPtr<FD3D12Heap> BackingHeap;

private:
	struct RetiredBlock
	{
		FD3D12Resource* PlacedResource;
		uint64 FrameFence;
		FD3D12BuddyAllocatorPrivateData Data;
#if defined(D3D12RHI_TRACK_DETAILED_STATS)
		// Actual size only used for tracking memory stats
		uint32 AllocationSize;
#endif
	};

	TArray<RetiredBlock> DeferredDeletionQueue;
	TArray<TSet<uint32>> FreeBlocks;
	uint64 LastUsedFrameFence;
	uint32 MaxOrder;
	uint32 TotalSizeUsed;
	HeapId TraceHeapId;

	bool HeapFullMessageDisplayed;

	inline uint32 SizeToUnitSize(uint32 size) const
	{
		return (size + (MinBlockSize - 1)) / MinBlockSize;
	}

	inline uint32 UnitSizeToOrder(uint32 size) const
	{
		unsigned long Result;
		_BitScanReverse(&Result, size + size - 1); // ceil(log2(size))
		return Result;
	}

	inline uint32 GetBuddyOffset(const uint32 &offset, const uint32 &size)
	{
		return offset ^ size;
	}

	uint32 OrderToUnitSize(uint32 order) const { return ((uint32)1) << order; }
	uint32 AllocateBlock(uint32 order);
	void DeallocateBlock(uint32 offset, uint32 order);

	bool CanAllocate(uint32 size, uint32 alignment);

	void DeallocateInternal(RetiredBlock& Block);

	void Allocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation);
};

//-----------------------------------------------------------------------------
//	Multi-Buddy Allocator
//-----------------------------------------------------------------------------
// Builds on top of the Buddy Allocator but covers some of it's deficiencies by
// managing multiple buddy allocator instances to better match memory usage over
// time.

class FD3D12MultiBuddyAllocator : public FD3D12ResourceAllocator
{
public:

	FD3D12MultiBuddyAllocator(FD3D12Device* ParentDevice,
		FRHIGPUMask VisibleNodes,
		const FD3D12ResourceInitConfig& InInitConfig,
		const FString& Name,
		EResourceAllocationStrategy InAllocationStrategy,
		uint32 InMaxAllocationSize,
		uint32 InDefaultPoolSize,
		uint32 InMinBlockSize,
		HeapId InTraceParentHeapId,
		ERHIPoolAllocationStrategy InPoolAllocationStrategy,
		uint16 InOverFlowPoolsCount);
	~FD3D12MultiBuddyAllocator();

	const EResourceAllocationStrategy GetAllocationStrategy() const { return AllocationStrategy; }

	bool TryAllocate(uint32 SizeInBytes, uint32 Alignment, FD3D12ResourceLocation& ResourceLocation);

	void Deallocate(FD3D12ResourceLocation& ResourceLocation);

	void Initialize();

	void Destroy();

	void CleanUpAllocations(uint64 InFrameLag);

	void DumpAllocatorStats(class FOutputDevice& Ar);
	void UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount);

	void ReleaseAllResources();

	void Reset();
	
protected:
	FD3D12BuddyAllocator* CreateNewAllocator(uint32 InMinSizeInBytes);
	void NewAllocatorAsyncAllocate();


	const EResourceAllocationStrategy AllocationStrategy;
	const uint32 MinBlockSize;
	const uint32 DefaultPoolSize;

	TArray<FD3D12BuddyAllocator*> Allocators;

private:
	HeapId TraceHeapId;

	TArray<FD3D12BuddyAllocator*> OverFlowAllocators;
	ERHIPoolAllocationStrategy PoolAllocationStrategy = ERHIPoolAllocationStrategy::Immediate;
	ERHIPoolAllocationStatus PoolAllocationStatus = ERHIPoolAllocationStatus::Idle;
	int16 OverFlowPoolsCount = 1;
};

//-----------------------------------------------------------------------------
//	FD3D12UploadHeapAllocator
//-----------------------------------------------------------------------------
// This is designed for allocation of scratch memory such as temporary staging buffers
// or shadow buffers for dynamic resources.
class FD3D12UploadHeapAllocator : public FD3D12AdapterChild, public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:

	FD3D12UploadHeapAllocator(FD3D12Adapter* InParent, FD3D12Device* InParentDevice, const FString& InName);

	void Init();
	void Destroy();

	// Allocates <size> bytes from the end of an available resource heap.
	void* AllocUploadResource(uint32 InSize, uint32 InAlignment, FD3D12ResourceLocation& ResourceLocation);
	void* AllocFastConstantAllocationPage(uint32 InSize, uint32 InAlignment, FD3D12ResourceLocation& ResourceLocation);

	void CleanUpAllocations(uint64 InFrameLag);
	void UpdateMemoryStats();

private:
	HeapId						TraceHeapId;
	// Buddy allocator used for all 'small' allocation - fast but aligns to power of 2
	FD3D12MultiBuddyAllocator	SmallBlockAllocator;
	// Pool allocator for all bigger allocations - less fast but less alignment waste
	FCriticalSection			BigBlockCS;
	FD3D12PoolAllocator			BigBlockAllocator;
	// Separate buddy allocator used for the fast constant allocator pages which get always freed within the same frame by default
	// (different allocator to avoid fragmentation with the other pools - always the same size allocations)
	FD3D12MultiBuddyAllocator	FastConstantPageAllocator;
};

//-----------------------------------------------------------------------------
//	FD3D12DefaultBufferPool
//-----------------------------------------------------------------------------
class FD3D12DefaultBufferPool : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12DefaultBufferPool(FD3D12Device* InParent, FD3D12MultiBuddyAllocator* InAllocator);
	~FD3D12DefaultBufferPool() { delete Allocator; }

	bool SupportsAllocation(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment) const;
	void AllocDefaultResource(
		D3D12_HEAP_TYPE InHeapType,
		const D3D12_RESOURCE_DESC& InDesc,
		EBufferUsageFlags InBufferUsage,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InCreateD3D12Access,
		uint32 InAlignment,
		const TCHAR* InName,
		FD3D12ResourceLocation& ResourceLocation);
	void CleanUpAllocations(uint64 InFrameLag);
	void UpdateMemoryStats(uint32& IOMemoryAllocated, uint32& IOMemoryUsed, uint32& IOMemoryFree, uint32& IOMemoryEndFree, uint32& IOAlignmentWaste, uint32& IOAllocatedPageCount, uint32& IOFullPageCount);

	static FD3D12ResourceInitConfig GetResourceAllocatorInitConfig(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage);
	static EResourceAllocationStrategy GetResourceAllocationStrategy(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment);

private:
	FD3D12MultiBuddyAllocator* Allocator;
};

#if USE_BUFFER_POOL_ALLOCATOR
typedef FD3D12PoolAllocator FD3D12BufferPool;
#else
typedef FD3D12DefaultBufferPool FD3D12BufferPool;
#endif

// FD3D12DefaultBufferAllocator
//
class FD3D12DefaultBufferAllocator : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12DefaultBufferAllocator(FD3D12Device* InParent, FRHIGPUMask VisibleNodes);

	// Grab a buffer from the available buffers or create a new buffer if none are available
	void AllocDefaultResource(
		D3D12_HEAP_TYPE InHeapType,
		const D3D12_RESOURCE_DESC& pDesc,
		EBufferUsageFlags InBufferUsage,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InCreateD3D12Access,
		FD3D12ResourceLocation& ResourceLocation,
		uint32 Alignment,
		const TCHAR* Name);

	void FreeDefaultBufferPools();
	void BeginFrame(FD3D12ContextArray const& Contexts);
	void CleanupFreeBlocks(uint64 InFrameLag);	
	void UpdateMemoryStats();

	static bool IsPlacedResource(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment);
	static ED3D12Access GetDefaultInitialD3D12Access(D3D12_HEAP_TYPE InHeapType, EBufferUsageFlags InBufferFlags, ED3D12ResourceStateMode InResourceStateMode);

private:

	FD3D12BufferPool* CreateBufferPool(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment);

	TArray<FD3D12BufferPool*> DefaultBufferPools;
	FCriticalSection CS;
	HeapId TraceHeapId;
};

//-----------------------------------------------------------------------------
//	Fast Allocation
//-----------------------------------------------------------------------------

struct FD3D12FastAllocatorPage
{
	FD3D12FastAllocatorPage()
		: PageSize(0)
		, NextFastAllocOffset(0)
		, FastAllocData(nullptr)
		, FrameFence(0) {};

	FD3D12FastAllocatorPage(uint32 Size)
		: PageSize(Size)
		, NextFastAllocOffset(0)
		, FastAllocData(nullptr)
		, FrameFence(0) {};

	~FD3D12FastAllocatorPage();

	void Reset()
	{
		NextFastAllocOffset = 0;
		FrameFence = 0;
	}

	void UpdateFence();

	const uint32 PageSize;
	TRefCountPtr<FD3D12Resource> FastAllocBuffer;
	uint32 NextFastAllocOffset;
	void* FastAllocData;
	uint64 FrameFence;
};

class FD3D12FastAllocatorPagePool : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12FastAllocatorPagePool(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, D3D12_HEAP_TYPE InHeapType, uint32 Size);

	FD3D12FastAllocatorPagePool(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, const D3D12_HEAP_PROPERTIES& InHeapProperties, uint32 Size);

	FD3D12FastAllocatorPage* RequestFastAllocatorPage();
	void ReturnFastAllocatorPage(FD3D12FastAllocatorPage* Page);
	void CleanupPages(uint64 FrameLag);

	inline uint32 GetPageSize() const { return PageSize; }

	inline D3D12_HEAP_TYPE GetHeapType() const { return HeapProperties.Type; }
	inline bool IsCPUWritable() const { return ::IsCPUWritable(GetHeapType(), &HeapProperties); }

	void Destroy();

protected:

	const uint32 PageSize;
	const D3D12_HEAP_PROPERTIES HeapProperties;

	TArray<FD3D12FastAllocatorPage*> Pool;
};

class FD3D12FastAllocator : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12FastAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, D3D12_HEAP_TYPE InHeapType, uint32 PageSize);
	FD3D12FastAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask, const D3D12_HEAP_PROPERTIES& InHeapProperties, uint32 PageSize);

	void* Allocate(uint32 Size, uint32 Alignment, class FD3D12ResourceLocation* ResourceLocation);
	void Destroy();

	void CleanupPages(uint64 FrameLag);

protected:
	FD3D12FastAllocatorPagePool PagePool;

	FD3D12FastAllocatorPage* CurrentAllocatorPage;

	FCriticalSection CS;
};

class FD3D12FastConstantAllocator : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12FastConstantAllocator(FD3D12Device* Parent, FRHIGPUMask VisibiltyMask);

	void* Allocate(uint32 Bytes, class FD3D12ResourceLocation& OutLocation, FD3D12ConstantBufferView* OutCBView);
	void ClearResource() { UnderlyingResource.Clear(); }

private:
	FD3D12ResourceLocation UnderlyingResource;

	uint32 Offset;
	uint32 PageSize;
};

//-----------------------------------------------------------------------------
//	FD3D12TextureAllocator
//-----------------------------------------------------------------------------

#if USE_TEXTURE_POOL_ALLOCATOR
class FD3D12TextureAllocatorPool : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12TextureAllocatorPool(FD3D12Device* Device, FRHIGPUMask VisibilityNode);

	HRESULT AllocateTexture(
		FD3D12ResourceDesc Desc,
		const D3D12_CLEAR_VALUE* ClearValue,
		EPixelFormat UEFormat,
		FD3D12ResourceLocation& TextureLocation,
		ED3D12Access InInitialD3D12Access,
		ED3D12Access InDefaultD3D12Access,
		const TCHAR* Name);
	
	void BeginFrame(FD3D12ContextArray const& Contexts);
	void CleanUpAllocations();
	void Destroy();
	bool GetMemoryStats(uint64& OutTotalAllocated, uint64& OutTotalUnused) const;

private:
	
	enum class EPoolType
	{
		ReadOnly4K,
		ReadOnly,
		RenderTarget,
		UAV,
		Count,
	};

	FD3D12PoolAllocator* PoolAllocators[(int)EPoolType::Count];
	HeapId TraceHeapId;
};
#else
class FD3D12TextureAllocator : public FD3D12MultiBuddyAllocator
{
public:
	FD3D12TextureAllocator(FD3D12Device* Device, FRHIGPUMask VisibleNodes, const FString& Name, uint32 HeapSize, D3D12_HEAP_FLAGS Flags, HeapId InTraceParentHeapId);

	~FD3D12TextureAllocator();

	HRESULT AllocateTexture(
		FD3D12ResourceDesc Desc,
		const D3D12_CLEAR_VALUE* ClearValue,
		FD3D12ResourceLocation& TextureLocation,
		ED3D12Access InInitialD3D12Access,
		ED3D12Access InDefaultD3D12Access,
		const TCHAR* Name);
};

class FD3D12TextureAllocatorPool : public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12TextureAllocatorPool(FD3D12Device* Device, FRHIGPUMask VisibilityNode);

	HRESULT AllocateTexture(
		FD3D12ResourceDesc Desc,
		const D3D12_CLEAR_VALUE* ClearValue,
		EPixelFormat UEFormat,
		FD3D12ResourceLocation& TextureLocation,
		ED3D12Access InInitialD3D12Access,
		ED3D12Access InDefaultD3D12Access,
		const TCHAR* Name);
	
	void BeginFrame(FD3D12ContextArray const& Contexts) {}
	void CleanUpAllocations() { ReadOnlyTexturePool.CleanUpAllocations(0); }

	void Destroy() { ReadOnlyTexturePool.Destroy(); }
	bool GetMemoryStats(uint64& OutTotalAllocated, uint64& OutTotalUnused) const { OutTotalAllocated = 0; OutTotalUnused = 0; return false; }

private:
	HeapId TraceHeapId;
	FD3D12TextureAllocator ReadOnlyTexturePool;
};
#endif
