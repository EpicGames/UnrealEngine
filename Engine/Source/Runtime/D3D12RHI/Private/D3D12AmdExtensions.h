// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_AMD_AGS
	#include "D3D12ThirdParty.h"

	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START
		#include <amd_ags.h>
	THIRD_PARTY_INCLUDES_END
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // WITH_AMD_AGS
