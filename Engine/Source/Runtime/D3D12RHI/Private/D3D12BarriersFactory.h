// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ID3D12Barriers.h"

#include "Containers/StaticArray.h"

#include <type_traits>

enum class ED3D12BarrierImplementationType
{
	Legacy,
	Enhanced,
	Invalid,
};

// Enforce that a parameter is a non-abstract implementation of a base class
template <typename TBase, typename T>
concept CIsImplementationOfBase =
	std::is_base_of_v<TBase, T>
	&& !std::is_abstract_v<T>;

// Must be a concrete implementation of ID3D12BarriersForAdapter or nullptr_t
template <typename T>
concept CD3D12BarriersForAdapterImpl =
	CIsImplementationOfBase<ID3D12BarriersForAdapter, T>
	|| std::is_same_v<T, nullptr_t>;

// Must be a concrete implementation of ID3D12BarriersForContext or nullptr_t
template <typename T>
concept CD3D12BarriersForContextImpl = 
	CIsImplementationOfBase<ID3D12BarriersForContext, T>
	|| std::is_same_v<T, nullptr_t>;

// Type used to bring some structure (heh) to the template parameters for TD3D12BarriersFactory
template <
	ED3D12BarrierImplementationType InImplType,
	CD3D12BarriersForAdapterImpl TAdapterImpl,
	CD3D12BarriersForContextImpl TContextImpl>
struct TD3D12BarriersFactoryEntry
{
	static constexpr ED3D12BarrierImplementationType ImplType = InImplType;
	using AdapterImplType = TAdapterImpl;
	using ContextImplType = TContextImpl;
};


// Don't define any of the *ImplTypes in this case since it's a terminator only and meaningless 
// as a factory entry. If we try to use the ImplTypes, something has gone wrong anyway.
template <>
struct TD3D12BarriersFactoryEntry<ED3D12BarrierImplementationType::Invalid, nullptr_t, nullptr_t> {};

// Used for ending the list of TFactoryEntries in the template parameters for TD3D12BarriersFactory
using FNullD3D12BarriersFactoryEntry =
	TD3D12BarriersFactoryEntry<ED3D12BarrierImplementationType::Invalid, nullptr_t, nullptr_t>;


//
// Check if a type is an instance of TD3D12BarriersFactoryEntry
//
// This is done by having a baseline struct template that accepts any type followed by
// a template-template type which we default to TD3D12BarriersFactoryEntry. This 
// baseline produces a struct that inherits from std::false_type and is what all types
// that don't follow the pattern of TTemplate<auto, typename...> will match. Template
// types that do follow the pattern of TTemplate<auto, typename...> will match the 
// specialization. That's good, but also means it would match anytime that takes that
// pattern of template parameters and specifically only want TD3D12BarriersFactoryEntry.
// To further refine the result, we then conditionally switch on if putting the 
// template-template type back together with its parameters produces the same type as
// gluing those same parameters together with TD3D12BarriersFactoryEntry. Whether the
// resulting struct inherits from std::true_type or std::false_type depends on the result 
// of that equality and ultimately answers the question of if the provided type is an
// instance of TD3D12BarriersFactoryEntry.
//
template<typename, template<auto, typename...> typename = TD3D12BarriersFactoryEntry>
struct TIsInstanceOfD3D12BarriersFactoryTemplate : std::false_type {};

template<template<auto, typename...> typename TTemplate, auto InImplType, typename... TParams>
struct TIsInstanceOfD3D12BarriersFactoryTemplate<TTemplate<InImplType, TParams...>, TTemplate> :
	std::conditional_t<
		std::is_same_v<
			TTemplate<InImplType, TParams...>,
			TD3D12BarriersFactoryEntry<InImplType, TParams...>
		>,
		std::true_type,
		std::false_type
	> {};

// Must be an instance of the TD3D12BarriersFactoryEntry template
template <typename T>
concept CD3D12BarriersFactoryEntry = TIsInstanceOfD3D12BarriersFactoryTemplate<T>::value;


//
// This factory has the task of both creating the various barrier implementation
// objects at runtime and also informing the Adapter and Context as to which types
// they should use to refer to the implementation objects. If a given platform is
// compiled with only a single implementation, then the types
//
//   - TD3D12BarriersFactory::BarriersForAdapterType
//   - TD3D12BarriersFactory::BarriersForContextType
//
// will reflect the concrete types of that single implementation. Otherwise, if 
// multiple implementations are compiled in (and therefore selectable at runtime),
// then the above types will be of the abstract interface types. This is to combat,
// at least, MSVC not being intelligent enough to de-virtualize the functions calls
// where appropriate. If you want something done...
//
// To implement a new platform, create a <PlatformPrefix>BarriersFactory.h file in 
// its D3D12RHI/Private folder. An Example would be
// Engine/Source/Runtime/D3D12RHI/Private/Windows/WindowsD3D12BarriersFactory.h
// This file needs to define the type FD3D12BarriersFactory as an instantiation
// of the TD3D12BarriersFactory.
//
// In that type definition define each barrier implementation as a 
// TD3D12BarriersFactoryEntry in the template parameters passed to 
// TD3D12BarriersFactory. Note that this list must end with 
// FNullD3D12BarriersFactoryEntry. This is to solve the trailing ',' in the case
// that a given platform can support multiple implementations. Those must also be
// compile time controllable and dealing with that trailing ',' gets messy so
// always having a trailing entry (that's ignored btw) comes in handy.
//
// This code also does its best to identify problems with its use during compile 
// time since usage may not be exactly intuitive.
//

template <CD3D12BarriersFactoryEntry... TFactoryEntries>
class TD3D12BarriersFactory
{
private:
	template <typename TFirst, typename...>
	struct TGetFirstTypeInPack { using Type = TFirst; };

	// 2 accounts for 1 entry + terminator
	static constexpr uint32 MinimumFactoryEntries = 2u;

public:
	template <CD3D12BarriersForAdapterImpl TAdapterImpl, CD3D12BarriersForContextImpl TContextImpl>
	using BarrierImpl = TTuple<TAdapterImpl, TContextImpl>;

	using BarriersForAdapterType =
		std::conditional_t<
			(sizeof...(TFactoryEntries) > MinimumFactoryEntries),
				ID3D12BarriersForAdapter,
				typename TGetFirstTypeInPack<TFactoryEntries...>::Type::AdapterImplType>;

	using BarriersForContextType =
		std::conditional_t<
			(sizeof...(TFactoryEntries) > MinimumFactoryEntries),
				ID3D12BarriersForContext,
				typename TGetFirstTypeInPack<TFactoryEntries...>::Type::ContextImplType>;

	[[nodiscard]]
	static BarriersForAdapterType* CreateBarriersForAdapter(
		ED3D12BarrierImplementationType PreferredType)
	{
		return CreateBarriersInternal<EComponentType::Adapter, TFactoryEntries...>(PreferredType);
	}

	[[nodiscard]]
	static BarriersForContextType* CreateBarriersForContext(
		ED3D12BarrierImplementationType PreferredType)
	{
		return CreateBarriersInternal<EComponentType::Context, TFactoryEntries...>(PreferredType);
	}

private:
	enum class EComponentType
	{
		Adapter,
		Context,
	};

	// If another component type is added, add a new specialization 
	// to use the same CreateBarriersInternal function for it
	template <EComponentType InCompType>
	struct CreateBarriersInternalReturnType { using Type = nullptr_t; };
	template<>
	struct CreateBarriersInternalReturnType<EComponentType::Adapter> { using Type = BarriersForAdapterType; };
	template<>
	struct CreateBarriersInternalReturnType<EComponentType::Context> { using Type = BarriersForContextType; };

	template <
		EComponentType InComponentType,
		CD3D12BarriersFactoryEntry TFirst,
		CD3D12BarriersFactoryEntry... TRest>
	[[nodiscard]]
	static CreateBarriersInternalReturnType<InComponentType>::Type* CreateBarriersInternal(
		ED3D12BarrierImplementationType PreferredType)
	{
		// TRest only contains the terminator
		// Note that this also means the last entry in the list will
		// be the default if the PreferredType cannot be be matched
		if constexpr (sizeof...(TRest) == 1)
		{
			if constexpr (InComponentType == EComponentType::Adapter)
			{
				return new TFirst::AdapterImplType();
			}
			else if constexpr (InComponentType == EComponentType::Context)
			{
				return new TFirst::ContextImplType();
			}
			else
			{
				static_assert(sizeof(InComponentType) == 0, "Unknown adapter type");
			}
		}
		else if (TFirst::ImplType == PreferredType)
		{
			if constexpr (InComponentType == EComponentType::Adapter)
			{
				return new TFirst::AdapterImplType();
			}
			else if constexpr (InComponentType == EComponentType::Context)
			{
				return new TFirst::ContextImplType();
			}
			else
			{
				static_assert(sizeof(InComponentType) == 0, "Unknown adapter type");
			}
		}
		else
		{
			return CreateBarriersInternal<InComponentType, TRest...>(PreferredType);
		}
	} //-V591

	template <typename T, typename TFirst, typename... TRest>
	static consteval bool CheckThatLastTypeInPackIs()
	{
		if constexpr (sizeof...(TRest) == 0)
		{
			return std::is_same_v<TFirst, T>;
		}
		else
		{
			return CheckThatLastTypeInPackIs<T, TRest...>();
		}
	}

	template <CD3D12BarriersFactoryEntry TFirst, CD3D12BarriersFactoryEntry... TRest>
	static consteval bool CheckThatNoImplTypeAppearsMoreThanOnce(
		TStaticArray<bool, static_cast<uint32>(ED3D12BarrierImplementationType::Invalid)> SeenTypes = {})
	{
		if constexpr (sizeof...(TRest) == 0)
		{
			return true;
		}
		else if (SeenTypes[static_cast<uint32>(TFirst::ImplType)])
		{
			return false;
		}
		else
		{
			SeenTypes[static_cast<uint32>(TFirst::ImplType)] = true;
			return CheckThatNoImplTypeAppearsMoreThanOnce<TRest...>(SeenTypes);
		}
	}

	static_assert(
		CheckThatLastTypeInPackIs<FNullD3D12BarriersFactoryEntry, TFactoryEntries...>(),
		"List of barrier implementations must end with FNullD3D12BarriersFactoryEntry!");

	static_assert(sizeof...(TFactoryEntries) >= MinimumFactoryEntries,
		"No list of barrier implementations provided!");

	static_assert(
		CheckThatNoImplTypeAppearsMoreThanOnce<TFactoryEntries...>(),
		"More than one implementation for a given type is provided. "
		"TD3D12BarriersFactory will always pick the first implementation of a given type!");
};