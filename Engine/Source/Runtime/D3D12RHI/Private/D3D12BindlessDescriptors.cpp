// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12BindlessDescriptors.h"
#include "D3D12RHIPrivate.h"
#include "D3D12Descriptors.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

int32 GBindlessResourceDescriptorHeapSize = 1000 * 1000;
static FAutoConsoleVariableRef CVarBindlessResourceDescriptorHeapSize(
	TEXT("D3D12.Bindless.ResourceDescriptorHeapSize"),
	GBindlessResourceDescriptorHeapSize,
	TEXT("Bindless resource descriptor heap size"),
	ECVF_ReadOnly
);

static int32 GBindlessResourceDescriptorGarbageCollectLatency = 600;
static FAutoConsoleVariableRef CVarBindlessResourceDescriptorGarbageCollectLatency(
	TEXT("D3D12.Bindless.GarbageCollectLatency"),
	GBindlessResourceDescriptorGarbageCollectLatency,
	TEXT("Amount of update cycles before heap is freed"),
	ECVF_ReadOnly);

int32 GBindlessSamplerDescriptorHeapSize = 2048;
static FAutoConsoleVariableRef CVarBindlessSamplerDescriptorHeapSize(
	TEXT("D3D12.Bindless.SamplerDescriptorHeapSize"),
	GBindlessSamplerDescriptorHeapSize,
	TEXT("Bindless sampler descriptor heap size"),
	ECVF_ReadOnly
);

int32 GBindlessHeapUpdateUniqueUploadThreshold = 1024;
static FAutoConsoleVariableRef CVarBindlessHeapUpdateUniqueUploadThreshold(
	TEXT("D3D12.Bindless.HeapUpdateUniqueUploadThreshold"),
	GBindlessHeapUpdateUniqueUploadThreshold,
	TEXT("Threshold of upload index count before we make the list of update indices unique."),
	ECVF_Default
);

FD3D12DescriptorHeap* UE::D3D12BindlessDescriptors::CreateCpuHeap(FD3D12Device* InDevice, ERHIDescriptorHeapType InType, uint32 InNewNumDescriptorsPerHeap)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/BindlessDescriptorHeap/CPU"));

	const TCHAR* const HeapName = (InType == ERHIDescriptorHeapType::Standard) ? TEXT("BindlessResourcesCPU") : TEXT("BindlessSamplersCPU");

	return InDevice->GetDescriptorHeapManager().AllocateIndependentHeap(
		HeapName,
		InType,
		InNewNumDescriptorsPerHeap,
		ED3D12DescriptorHeapFlags::None
	);
}

FD3D12DescriptorHeap* UE::D3D12BindlessDescriptors::CreateGpuHeap(FD3D12Device* InDevice, ERHIDescriptorHeapType InType, uint32 InNewNumDescriptorsPerHeap)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/BindlessDescriptorHeap/GPU"));
	SCOPED_NAMED_EVENT_F(TEXT("CreateNewBindlessHeap (%d)"), FColor::Turquoise, InNewNumDescriptorsPerHeap);

	const TCHAR* const HeapName = (InType == ERHIDescriptorHeapType::Standard) ? TEXT("BindlessResources") : TEXT("BindlessSamplers");

	return InDevice->GetDescriptorHeapManager().AllocateIndependentHeap(
		HeapName,
		InType,
		InNewNumDescriptorsPerHeap,
		ED3D12DescriptorHeapFlags::GpuVisible
	);
}

void UE::D3D12BindlessDescriptors::DeferredFreeHeap(FD3D12Device* InDevice, FD3D12DescriptorHeap* InHeap)
{
	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(InHeap, FD3D12DeferredDeleteObject::EType::BindlessDescriptorHeap);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12BindlessSamplerManager

FD3D12BindlessSamplerManager::FD3D12BindlessSamplerManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator)
	: FD3D12DeviceChild(InDevice)
	, GpuHeap(UE::D3D12BindlessDescriptors::CreateGpuHeap(InDevice, ERHIDescriptorHeapType::Sampler, InAllocator.GetSamplerCapacity()))
	, Configuration(InAllocator.GetConfiguration())
{
}

void FD3D12BindlessSamplerManager::CleanupResources()
{
	GpuHeap = nullptr;
}

void FD3D12BindlessSamplerManager::InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12SamplerState* SamplerState)
{
	check(DstHandle.GetType() == ERHIDescriptorType::Sampler);

	UE::D3D12Descriptors::CopyDescriptor(GetParentDevice(), GpuHeap, DstHandle, SamplerState->OfflineDescriptor);
}

void FD3D12BindlessSamplerManager::OpenCommandList(FD3D12CommandContext& Context)
{
	if (IsBindlessEnabledForAnyGraphics(GetConfiguration()))
	{
		Context.StateCache.SetNewBindlessSamplerHeap(GetHeap());
	}
}

void FD3D12BindlessSamplerManager::CloseCommandList(FD3D12CommandContext& Context)
{
	if (IsBindlessEnabledForAnyGraphics(GetConfiguration()))
	{
		Context.StateCache.SetNewBindlessSamplerHeap(nullptr);
	}
}

FD3D12DescriptorHeap* FD3D12BindlessSamplerManager::GetExplicitHeapForContext(FD3D12CommandContext& Context) const
{
	return GetHeap();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12BindlessDescriptorAllocator

FD3D12BindlessDescriptorAllocator::FD3D12BindlessDescriptorAllocator(FD3D12Adapter* InParent)
	: FD3D12AdapterChild(InParent)
{
}

void FD3D12BindlessDescriptorAllocator::Init()
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/BindlessDescriptorAllocator"));

	BindlessConfiguration = UE::RHICore::GetBindlessConfigurationOnStartup(GMaxRHIShaderPlatform);

	MaxResourceHeapSize = GetParentAdapter()->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Standard);
	MaxSamplerHeapSize = GetParentAdapter()->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Sampler);

	check(MaxResourceHeapSize != 0 && MaxSamplerHeapSize != 0);

	if (BindlessConfiguration != ERHIBindlessConfiguration::Disabled)
	{
		uint32 NumResourceDescriptors = FMath::Max<int32>(GBindlessResourceDescriptorHeapSize, 1);
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		NumResourceDescriptors += GBindlessOnlineDescriptorHeapBlockSize;
#endif

		uint32 NumSamplerDescriptors = FMath::Max<int32>(GBindlessSamplerDescriptorHeapSize, 1);
		if (NumSamplerDescriptors > D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE)
		{
			UE_LOGF(LogD3D12RHI, Error, "D3D12.Bindless.SamplerDescriptorHeapSize was set to %d, which is higher than the D3D12 maximum of %d. Adjusting the value to prevent a crash.",
				NumSamplerDescriptors,
				D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE
			);
			NumSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;
		}

		UE_LOGF(LogD3D12RHI, Log, "Bindless is enabled (Configuration=%ls). Initializing allocators with %d Resource and %d Sampler descriptors", GetBindlessConfigurationString(BindlessConfiguration), NumResourceDescriptors, NumSamplerDescriptors);

		const TStatId ResourceStats[] =
		{
			GET_STATID(STAT_ResourceDescriptorsAllocated),
			GET_STATID(STAT_BindlessResourceDescriptorsAllocated),
		};

		ResourceAllocator = new FRHIHeapDescriptorAllocator(D3D12DescriptorTypeMaskFromHeapType(ERHIDescriptorHeapType::Standard), NumResourceDescriptors, ResourceStats);

		const TStatId SamplerStats[] =
		{
			GET_STATID(STAT_SamplerDescriptorsAllocated),
			GET_STATID(STAT_BindlessSamplerDescriptorsAllocated),
		};

		SamplerAllocator = new FRHIHeapDescriptorAllocator(D3D12DescriptorTypeMaskFromHeapType(ERHIDescriptorHeapType::Sampler), NumSamplerDescriptors, SamplerStats);

		// Add the types to the global groups
		GRHIGlobals.DescriptorTypeGroups.Emplace(ResourceAllocator->GetTypeMask());
		GRHIGlobals.DescriptorTypeGroups.Emplace(SamplerAllocator->GetTypeMask());
	}
}

FRHIDescriptorHandle FD3D12BindlessDescriptorAllocator::AllocateDescriptor(ERHIDescriptorType DescriptorType)
{
	if (DescriptorType == ERHIDescriptorType::Sampler)
	{
		if (!AreSamplersBindless())
		{
			return FRHIDescriptorHandle();
		}

		const FRHIDescriptorHandle Result = SamplerAllocator->Allocate(DescriptorType);
		check(Result.IsValid());
		return Result;
	}

	if (!AreResourcesBindless())
	{
		return FRHIDescriptorHandle();
	}

	FRHIDescriptorHandle Result = ResourceAllocator->Allocate(DescriptorType);

#if D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING
	if (!Result.IsValid())
	{
		TOptional<FRHIDescriptorAllocation> Allocation = ResizeGrowAndAllocate(1);
		Result = FRHIDescriptorHandle(DescriptorType, Allocation->StartIndex);
	}
#endif

	check(Result.IsValid());
	return Result;
}

TOptional<FRHIDescriptorAllocation> FD3D12BindlessDescriptorAllocator::AllocateDescriptors(ERHIDescriptorTypeMask DescriptorTypeMask, uint32 DescriptorCount)
{
	if (SamplerAllocator && SamplerAllocator->HandlesAllocations(DescriptorTypeMask))
	{
		check(!EnumHasAnyFlags(DescriptorTypeMask, ERHIDescriptorTypeMask::Standard));
		return SamplerAllocator->Allocate(DescriptorCount);
	}

	if (ResourceAllocator && ResourceAllocator->HandlesAllocations(DescriptorTypeMask))
	{
		check(!EnumHasAnyFlags(DescriptorTypeMask, ERHIDescriptorTypeMask::Sampler));
		return ResourceAllocator->Allocate(DescriptorCount);
	}

	return TOptional<FRHIDescriptorAllocation>();
}

#if D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING
TOptional<FRHIDescriptorAllocation> FD3D12BindlessDescriptorAllocator::ResizeGrowAndAllocate(uint32 NumAllocations)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12Adapter::BindlessResourceAllocateHandle(GrowHeap));

	FScopeLock ScopeLock(&ResourceHeapsCS);

	// Once we have the lock, make sure we don't need to grow needlessly; a different thread may have resized while we were waiting.
	if (TOptional<FRHIDescriptorAllocation> Allocation = ResourceAllocator->Allocate(NumAllocations))
	{
		return Allocation;
	}

	// Grow the descriptor handle allocator
	uint32 CurrentNumDescriptors = ResourceAllocator->GetCapacity();
	uint32 NewNumDescriptors = FMath::Min<uint32>(CurrentNumDescriptors * 2, MaxResourceHeapSize);

	if (CurrentNumDescriptors == NewNumDescriptors)
	{
		UE_LOGF(LogD3D12RHI, Fatal, "Hit D3D12 device limits on descriptors when attempting to allocate larger descriptor heap of size %d.", NewNumDescriptors);
	}

	TOptional<FRHIDescriptorAllocation> Allocation = ResourceAllocator->ResizeGrowAndAllocate(NewNumDescriptors, NumAllocations);

	// Grow the CPU heaps for all devices
	for (FD3D12Device* ParentDevice : GetParentAdapter()->GetDevices())
	{
		checkSlow(ParentDevice->GetBindlessDescriptorManager().GetResourceManager() != nullptr);
		ParentDevice->GetBindlessDescriptorManager().GetResourceManager()->GrowCPUHeap(CurrentNumDescriptors, NewNumDescriptors);
	}

	return Allocation;
}
#endif

void FD3D12BindlessDescriptorAllocator::FreeDescriptor(FRHIDescriptorHandle Handle)
{
	if (Handle.IsValid())
	{
		if (SamplerAllocator && SamplerAllocator->HandlesAllocation(Handle.GetType()))
		{
			SamplerAllocator->Free(Handle);
		}
		else if (ResourceAllocator && ResourceAllocator->HandlesAllocation(Handle.GetType()))
		{
			ResourceAllocator->Free(Handle);
		}
		else
		{
			// bad configuration?
			checkNoEntry();
		}
	}
}

void FD3D12BindlessDescriptorAllocator::FreeDescriptors(FRHIDescriptorAllocation Allocation, ERHIDescriptorTypeMask TypeMask)
{
	if (SamplerAllocator && SamplerAllocator->HandlesAllocations(TypeMask))
	{
		SamplerAllocator->Free(Allocation);
	}
	else if (ResourceAllocator && ResourceAllocator->HandlesAllocations(TypeMask))
	{
		ResourceAllocator->Free(Allocation);
	}
	else
	{
		// bad configuration?
		checkNoEntry();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12BindlessResourceManager

#if !D3D12RHI_CUSTOM_BINDLESS_RESOURCE_MANAGER

FD3D12BindlessResourceManager::FD3D12BindlessResourceManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator)
	: FD3D12DeviceChild(InDevice)
	, HeapsCS(InAllocator.GetResourceHeapsCS())
	, CpuHeap(UE::D3D12BindlessDescriptors::CreateCpuHeap(InDevice, ERHIDescriptorHeapType::Standard, InAllocator.GetResourceCapacity()))
	, Configuration(InAllocator.GetConfiguration())
{	
	if (IsBindlessEnabledForAnyGraphics(GetConfiguration()))
	{
		// Always allocate a heap when full bindless
		ActiveGpuHeapIndex = AddActiveGPUHeap();
	}
}

void FD3D12BindlessResourceManager::GrowCPUHeap(uint32 OriginalNumDescriptors, uint32 NewNumDescriptors)
{
	// Allocate new cpu heap & copy over the content
	FD3D12DescriptorHeapPtr NewCpuHeap = UE::D3D12BindlessDescriptors::CreateCpuHeap(GetParentDevice(), ERHIDescriptorHeapType::Standard, NewNumDescriptors);
	UE::D3D12Descriptors::CopyDescriptors(GetParentDevice(), NewCpuHeap, CpuHeap, 0, OriginalNumDescriptors);
	CpuHeap = NewCpuHeap;

	// Only request a new active GPU heap if one was already active.
	if (ActiveGpuHeapIndex >= 0)
	{
		bRequestNewActiveGpuHeap = true;
		bCPUHeapResized = true;
	}
}

void FD3D12BindlessResourceManager::CleanupResources()
{
	CpuHeap.SafeRelease();

	ReleaseGPUHeaps();
}

void FD3D12BindlessResourceManager::ReleaseGPUHeaps()
{
	{		
		for (FGpuHeapData& GpuHeap : ActiveGpuHeaps)
		{
			if (GpuHeap.bInUse)
			{
				// Defer delete after GPU is done using it (doesn't want to be recycled anymore)
				GetParentDevice()->GetDescriptorHeapManager().DeferredFreeHeap(GpuHeap.GpuHeap);
			}
			else
			{
				GpuHeap.GpuHeap.SafeRelease();
			}
		}
		ActiveGpuHeaps.Empty();

		for (FGpuHeapData& GpuHeap : PooledGpuHeaps)
		{
			GpuHeap.GpuHeap.SafeRelease();
		}
		PooledGpuHeaps.Empty();

		SET_DWORD_STAT(STAT_D3D12BindlessResourceHeapsInUseByGPU, 0);
		SET_DWORD_STAT(STAT_D3D12BindlessResourceHeapsAllocated, 0);
		SET_DWORD_STAT(STAT_D3D12BindlessResourceHeapsActive, 0);
		SET_MEMORY_STAT(STAT_D3D12BindlessResourceHeapGPUMemoryUsage, 0);

		ActiveGpuHeapIndex = -1;
		InUseGPUHeaps = 0;
		bRequestNewActiveGpuHeap = false;
	}
}

int FD3D12BindlessResourceManager::AddActiveGPUHeap()
{
	int NewHeapIndex = ActiveGpuHeaps.Num();

	FGpuHeapData& GpuHeapData = ActiveGpuHeaps.AddDefaulted_GetRef();

	// Get GPU heap from pool?			
	if (!PooledGpuHeaps.IsEmpty())
	{
		GpuHeapData = PooledGpuHeaps.Pop(EAllowShrinking::No);
	}
	else
	{
		GpuHeapData.GpuHeap = UE::D3D12BindlessDescriptors::CreateGpuHeap(GetParentDevice(), CpuHeap->GetType(), CpuHeap->GetNumDescriptors());

		INC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsAllocated);
		INC_MEMORY_STAT_BY(STAT_D3D12BindlessResourceHeapGPUMemoryUsage, GpuHeapData.GpuHeap->GetMemorySize());
	}

	INC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsActive);

	// Copy over the current CPU state (which contains all updates and latest correct state)
	CopyCpuHeap(GpuHeapData.GpuHeap);

	GpuHeapData.bInUse = true; // mark as in use
	UpdateInUseGPUHeaps(true);

	return NewHeapIndex;
}

void FD3D12BindlessResourceManager::UpdateInUseGPUHeaps(bool bInUse)
{
	if (bInUse)
	{	
		InUseGPUHeaps++;
		MaxInUseGPUHeaps = FMath::Max(MaxInUseGPUHeaps, InUseGPUHeaps);
		INC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsInUseByGPU);
	}
	else
	{
		InUseGPUHeaps--;
		DEC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsInUseByGPU);
	}
}

void FD3D12BindlessResourceManager::GarbageCollect()
{
	FScopeLock ScopeLock(&HeapsCS);
		
	// Release all GPU heaps when bindless heaps have not been used for certain amount of time with bindless for RayTracing only (assume RayTracing disabled)
	if (GetConfiguration() == ERHIBindlessConfiguration::RayTracing && (LastUsedExplicitHeapCycle + GBindlessResourceDescriptorGarbageCollectLatency < GarbageCollectCycle))
	{
		ReleaseGPUHeaps();
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12BindlessResourceManager::GarbageCollect);

		// Update the moving window max gpu heaps and reset the working value
		MovingWindowMaxInUseGPUHeaps.PushValue(MaxInUseGPUHeaps);
		MaxInUseGPUHeaps = InUseGPUHeaps;

		// Check current moving max with extra n heaps for working space - if above set value then release from active heaps to pool
		int32 TargetActiveGPUHeaps = MovingWindowMaxInUseGPUHeaps.GetMax() + 4;
		if (ActiveGpuHeaps.Num() > TargetActiveGPUHeaps)
		{
			for (int32 HeapIndex = 0; HeapIndex < ActiveGpuHeaps.Num(); ++HeapIndex)
			{
				FGpuHeapData& GpuHeap = ActiveGpuHeaps[HeapIndex];
				if (!GpuHeap.bInUse)
				{
					GpuHeap.UpdatedIndices.Empty();
					GpuHeap.LastUsedGarbageCollectCycle = GarbageCollectCycle;

					PooledGpuHeaps.Add(GpuHeap);
					ActiveGpuHeaps.RemoveAtSwap(HeapIndex, EAllowShrinking::No);

					DEC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsActive);

					// Update the active gpu index as well when it was swapped
					if (ActiveGpuHeapIndex == ActiveGpuHeaps.Num())
					{
						ActiveGpuHeapIndex = HeapIndex;
					}

					HeapIndex--;

					// Early out if removed enough
					if (ActiveGpuHeaps.Num() <= TargetActiveGPUHeaps)
					{
						break;
					}
				}
			}
		}

		// Check which pooled heaps might need to be destroyed
		if (GBindlessResourceDescriptorGarbageCollectLatency > 0)
		{
			for (int32 HeapIndex = 0; HeapIndex < PooledGpuHeaps.Num(); ++HeapIndex)
			{
				FGpuHeapData& GpuHeap = PooledGpuHeaps[HeapIndex];
				check(!GpuHeap.bInUse);
				if ((GpuHeap.LastUsedGarbageCollectCycle + GBindlessResourceDescriptorGarbageCollectLatency <= GarbageCollectCycle))
				{
					DEC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsAllocated);
					DEC_MEMORY_STAT_BY(STAT_D3D12BindlessResourceHeapGPUMemoryUsage, GpuHeap.GpuHeap->GetMemorySize());

					GpuHeap.GpuHeap.SafeRelease();
					PooledGpuHeaps.RemoveAtSwap(HeapIndex, EAllowShrinking::No);
					HeapIndex--;
				}
			}
		}
	}

	GarbageCollectCycle++;
}

void FD3D12BindlessResourceManager::Recycle(FD3D12DescriptorHeap* DescriptorHeap)
{
	FScopeLock ScopeLock(&HeapsCS);

	bool bFound = false;
	for (FGpuHeapData& GpuHeap : ActiveGpuHeaps)
	{
		if (GpuHeap.GpuHeap == DescriptorHeap)
		{
			check(GpuHeap.bInUse);
			GpuHeap.bInUse = false;
			bFound = true;

			UpdateInUseGPUHeaps(false);

			break;
		}
	}
}

void FD3D12BindlessResourceManager::InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12View* View)
{
	if (DstHandle.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12BindlessResourceManager::InitializeDescriptor);
		
		FScopeLock ScopeLock(&HeapsCS);

		// Update both CPU and active GPU heap since it's initialization and we know the handle isn't currently in use by the GPU
		FD3D12OfflineDescriptor OfflineCpuHandle = View->GetOfflineCpuHandle();	
		UE::D3D12Descriptors::CopyDescriptor(GetParentDevice(), CpuHeap, DstHandle, OfflineCpuHandle);

		// Copy descriptor to active gpu heaps and to dirty list (needs lock because active gpu heap could be changed on RHI thread)
		if (ActiveGpuHeapIndex >= 0 && !bCPUHeapResized)
		{
			UE::D3D12Descriptors::CopyDescriptor(GetParentDevice(), ActiveGpuHeaps[ActiveGpuHeapIndex].GpuHeap, DstHandle, OfflineCpuHandle);
			ActiveGpuHeaps[ActiveGpuHeapIndex].UpdatedIndices.Add(DstHandle.GetIndex());
		}

		INC_DWORD_STAT(STAT_D3D12BindlessResourceDescriptorsInitialized);
	}
}

void FD3D12BindlessResourceManager::UpdateDescriptor(FD3D12ContextArray const& Contexts, FRHIDescriptorHandle DstHandle, const FD3D12View* View)
{
	if (DstHandle.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12BindlessResourceManager::UpdateDescriptor);
	
		FScopeLock ScopeLock(&HeapsCS);

		// Update the shared CPU heap
		UE::D3D12Descriptors::CopyDescriptor(GetParentDevice(), CpuHeap, DstHandle, View->GetOfflineCpuHandle());

		// Add to update list so it's updated for the next heap
		if (ActiveGpuHeapIndex >= 0)
		{
			// Request allocation of new heap because current GPU heap is used by GPU and can't modify handles in use
			uint32 const GPUIndex = GetParentDevice()->GetGPUIndex();
			for (FD3D12CommandContextBase* ContextBase : Contexts)
			{
				if (ContextBase)
				{
					FD3D12CommandContext& Context = *ContextBase->GetSingleDeviceContext(GPUIndex);
					Context.GetBindlessState().RefreshDescriptorHeap();
					check(!Context.GetExecutingCommandList().AllowParallelTranslate());
				}
			}

			bRequestNewActiveGpuHeap = true;
			ActiveGpuHeaps[ActiveGpuHeapIndex].UpdatedIndices.Add(DstHandle.GetIndex());
		}

		INC_DWORD_STAT(STAT_D3D12BindlessResourceDescriptorsUpdated);
	}
}

void FD3D12BindlessResourceManager::FlushPendingDescriptorUpdates(FD3D12CommandContext& Context)
{
	FD3D12ContextBindlessState& State = Context.GetBindlessState();

	// Create a new heap because there have been descriptor updates?
	if (State.bRefreshHeap || bRequestNewActiveGpuHeap)
	{
		// First finalize the previous heap if it was set.
		FinalizeHeapOnState(State);

		// Then assign the current heap to the state
		AssignHeapToState(State);

		if (IsBindlessEnabledForAnyGraphics(GetConfiguration()) && ensure(Context.IsOpen()))
		{
			// Finally tell the Context that we're using this heap,
			// this call also makes sure the heap is set on the d3d command list.
			Context.StateCache.SetNewBindlessResourcesHeap(State.CurrentGpuHeap);
		}
	}
}

void FD3D12BindlessResourceManager::OpenCommandList(FD3D12CommandContext& Context)
{
	FD3D12ContextBindlessState& State = Context.GetBindlessState();

	// Assign the current active Gpu heap to the context
	AssignHeapToState(State);

	if (IsBindlessEnabledForAnyGraphics(GetConfiguration()))
	{
		// Assign the heap to the descriptor cache
		Context.StateCache.SetNewBindlessResourcesHeap(State.CurrentGpuHeap);
	}
}

void FD3D12BindlessResourceManager::CloseCommandList(FD3D12CommandContext& Context)
{
	FD3D12ContextBindlessState& State = Context.GetBindlessState();

	// First finalize the current heap if any was set
	FinalizeHeapOnState(State);

	if (IsBindlessEnabledForAnyGraphics(GetConfiguration()))
	{
		// Then clear the reference from the state cache
		Context.StateCache.SetNewBindlessResourcesHeap(nullptr);
	}
}

void FD3D12BindlessResourceManager::FinalizeContext(FD3D12CommandContext& Context)
{
	if (Context.IsOpen())
	{
		Context.CloseCommandList();
	}

	FD3D12ContextBindlessState& State = Context.GetBindlessState();

	// If context wasn't opened but did have descriptor updates make sure the shared gpu heap is updated
	// (can happen due to texture reference updates not adding any real GPU work)
	FinalizeHeapOnState(State);
}

FD3D12DescriptorHeap* FD3D12BindlessResourceManager::GetHeap(ERHIPipeline Pipeline) const
{
	checkNoEntry();
	return nullptr;
}

FD3D12DescriptorHeap* FD3D12BindlessResourceManager::GetExplicitHeapForContext(FD3D12CommandContext& Context)
{
	FD3D12ContextBindlessState& State = Context.GetBindlessState();

	// Assign GPU heap when it's still unassigned (can happen when RT only and not been used yet - will get full copy of updated CPU state)
	if (State.CurrentGpuHeap == nullptr && GetConfiguration() == ERHIBindlessConfiguration::RayTracing)
	{
		FScopeLock ScopeLock(&HeapsCS);
		ActiveGpuHeapIndex = AddActiveGPUHeap();
		State.CurrentGpuHeap = ActiveGpuHeaps[ActiveGpuHeapIndex].GpuHeap;
	}

	LastUsedExplicitHeapCycle = GarbageCollectCycle;
	check(State.CurrentGpuHeap);
	return State.CurrentGpuHeap;
}

void FD3D12BindlessResourceManager::CopyCpuHeap(FD3D12DescriptorHeap* DestinationHeap)
{
	// Copy the smallest possible set of descriptors from the CPU heap to the new GPU heap.
	FRHIDescriptorAllocatorRange AllocatedRange(0, 0);
	if (GetParentDevice()->GetBindlessDescriptorAllocator().GetResourceAllocatedRange(AllocatedRange))
	{
		const uint32 NumDescriptorsToCopy = AllocatedRange.Last - AllocatedRange.First + 1;
		UE::D3D12Descriptors::CopyDescriptors(GetParentDevice(), DestinationHeap, CpuHeap, AllocatedRange.First, NumDescriptorsToCopy);

		INC_DWORD_STAT_BY(STAT_D3D12BindlessResourceGPUDescriptorsCopied, NumDescriptorsToCopy);
	}
}

void FD3D12BindlessResourceManager::AssignHeapToState(FD3D12ContextBindlessState& State)
{
	checkf(State.CurrentGpuHeap == nullptr, TEXT("FinalizeHeapOnState was not called before AssignHeapToState"));

	FScopeLock ScopeLock(&HeapsCS);

	// Do we have a heap allocated, then assign
	if (ActiveGpuHeapIndex >= 0)
	{	
		// By default use the active GPU heap (will be versioned when needed during update while GPU is using it)
		State.CurrentGpuHeap = ActiveGpuHeaps[ActiveGpuHeapIndex].GpuHeap;
	}
	else
	{
		// Should always have a heap when running with full bindless
		check(GetConfiguration() != ERHIBindlessConfiguration::All);
	}
}

void FD3D12BindlessResourceManager::FinalizeHeapOnState(FD3D12ContextBindlessState& State)
{
	// Possibly version the GPU heap if it not requested by another queue yet
	CheckRequestNewActiveGPUHeap();

	// Clear the state data
	State.CurrentGpuHeap = nullptr;
	State.bRefreshHeap = false;
}

void FD3D12BindlessResourceManager::CheckRequestNewActiveGPUHeap()
{
	if (!bRequestNewActiveGpuHeap)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12BindlessResourceManager::RequestNewActiveGPUHeap);

	FScopeLock ScopeLock(&HeapsCS);

	if (!bRequestNewActiveGpuHeap)
	{
		return;
	}
	
	int32 NewActiveGpuHeapIndex = -1;
	if (bCPUHeapResized)
	{
		// Resizing the heap size then free all current allocated GPU heaps
		ReleaseGPUHeaps();
	}
	else
	{
		// Update the the last used garbage collect cycle before moving over to a new heap
		ActiveGpuHeaps[ActiveGpuHeapIndex].LastUsedGarbageCollectCycle = GarbageCollectCycle;
		
		// Queue the heap for recycle when the GPU is done using it
		UE::D3D12BindlessDescriptors::DeferredFreeHeap(GetParentDevice(), ActiveGpuHeaps[ActiveGpuHeapIndex].GpuHeap);

		int32 NumGpuHeaps = ActiveGpuHeaps.Num();

		// Copy over dirty handles to all other heaps so they are updated when reused as well
		for (int32 GpuHeapIndex = 0; GpuHeapIndex < NumGpuHeaps; ++GpuHeapIndex)
		{
			if (GpuHeapIndex != ActiveGpuHeapIndex)
			{
				ActiveGpuHeaps[GpuHeapIndex].UpdatedIndices.Append(ActiveGpuHeaps[ActiveGpuHeapIndex].UpdatedIndices);
			}
		}

		// Try and reuse a pooled heap (incremented from last used to reduce the possible spike on reuse of lots of heap and dirty handle increase)
		for (int32 NextIndex = 1; NextIndex < NumGpuHeaps; ++NextIndex)
		{
			int32 GpuHeapIndex = (ActiveGpuHeapIndex + NextIndex) % NumGpuHeaps;

			// Not used by the GPU anymore and not the current one
			if (GpuHeapIndex != ActiveGpuHeapIndex && !ActiveGpuHeaps[GpuHeapIndex].bInUse)
			{
				NewActiveGpuHeapIndex = GpuHeapIndex;
				break;
			}
		}
	}

	// Found a pooled heap, then copy over the dirty descriptor handles
	if (NewActiveGpuHeapIndex >= 0)
	{
		// NOTE: For small update lists, copying duplicates is faster than deduplicating.
		//       For large lists (see GBindlessHeapUpdateUniqueUploadThreshold), we sort + unique
		//       to avoid excessive driver overhead from large CopyDescriptors calls.

		FGpuHeapData& GpuHeapData = ActiveGpuHeaps[NewActiveGpuHeapIndex];
		INC_DWORD_STAT_BY(STAT_D3D12BindlessResourceGPUDescriptorsCopied, GpuHeapData.UpdatedIndices.Num());

		// Deduplicate the updated indices when the list is large enough that sorting + unique
		// is cheaper than copying redundant descriptors. The first condition catches pathological
		// cases where accumulated updates exceed the heap size (guaranteed duplicates).
		if (GpuHeapData.UpdatedIndices.Num() > (int32)GpuHeapData.GpuHeap->GetNumDescriptors() ||
			GpuHeapData.UpdatedIndices.Num() > GBindlessHeapUpdateUniqueUploadThreshold)
		{
			Algo::Sort(GpuHeapData.UpdatedIndices);
			GpuHeapData.UpdatedIndices.SetNum(Algo::Unique(GpuHeapData.UpdatedIndices));
		}

		UE::D3D12Descriptors::CopyDescriptors(GetParentDevice(), GpuHeapData.GpuHeap, CpuHeap, GpuHeapData.UpdatedIndices);
		GpuHeapData.UpdatedIndices.Reset();

		// Mark in use by GPU again
		GpuHeapData.bInUse = true;
		UpdateInUseGPUHeaps(true);
	}
	else
	{
		NewActiveGpuHeapIndex = AddActiveGPUHeap();
	}

	// clear the request
	bRequestNewActiveGpuHeap = false;
	bCPUHeapResized = false;

	// Update the active gpu index
	ActiveGpuHeapIndex = NewActiveGpuHeapIndex;
	INC_DWORD_STAT(STAT_D3D12BindlessResourceHeapsVersioned);
}

#endif // D3D12RHI_CUSTOM_BINDLESS_RESOURCE_MANAGER

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12BindlessDescriptorManager

FD3D12BindlessDescriptorManager::FD3D12BindlessDescriptorManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator)
	: FD3D12DeviceChild(InDevice)
	, Allocator(InAllocator)
{
}

FD3D12BindlessDescriptorManager::~FD3D12BindlessDescriptorManager() = default;

void FD3D12BindlessDescriptorManager::Init()
{
	Configuration = Allocator.GetConfiguration();

	if (Configuration != ERHIBindlessConfiguration::Disabled)
	{
		ResourceManager = MakeUnique<FD3D12BindlessResourceManager>(GetParentDevice(), Allocator);
		SamplerManager = MakeUnique<FD3D12BindlessSamplerManager>(GetParentDevice(), Allocator);
	}
}

void FD3D12BindlessDescriptorManager::CleanupResources()
{
	if (ResourceManager)
	{
		ResourceManager->CleanupResources();
	}

	if (SamplerManager)
	{
		SamplerManager->CleanupResources();
	}
}

void FD3D12BindlessDescriptorManager::GarbageCollect()
{
	if (ResourceManager)
	{
		ResourceManager->GarbageCollect();
	}
}

void FD3D12BindlessDescriptorManager::Recycle(FD3D12DescriptorHeap* DescriptorHeap)
{
	if (ResourceManager)
	{
		ResourceManager->Recycle(DescriptorHeap);
	}
}

void FD3D12BindlessDescriptorManager::ImmediateFree(FRHIDescriptorHandle InHandle)
{
	Allocator.FreeDescriptor(InHandle);
}

void FD3D12BindlessDescriptorManager::DeferredFreeFromDestructor(FRHIDescriptorHandle InHandle)
{
	if (InHandle.IsValid())
	{
		FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(InHandle, GetParentDevice());
	}
}

void FD3D12BindlessDescriptorManager::InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12SamplerState* SamplerState)
{
	if (SamplerManager)
	{
		SamplerManager->InitializeDescriptor(DstHandle, SamplerState);
		return;
	}

	// Bad configuration?
	checkNoEntry();
}

void FD3D12BindlessDescriptorManager::InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12View* View)
{
	if (ResourceManager && ResourceManager->HandlesAllocation(DstHandle.GetType()))
	{
		ResourceManager->InitializeDescriptor(DstHandle, View);
		return;
	}

	// Bad configuration?
	checkNoEntry();
}

void FD3D12BindlessDescriptorManager::UpdateDescriptor(FD3D12ContextArray const& Contexts, FRHIDescriptorHandle DstHandle, const FD3D12View* View)
{
	if (ResourceManager)
	{
		ResourceManager->UpdateDescriptor(Contexts, DstHandle, View);
		return;
	}

	// Bad configuration?
	checkNoEntry();
}

void FD3D12BindlessDescriptorManager::FinalizeContext(FD3D12CommandContext& Context)
{
	if (ResourceManager)
	{
		ResourceManager->FinalizeContext(Context);
	}
}

void FD3D12BindlessDescriptorManager::OpenCommandList(FD3D12CommandContext& Context)
{
	if (ResourceManager)
	{
		ResourceManager->OpenCommandList(Context);
	}

	if (SamplerManager)
	{
		SamplerManager->OpenCommandList(Context);
	}
}

void FD3D12BindlessDescriptorManager::CloseCommandList(FD3D12CommandContext& Context)
{
	if (ResourceManager)
	{
		ResourceManager->CloseCommandList(Context);
	}

	if (SamplerManager)
	{
		SamplerManager->CloseCommandList(Context);
	}
}

void FD3D12BindlessDescriptorManager::FlushPendingDescriptorUpdates(FD3D12CommandContext& Context)
{
	if (ResourceManager)
	{
		ResourceManager->FlushPendingDescriptorUpdates(Context);
	}
}

FD3D12DescriptorHeapPair FD3D12BindlessDescriptorManager::GetExplicitHeapsForContext(FD3D12CommandContext& Context, ERHIBindlessConfiguration InConfiguration)
{
	FD3D12DescriptorHeapPair Result{};

	if (GetConfiguration() != ERHIBindlessConfiguration::Disabled && GetConfiguration() >= InConfiguration)
	{
		Result.ResourceHeap = ResourceManager->GetExplicitHeapForContext(Context);
		Result.SamplerHeap = SamplerManager->GetExplicitHeapForContext(Context);
	}

	return Result;
}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
TRHIPipelineArray<FD3D12DescriptorHeapPtr> FD3D12BindlessDescriptorManager::AllocateResourceHeapsForAllPipelines(int32 InSize)
{
	if (ResourceManager)
	{
		return ResourceManager->AllocateResourceHeapsForAllPipelines(InSize);
	}

	// Bad configuration?
	checkNoEntry();
	return TRHIPipelineArray<FD3D12DescriptorHeapPtr>();
}
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
