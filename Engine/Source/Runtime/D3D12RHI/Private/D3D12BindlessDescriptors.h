// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#include "D3D12Descriptors.h"
#include "RHIDefinitions.h"
#include "RHIDescriptorAllocator.h"
#include "Templates/RefCounting.h"

class FRHICommandListBase;
class FD3D12SamplerState;
class FD3D12ShaderResourceView;
class FD3D12UnorderedAccessView;

struct FD3D12Payload;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

#include COMPILED_PLATFORM_HEADER(D3D12BindlessDescriptors.h)

#if !defined(D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING)
	#error D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING needs to be defined
#endif

namespace UE::D3D12BindlessDescriptors
{
	FD3D12DescriptorHeap* CreateCpuHeap(FD3D12Device* InDevice, ERHIDescriptorHeapType InType, uint32 InNewNumDescriptorsPerHeap);
	FD3D12DescriptorHeap* CreateGpuHeap(FD3D12Device* InDevice, ERHIDescriptorHeapType InType, uint32 InNewNumDescriptorsPerHeap);
	void DeferredFreeHeap(FD3D12Device* InDevice, FD3D12DescriptorHeap* InHeap);
}

/** Manager for configuration settings and shared descriptor allocators, stored on the adapter. */
class FD3D12BindlessDescriptorAllocator : public FD3D12AdapterChild
{
public:
	FD3D12BindlessDescriptorAllocator() = delete;
	FD3D12BindlessDescriptorAllocator(FD3D12Adapter* InParent);

	void Init();

	ERHIBindlessConfiguration GetConfiguration() const { return BindlessConfiguration; }

	bool AreResourcesBindless() const { return ResourceAllocator != nullptr; }
	bool AreSamplersBindless() const { return SamplerAllocator != nullptr; }

	// Bindless descriptor allocators are stored in the adapter, so descriptor handles can be allocated once and shared for multi-GPU objects
	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType);
	TOptional<FRHIDescriptorAllocation> AllocateDescriptors(ERHIDescriptorTypeMask DescriptorTypeMask, uint32 DescriptorCount);

	void FreeDescriptor(FRHIDescriptorHandle Handle);
	void FreeDescriptors(FRHIDescriptorAllocation Allocation, ERHIDescriptorTypeMask TypeMask);

	FCriticalSection& GetResourceHeapsCS() { return ResourceHeapsCS; }

	uint32 GetResourceCapacity() const { return ResourceAllocator->GetCapacity(); }
	uint32 GetSamplerCapacity()  const { return SamplerAllocator->GetCapacity(); }

	bool GetResourceAllocatedRange(FRHIDescriptorAllocatorRange& OutAllocatedRange) const { return ResourceAllocator->GetAllocatedRange(OutAllocatedRange); }

#if D3D12RHI_CUSTOM_BINDLESS_RESOURCE_MANAGER
	FRHIHeapDescriptorAllocator* GetResourceAllocator() { return ResourceAllocator; }
#endif

private:
#if D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING
	TOptional<FRHIDescriptorAllocation> ResizeGrowAndAllocate(uint32 NumAllocations);
#endif

	ERHIBindlessConfiguration BindlessConfiguration{};

	uint32 MaxResourceHeapSize = 0;
	uint32 MaxSamplerHeapSize = 0;

	FCriticalSection ResourceHeapsCS;
	FRHIHeapDescriptorAllocator* ResourceAllocator = nullptr;
	FRHIHeapDescriptorAllocator* SamplerAllocator = nullptr;
};

/** Manager specifically for bindless sampler descriptors. */
class FD3D12BindlessSamplerManager : public FD3D12DeviceChild
{
public:
	FD3D12BindlessSamplerManager() = delete;
	FD3D12BindlessSamplerManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator);

	void CleanupResources();

	void InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12SamplerState* SamplerState);

	void OpenCommandList(FD3D12CommandContext& Context);
	void CloseCommandList(FD3D12CommandContext& Context);

	FD3D12DescriptorHeap* GetExplicitHeapForContext(FD3D12CommandContext& Context) const;

	FD3D12DescriptorHeap* GetHeap() const { return GpuHeap.GetReference(); }
	ERHIBindlessConfiguration GetConfiguration() const { return Configuration; }

	constexpr ERHIDescriptorTypeMask GetTypeMask() const
	{
		return ERHIDescriptorTypeMask::Sampler;
	}

	constexpr bool HandlesAllocation(ERHIDescriptorType InType) const
	{
		return EnumHasAnyFlags(GetTypeMask(), RHIDescriptorTypeMaskFromType(InType));
	}

private:
	FD3D12DescriptorHeapPtr      GpuHeap;
	ERHIBindlessConfiguration    Configuration;
};

#if !D3D12RHI_CUSTOM_BINDLESS_RESOURCE_MANAGER

// Helper container for all context related bindless state.
struct FD3D12ContextBindlessState
{
	FD3D12DescriptorHeapPtr CurrentGpuHeap;
	bool                    bRefreshHeap = false;

	FD3D12ContextBindlessState() = default;
	~FD3D12ContextBindlessState()
	{
		check(!bRefreshHeap);
	}

	void RefreshDescriptorHeap()
	{
		bRefreshHeap = true;
	}
};

/** Simple helper class to compute moving max in given amount of values. */
template <typename T, int32 ArraySize>
class FMovingWindowMax
{
public:
	FMovingWindowMax()
	: RemoveNextIdx(0)
	, NumValuesUsed(0)
	{
		static_assert(ArraySize > 0, "ArraySize must be greater than zero");
	}

	void PushValue(T Value)
	{
		if (ArraySize == NumValuesUsed)
		{
			ValuesArray[RemoveNextIdx] = Value;
			RemoveNextIdx = (RemoveNextIdx + 1) % ArraySize;
		}
		else
		{
			ValuesArray[NumValuesUsed] = Value;
			++NumValuesUsed;
		}
	}

	T GetMax() const
	{		
		T Max = static_cast<T>(0);
		for (int32 Index = 0; Index < NumValuesUsed; ++Index)
		{
			Max = FMath::Max(Max, ValuesArray[Index]);
		}
		return Max;
	}

private:
	TStaticArray<T, ArraySize> ValuesArray;

	/** The array Index of the next item to remove when the moving window is full */
	int32 RemoveNextIdx;
	int32 NumValuesUsed;
};

/** Manager specifically for bindless resource descriptors. Has to handle renames on command lists. */
class FD3D12BindlessResourceManager : public FD3D12DeviceChild
{
public:
	FD3D12BindlessResourceManager() = delete;
	FD3D12BindlessResourceManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator);

	void CleanupResources();
	
	void GarbageCollect();
	void Recycle(FD3D12DescriptorHeap* DescriptorHeap);

	void InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12View* View);
	void UpdateDescriptor(FD3D12ContextArray const& Contexts, FRHIDescriptorHandle DstHandle, const FD3D12View* View);

	void FlushPendingDescriptorUpdates(FD3D12CommandContext& Context);

	void OpenCommandList(FD3D12CommandContext& Context);
	void CloseCommandList(FD3D12CommandContext& Context);
	void FinalizeContext(FD3D12CommandContext& Context);

	FD3D12DescriptorHeap* GetHeap(ERHIPipeline Pipeline) const;
	FD3D12DescriptorHeap* GetExplicitHeapForContext(FD3D12CommandContext& Context);

	ERHIBindlessConfiguration GetConfiguration() const { return Configuration; }

	// Called from FD3D12Adapter::AllocateBindlessResourceHandle
	void GrowCPUHeap(uint32 OriginalNumDescriptors, uint32 NewNumDescriptors);

	constexpr ERHIDescriptorTypeMask GetTypeMask() const
	{
		return ERHIDescriptorTypeMask::CBV | ERHIDescriptorTypeMask::SRV | ERHIDescriptorTypeMask::UAV;
	}

	constexpr bool HandlesAllocation(ERHIDescriptorType InType) const
	{
		return EnumHasAnyFlags(GetTypeMask(), RHIDescriptorTypeMaskFromType(InType));
	}

private:
	void CopyCpuHeap(FD3D12DescriptorHeap* DestinationHeap);
	void AssignHeapToState(FD3D12ContextBindlessState& State);
	void FinalizeHeapOnState(FD3D12ContextBindlessState& State);

	void CheckRequestNewActiveGPUHeap();
	int AddActiveGPUHeap();
	void ReleaseGPUHeaps();
	void UpdateInUseGPUHeaps(bool bInUse);
	
	// Critical section shared across devices
	FCriticalSection&				HeapsCS;
	FD3D12DescriptorHeapPtr         CpuHeap;
	const ERHIBindlessConfiguration Configuration;

	uint64 							GarbageCollectCycle = 0;
	uint64							LastUsedExplicitHeapCycle = 0;

	bool							bRequestNewActiveGpuHeap = false;
	bool							bCPUHeapResized = false;

	uint32							InUseGPUHeaps = 0;
	uint32							MaxInUseGPUHeaps = 0;
	FMovingWindowMax<uint32, 100>	MovingWindowMaxInUseGPUHeaps;

	struct FGpuHeapData
	{
		FD3D12DescriptorHeapPtr       GpuHeap;
		TArray<FD3D12DescriptorIndex> UpdatedIndices;
		bool                          bInUse = true;
		uint64                        LastUsedGarbageCollectCycle = 0;
	};		
		
	int32							ActiveGpuHeapIndex = -1;
	TArray<FGpuHeapData>			ActiveGpuHeaps;
	TArray<FGpuHeapData>			PooledGpuHeaps;
};

#endif

struct FD3D12DescriptorHeapPair
{
	FD3D12DescriptorHeap* SamplerHeap;
	FD3D12DescriptorHeap* ResourceHeap;
};

/** Manager for descriptors used in bindless rendering. */
class FD3D12BindlessDescriptorManager : public FD3D12DeviceChild
{
public:
	FD3D12BindlessDescriptorManager(FD3D12Device* InDevice, FD3D12BindlessDescriptorAllocator& InAllocator);
	~FD3D12BindlessDescriptorManager();

	void Init();
	void CleanupResources();

	FD3D12BindlessDescriptorAllocator& GetAllocator()
	{
		return Allocator;
	}

	FD3D12BindlessResourceManager* GetResourceManager() const
	{
		return ResourceManager.Get();
	}

	FD3D12BindlessSamplerManager* GetSamplerManager()  const
	{
		return SamplerManager.Get();
	}

	ERHIBindlessConfiguration GetConfiguration() const
	{
		return Configuration;
	}

	void ImmediateFree(FRHIDescriptorHandle InHandle);
	void DeferredFreeFromDestructor(FRHIDescriptorHandle InHandle);

	void GarbageCollect();
	void Recycle(FD3D12DescriptorHeap* DescriptorHeap);

	void InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12SamplerState* SamplerState);
	void InitializeDescriptor(FRHIDescriptorHandle DstHandle, const FD3D12View* View);
	void UpdateDescriptor(FD3D12ContextArray const& Contexts, FRHIDescriptorHandle DstHandle, const FD3D12View* SourceView);

	void FinalizeContext(FD3D12CommandContext& Context);

	void OpenCommandList(FD3D12CommandContext& Context);
	void CloseCommandList(FD3D12CommandContext& Context);

	void FlushPendingDescriptorUpdates(FD3D12CommandContext& Context);
	void SetHeapsForRayTracing(FD3D12CommandContext& Context);

	FD3D12DescriptorHeapPair GetExplicitHeapsForContext(FD3D12CommandContext& Context, ERHIBindlessConfiguration InConfiguration);

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	TRHIPipelineArray<FD3D12DescriptorHeapPtr> AllocateResourceHeapsForAllPipelines(int32 InSize);
#endif

private:
	FD3D12BindlessDescriptorAllocator& Allocator;

	TUniquePtr<FD3D12BindlessResourceManager> ResourceManager;
	TUniquePtr<FD3D12BindlessSamplerManager>  SamplerManager;

	ERHIBindlessConfiguration Configuration{};
};

#endif // !PLATFORM_SUPPORTS_BINDLESS_RENDERING
