// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Buffer.cpp: D3D Common code for buffers.
=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreBufferInitializer.h"
#include "RHICoreStats.h"

extern int32 GD3D12BindResourceLabels;

FD3D12Buffer::~FD3D12Buffer()
{
	if (EnumHasAnyFlags(GetUsage(), EBufferUsageFlags::VertexBuffer) && GetParentDevice())
	{
		FD3D12CommandContext& DefaultContext = GetParentDevice()->GetDefaultCommandContext();
		DefaultContext.StateCache.ClearVertexBuffer(&ResourceLocation);
	}

	bool bTransient = ResourceLocation.IsTransient();
	if (!bTransient)
	{
		D3D12BufferStats::UpdateBufferStats(*this, false);
	}
}

void FD3D12Buffer::UploadResourceData(
	FD3D12CommandContext& CommandContext,
	ED3D12Access InDestinationD3D12Access,
	FD3D12ResourceLocation& DestinationResourceLocation,
	const FD3D12ResourceLocation& SourceResourceLocation,
	uint32 Size)
{
	FD3D12Resource* Destination = DestinationResourceLocation.GetResource();
	check(Destination->GetDesc().Dimension == D3D12_RESOURCE_DIMENSION_BUFFER);

	// Copy from the temporary upload heap to the default resource

	const ERHIPipeline Pipe = CommandContext.GetPipeline();
	const ED3D12Access InitialAccess = Destination->GetInitialAccess();

	// if resource doesn't require state tracking then transition to copy dest here (could have been suballocated from shared resource) - not very optimal and should be batched
	if (!Destination->RequiresResourceStateTracking())
	{
		CommandContext.AddBarrier(Destination, InitialAccess, ED3D12Access::CopyDest, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
	}

	CommandContext.FlushResourceBarriers();

	CommandContext.CopyBufferRegionChecked(
		Destination->GetResource(), Destination->GetName(),
		DestinationResourceLocation.GetOffsetFromBaseOfResource(),
		SourceResourceLocation.GetResource()->GetResource(), FName(),
		SourceResourceLocation.GetOffsetFromBaseOfResource(),
		Size);

	// Update the resource state after the copy has been done (will take care of updating the residency as well)
	if (InDestinationD3D12Access != ED3D12Access::CopyDest)
	{
		CommandContext.AddBarrier(Destination, ED3D12Access::CopyDest, InDestinationD3D12Access, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
	}

	CommandContext.UpdateResidency(SourceResourceLocation.GetResource());
	CommandContext.UpdateResidency(Destination);

	CommandContext.ConditionalSplitCommandList();

	// If the resource is untracked, the destination state must match the default state of the resource.
	check(Destination->RequiresResourceStateTracking() || (Destination->GetDefaultAccess() == InDestinationD3D12Access));

	// Buffer is now written and ready, so unlock the block (locked after creation and can be defragmented if needed)
	DestinationResourceLocation.UnlockPoolData();
}

static FD3D12ResourceLocation AllocateUploadMemory(FD3D12Device* Device, uint32 Size, uint32 Alignment)
{
	FD3D12ResourceLocation Location(Device);
	if (!IsInRHIThread() && !IsInRenderingThread())
	{
		Device->GetParentAdapter()->GetUploadHeapAllocator(Device->GetGPUIndex()).AllocUploadResource(Size, Alignment, Location);
	}
	else
	{
		Device->GetDefaultFastAllocator().Allocate(Size, Alignment, &Location);
	}
	return Location;
}

static FD3D12ResourceLocation AllocateUploadMemory(FD3D12Buffer* Buffer, const FRHIBufferDesc& Desc)
{
	return AllocateUploadMemory(Buffer->GetParentDevice(), Desc.Size, Buffer->BufferAlignment);
}

void FD3D12CommandContext::CopyBufferRegionChecked(
	ID3D12Resource* DestResource, const FName& DestName, uint64 DestOffset,
	ID3D12Resource* SourceResource, const FName& SourceName, uint64 SourceOffset,
	uint32 ByteCount
)
{
#if ENABLE_COPY_BUFFER_REGION_CHECK
	RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestOffset + ByteCount <= DestResource->GetDesc().Width, TEXT("Dest byte range out of bounds for: '%s'"), *DestName.ToString());
	RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, SourceOffset + ByteCount <= SourceResource->GetDesc().Width, TEXT("Source byte range out of bounds for: '%s'"), *SourceName.ToString());
#endif // ENABLE_COPY_BUFFER_REGION_CHECK
	
	// Pass down callchain
	GraphicsCommandList()->CopyBufferRegion(
		DestResource, DestOffset,
		SourceResource, SourceOffset,
		ByteCount
	);
}

FD3D12SyncPointRef FD3D12Buffer::UploadResourceDataViaCopyQueue(FD3D12CommandContext& OwningContext, FResourceArrayUploadInterface* InResourceArray)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UploadResourceDataViaCopyQueue);

	// assume not dynamic and not on async thread (probably fine but untested)
	check(IsInRHIThread() || IsInRenderingThread());
	check(!(GetUsage() & EBufferUsageFlags::AnyDynamic));

	uint32 BufferSize = GetSize();

	// Get an upload heap and copy the data
	FD3D12ResourceLocation SrcResourceLoc(GetParentDevice());
	void* pData = GetParentDevice()->GetDefaultFastAllocator().Allocate(BufferSize, 4UL, &SrcResourceLoc);
	check(pData);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyToUploadMemory);
	FMemory::Memcpy(pData, InResourceArray->GetResourceData(), BufferSize);
	}

	// Allocate copy queue command list and perform the copy op
	FD3D12Device* Device = SrcResourceLoc.GetParentDevice();

	FD3D12SyncPointRef SyncPoint;
	{
		FD3D12CopyScope CopyScope(Device, ED3D12SyncPointType::GPUOnly);
		SyncPoint = CopyScope.GetSyncPoint();

		RHI_BREADCRUMB_CHECK_SHIPPING(OwningContext, ResourceLocation.GetOffsetFromBaseOfResource() + BufferSize <= ResourceLocation.GetResource()->GetDesc().Width);
		RHI_BREADCRUMB_CHECK_SHIPPING(OwningContext, SrcResourceLoc.GetOffsetFromBaseOfResource() + BufferSize <= SrcResourceLoc.GetResource()->GetDesc().Width);

		// Perform actual copy op
		CopyScope.Context.CopyCommandList()->CopyBufferRegion(
			ResourceLocation.GetResource()->GetResource(),
			ResourceLocation.GetOffsetFromBaseOfResource(),
			SrcResourceLoc.GetResource()->GetResource(),
			SrcResourceLoc.GetOffsetFromBaseOfResource(), BufferSize);

		// Residency update needed since it's just been created?
		CopyScope.Context.UpdateResidency(ResourceLocation.GetResource());
		CopyScope.Context.UpdateResidency(SrcResourceLoc.GetResource());
	}

	// Buffer is now written and ready, so unlock the block
	ResourceLocation.UnlockPoolData();

	// Discard the resource array's contents.
	InResourceArray->Discard();

	return SyncPoint;
}


void FD3D12Adapter::AllocateBuffer(FD3D12Device* Device,
	const D3D12_RESOURCE_DESC& InDesc,
	uint32 Size,
	EBufferUsageFlags InUsage,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InCreateD3D12Access,
	uint32 Alignment,
	FD3D12Buffer* Buffer,
	FD3D12ResourceLocation& ResourceLocation,
	ID3D12ResourceAllocator* ResourceAllocator,
	const TCHAR* InDebugName,
	const FName& OwnerName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::AllocateBuffer);

	// Explicitly check that the size is nonzero before allowing CreateBuffer to opaquely fail.
	if (Size == 0)
	{
		UE_LOGF(
			LogD3D12RHI, Fatal,
			"Attempt to create zero-sized buffer '%ls', owner '%ls', usage 0x%x",
			InDebugName ? InDebugName : TEXT("(null)"), *OwnerName.ToString(), static_cast<uint32>(InUsage)
		);
	}
	
	if (EnumHasAnyFlags(InUsage, EBufferUsageFlags::AnyDynamic))
	{
		check(ResourceAllocator == nullptr);
		check(InResourceStateMode != ED3D12ResourceStateMode::MultiState);
		check(InCreateD3D12Access == ED3D12Access::GenericRead);
		GetUploadHeapAllocator(Device->GetGPUIndex()).AllocUploadResource(Size, Alignment, ResourceLocation);
		check(ResourceLocation.GetSize() >= Size);
	}
	else
	{
		if (ResourceAllocator)
		{
			ResourceAllocator->AllocateResource(
				Device->GetGPUIndex(),
				D3D12_HEAP_TYPE_DEFAULT,
				InDesc,
				InDesc.Width,
				Alignment,
				InCreateD3D12Access,
				InResourceStateMode,
				InCreateD3D12Access,
				nullptr,
				InDebugName,
				ResourceLocation);
		}
		else
		{
			Device->GetDefaultBufferAllocator().AllocDefaultResource(
				D3D12_HEAP_TYPE_DEFAULT,
				InDesc,
				InUsage,
				InResourceStateMode,
				InCreateD3D12Access,
				ResourceLocation,
				Alignment,
				InDebugName);
		}
		ResourceLocation.SetOwner(Buffer);
		check(ResourceLocation.GetSize() >= Size);
	}
}

//
// @TODO - This function is awkward and ambiguous because the CreateDesc includes an initial state
// But it also takes a separate create state. Right now the value in CreateDesc is completely ignored
// but that seems wrong... suggest removing InCreatED3D12Access argument in favor of using the value
// in CreateDesc
//
FD3D12Buffer* FD3D12Adapter::CreateRHIBuffer(
	const D3D12_RESOURCE_DESC& InDesc,
	uint32 Alignment,
	const FRHIBufferCreateDesc& CreateDesc,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InCreateD3D12Access,
	bool bKeepUnlocked,
	ID3D12ResourceAllocator* ResourceAllocator)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::CreateRHIBuffer);
	SCOPE_CYCLE_COUNTER(STAT_D3D12CreateBufferTime);

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	check(InDesc.Width >= CreateDesc.Size);

	const bool bDynamic = EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::AnyDynamic);
	FD3D12Buffer* BufferOut = nullptr;

	// Theoretically, we could assert if GPUMask isn't correct, but at the moment the RDG and RHI buffer descriptions don't include the
	// GPU mask, so there's no way for the caller to configure it (only the lower level CreateInfo includes it).  Note that differentiation
	// for NNE (DirectML) is required beyond just setting the mask anyway, in the sense of forcing separate GPU0 visible only heaps, not just
	// filtering which GPU copies are allocated.  Because this is necessary to solve a crash, it's higher priority than GPUMask support,
	// which may be added in the future.
	FRHIGPUMask GPUMask = EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::NNE) ? FRHIGPUMask::GPU0() : CreateDesc.GPUMask;

	const uint32 FirstGPUIndex = GPUMask.GetFirstIndex();

	FD3D12Buffer* NewBuffer0 = nullptr;
	BufferOut = CreateLinkedObject<FD3D12Buffer>(GPUMask, [&](FD3D12Device* Device, FD3D12Buffer* FirstLinkedObject)
	{
		FD3D12Buffer* NewBuffer = new FD3D12Buffer(Device, CreateDesc);
		NewBuffer->BufferAlignment = Alignment;

		if (!bDynamic || (Device->GetGPUIndex() == FirstGPUIndex) || EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::MultiGPUAllocate))
		{
			AllocateBuffer(
				Device,
				InDesc,
				CreateDesc.Size,
				CreateDesc.Usage,
				InResourceStateMode,
				InCreateD3D12Access,
				Alignment,
				NewBuffer,
				NewBuffer->ResourceLocation,
				ResourceAllocator,
				CreateDesc.DebugName,
				CreateDesc.OwnerName);

			NewBuffer0 = NewBuffer;
		}
		else
		{
			check(NewBuffer0);
			FD3D12ResourceLocation::ReferenceNode(Device, NewBuffer->ResourceLocation, NewBuffer0->ResourceLocation);
		}

		
		// Unlock immediately if there is no initial data
		if (!bDynamic && !bKeepUnlocked)
		{
			NewBuffer->ResourceLocation.UnlockPoolData();
		}

		return NewBuffer;
	});

	// Don't track transient buffer stats here
	if (!BufferOut->ResourceLocation.IsTransient())
	{
		D3D12BufferStats::UpdateBufferStats(*BufferOut, true);
	}

	return BufferOut;
}

void FD3D12Buffer::Rename(FD3D12ContextArray const& Contexts, FD3D12ResourceLocation& NewLocation)
{
	FD3D12ResourceLocation::TransferOwnership(ResourceLocation, NewLocation);
	ResourceRenamed(Contexts);
}

void FD3D12Buffer::RenameLDAChain(FD3D12ContextArray const& Contexts, FD3D12ResourceLocation& NewLocation)
{
	// Dynamic buffers use cross-node resources (with the exception of EBufferUsageFlags::MultiGPUAllocate)
	//ensure(GetUsage() & EBufferUsageFlags::AnyDynamic);
	Rename(Contexts, NewLocation);

	if (GNumExplicitGPUsForRendering > 1)
	{
		ensure(GetParentDevice() == NewLocation.GetParentDevice());

		if (EnumHasAnyFlags(GetUsage(), EBufferUsageFlags::MultiGPUAllocate) == false)
		{
			ensure(IsHeadLink());

			// Update all of the resources in the LDA chain to reference this cross-node resource
			for (auto NextBuffer = ++FLinkedObjectIterator(this); NextBuffer; ++NextBuffer)
			{
				FD3D12ResourceLocation::ReferenceNode(NextBuffer->GetParentDevice(), NextBuffer->ResourceLocation, ResourceLocation);
				NextBuffer->ResourceRenamed(Contexts);
			}
		}
	}
}

void FD3D12Buffer::TakeOwnership(FD3D12Buffer& Other)
{
	check(!Other.LockedData.bLocked);

	// Clean up any resource this buffer already owns
	ReleaseOwnership();

	// Transfer ownership of Other's resources to this instance
	FRHIBuffer::TakeOwnership(Other);
	FD3D12ResourceLocation::TransferOwnership(ResourceLocation, Other.ResourceLocation);
}

void FD3D12Buffer::ReleaseOwnership()
{
	check(!LockedData.bLocked);
	check(IsHeadLink());

	FRHIBuffer::ReleaseOwnership();

	if (!ResourceLocation.IsTransient())
	{
		D3D12BufferStats::UpdateBufferStats(*this, false);
	}

	ResourceLocation.Clear();
}

void FD3D12Buffer::GetResourceDescAndAlignment(const FRHIBufferCreateDesc& CreateDesc, D3D12_RESOURCE_DESC& ResourceDesc, uint32& Alignment)
{
	ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(CreateDesc.Size);

	// Align size to 16 so RAW buffer view can be created without loosing any data at the end when dividing num elements by 4
	ResourceDesc.Width = Align(ResourceDesc.Width, RHI_RAW_VIEW_ALIGNMENT);

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::UnorderedAccess))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS;
	}

	if (!EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::ShaderResource | EBufferUsageFlags::AccelerationStructure))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE;
	}

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::DrawIndirect))
	{
		ResourceDesc.Flags |= D3D12RHI_RESOURCE_FLAG_ALLOW_INDIRECT_BUFFER;
	}

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::Shared))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;
	}

#if D3D12_RHI_RAYTRACING
	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::AccelerationStructure))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE;
	}
#endif

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::ReservedResource))
	{
		checkf(CreateDesc.Stride <= GRHIGlobals.ReservedResources.TileSizeInBytes,
			   TEXT("Reserved buffer stride %d must not be greater than the reserved resource tile size %d"),
			   CreateDesc.Stride, GRHIGlobals.ReservedResources.TileSizeInBytes);

		Alignment = GRHIGlobals.ReservedResources.TileSizeInBytes;
	}
	else
	{
		// Structured buffers, non-ByteAddress buffers, need to be aligned to their stride to ensure that they can be addressed correctly with element based offsets.
		Alignment = (CreateDesc.Stride > 0) && (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::StructuredBuffer) || !EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::ByteAddressBuffer | EBufferUsageFlags::DrawIndirect)) ? FMath::LeastCommonMultiplier(CreateDesc.Stride, RHI_RAW_VIEW_ALIGNMENT) : RHI_RAW_VIEW_ALIGNMENT;
	}
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
void FD3D12Buffer::UpdateAllocationTags() const
{
	FD3D12Resource* D3D12Resource = GetResource();
	if (D3D12Resource)
	{
		auto ExecuteUpdateAllocationTags = [](uint64 LLMPtr, uint64 TracePtr, uint64 Size, bool bVideoMemory=true)
		{
            // PLATFORM_WINDOWS does not perform per buffer LLM tracking currently
#if !PLATFORM_WINDOWS
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, (const void*)LLMPtr));
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, (const void*)LLMPtr, Size));
#endif // #if !PLATFORM_WINDOWS

#if D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY
			bVideoMemory = false;
#endif

#if UE_MEMORY_TRACE_ENABLED
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel))
			{
				MemoryTrace_UpdateAlloc(TracePtr, bVideoMemory ? EMemoryTraceRootHeap::VideoMemory : EMemoryTraceRootHeap::SystemMemory);
			}
#endif
		};

		if (D3D12Resource->IsReservedResource())
		{
			D3D12Resource->GetBackingHeapsGpuAddresses(ExecuteUpdateAllocationTags);
			// Done so that FD3D12Resource::CommitReservedResource is able to restore the proper tag with LLM_REALLOC_SCOPE
			ExecuteUpdateAllocationTags((uint64)D3D12Resource->ReservedResourceData.Get(), (uint64)D3D12Resource->ReservedResourceData.Get(), sizeof(FD3D12Resource::FD3D12ReservedResourceData), false);
		}
		else
		{
			FD3D12ResourceLocation::EAllocatorType AllocatorType = ResourceLocation.GetAllocatorType();
			uint64 AddressForLLM = (AllocatorType == FD3D12ResourceLocation::AT_Pool || AllocatorType == FD3D12ResourceLocation::AT_Default) ? (uint64)ResourceLocation.GetAddressForLLMTracking() : ResourceLocation.GetGPUVirtualAddress();
			ExecuteUpdateAllocationTags(AddressForLLM, (uint64)ResourceLocation.GetGPUVirtualAddress(), ResourceLocation.GetSize());
		}
	}
}

void FD3D12DynamicRHI::RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	check(RHICmdList.IsBottomOfPipe());
    FD3D12DynamicRHI::ResourceCast(BufferRHI)->UpdateAllocationTags();
}
#endif // #if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED

FD3D12DynamicRHI::FCreateBufferInternalResult FD3D12DynamicRHI::CreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc, bool bHasInitialData, ID3D12ResourceAllocator* ResourceAllocator)
{
	D3D12_RESOURCE_DESC Desc{};
	uint32 Alignment{};
	FD3D12Buffer::GetResourceDescAndAlignment(CreateDesc, Desc, Alignment);

	ED3D12ResourceStateMode StateMode = EnumHasAllFlags(CreateDesc.Usage, EBufferUsageFlags::AccelerationStructure)
		? ED3D12ResourceStateMode::SingleState
		: ED3D12ResourceStateMode::Default;

	const bool bIsDynamic = EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::AnyDynamic);

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::ReservedResource))
	{
		checkf(!bHasInitialData, TEXT("Reserved resources may not have initial data"));
		checkf(!bIsDynamic, TEXT("Reserved resources may not be dynamic"));
		checkf(!ResourceAllocator, TEXT("Reserved resources may not use a custom resource allocator"));
	}

	const D3D12_HEAP_TYPE HeapType = bIsDynamic ? D3D12_HEAP_TYPE_UPLOAD : D3D12_HEAP_TYPE_DEFAULT;
	const FD3D12Resource::FD3D12ResourceTypeHelper Type(Desc, HeapType);

	// Does this resource support tracking?
	const bool bSupportResourceStateTracking = !bIsDynamic && FD3D12DefaultBufferAllocator::IsPlacedResource(Desc.Flags, StateMode, Alignment) && Type.bWritable;

	// Initial state is derived from the InResourceState if it supports tracking
	const ED3D12Access DesiredD3D12Access = bSupportResourceStateTracking ? 
		Type.GetOptimalInitialD3D12Access(ConvertToD3D12Access(CreateDesc.InitialState), false) :
		FD3D12DefaultBufferAllocator::GetDefaultInitialD3D12Access(HeapType, CreateDesc.Usage, StateMode);

	// Setup the state at which the resource needs to be created - copy dest only supported for placed resources
	const ED3D12Access CreatED3D12Access =
		(bHasInitialData && bSupportResourceStateTracking) ? ED3D12Access::CopyDest : DesiredD3D12Access;

	FD3D12Buffer* Buffer = GetAdapter().CreateRHIBuffer(
		Desc,
		Alignment,
		CreateDesc,
		StateMode,
		CreatED3D12Access,
		bHasInitialData,
		ResourceAllocator);
	check(Buffer->ResourceLocation.IsValid());

	return FCreateBufferInternalResult{ Buffer, DesiredD3D12Access };
}

static FRHIBufferInitializer CreateBufferInitializerForWriting(FRHICommandListBase& RHICmdList, const FD3D12DynamicRHI::FCreateBufferInternalResult& CreateResult, const FRHIBufferCreateDesc& CreateDesc)
{
	FD3D12Buffer* Buffer = CreateResult.Buffer;
	ED3D12Access DesiredD3D12Access = CreateResult.DesiredD3D12Access;

	if (EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::AnyDynamic))
	{
		// Copy directly in mapped data
		const FD3D12ResourceLocation& UploadLocation = Buffer->ResourceLocation;
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer, UploadLocation.GetMappedBaseAddress(), UploadLocation.GetSize());
	}

	FD3D12ResourceLocation UploadLocation(AllocateUploadMemory(Buffer, CreateDesc));

	// Note: we have to get this pointer before the Initializer is constructed because it will be moved before we get the pointer
	void* WritableData = UploadLocation.GetMappedBaseAddress();

	return UE::RHICore::FCustomBufferInitializer(RHICmdList, Buffer, WritableData, CreateDesc.Size,
		[Buffer = TRefCountPtr<FD3D12Buffer>(Buffer), DesiredD3D12Access, UploadLocation = MoveTemp(UploadLocation)](FRHICommandListBase& RHICmdList) mutable
		{
			RHICmdList.EnqueueLambda(
				[Buffer = Buffer.GetReference(), DesiredD3D12Access, UploadLocation = MoveTemp(UploadLocation)](FRHICommandListBase& ExecutingCmdList)
				{
					const FRHIGPUMask EffectiveMask = ExecutingCmdList.GetGPUMask();
					for (uint32 GPUIndex : EffectiveMask)
					{
						FD3D12CommandContext& CommandContext = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
						FD3D12Buffer* DeviceBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Buffer, GPUIndex);

						Buffer->UploadResourceData(
							CommandContext,
							DesiredD3D12Access,
							DeviceBuffer->ResourceLocation,
							UploadLocation,
							Buffer->GetSize());
					}
				});

			return TRefCountPtr<FRHIBuffer>(MoveTemp(Buffer));
		});
}

FRHIBufferInitializer FD3D12DynamicRHI::RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	if (CreateDesc.IsNull())
	{
		FD3D12Buffer* Buffer = GetAdapter().CreateLinkedObject<FD3D12Buffer>(
			CreateDesc.GPUMask,
			[CreateDesc](FD3D12Device* Device, FD3D12Buffer* FirstLinkedObject)
			{
				return new FD3D12Buffer(Device, CreateDesc);
			});

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

#if D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY
	// Unified platforms don't need to use the copy queue for uploads.
	constexpr bool bCreateAsCopyDest = false;
	FCreateBufferInternalResult CreateResult = CreateBufferInternal(CreateDesc, bCreateAsCopyDest, nullptr);

	const FD3D12ResourceLocation& UploadLocation = CreateResult.Buffer->ResourceLocation;

	void* const WritableData = reinterpret_cast<void*>(UploadLocation.GetGPUVirtualAddress());
	const uint64 WritableSize = UploadLocation.GetSize();

	return UE::RHICore::CreateUnifiedMemoryBufferInitializer(RHICmdList, CreateDesc, CreateResult.Buffer, WritableData);
#else
	const bool bCreateAsCopyDest = (CreateDesc.InitAction != ERHIBufferInitAction::Default);

	FCreateBufferInternalResult CreateResult = CreateBufferInternal(CreateDesc, bCreateAsCopyDest, nullptr);

	if (CreateDesc.InitAction == ERHIBufferInitAction::Default)
	{
		// Just return the buffer with its default contents
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, CreateResult.Buffer);
	}

	// TODO: the UploadLocation logic is too complex to "just" write a different code path for each initializer type, so we will just use a common Initializer.
	if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray)
	{
		FRHIBufferInitializer Initializer = CreateBufferInitializerForWriting(RHICmdList, CreateResult, CreateDesc);

		// Resource array initialization goes through the same steps as external code that writes its own data.
		FResourceArrayUploadInterface* InitialData = CreateDesc.InitialData;
		Initializer.WriteData(InitialData->GetResourceData(), InitialData->GetResourceDataSize());

		// Discard the resource array's contents.
		InitialData->Discard();

		return MoveTemp(Initializer);
	}
	
	if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
	{
		FRHIBufferInitializer Initializer = CreateBufferInitializerForWriting(RHICmdList, CreateResult, CreateDesc);

		// TODO: write a custom Initializer method that enqueues zeroing on the upload context.
		Initializer.FillWithValue(0);

		return MoveTemp(Initializer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
	{
		return CreateBufferInitializerForWriting(RHICmdList, CreateResult, CreateDesc);
	}

	return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
#endif // D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY
}

void* FD3D12DynamicRHI::LockBuffer(FRHICommandListBase& RHICmdList, FD3D12Buffer* Buffer, uint32 BufferSize, EBufferUsageFlags BufferUsage, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12LockBufferTime);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(Buffer->GetName(), Buffer->GetName(), Buffer->GetOwnerName());

	checkf(Size <= BufferSize, TEXT("Requested lock size %u is larger than the total size %u for buffer '%s'."), Size, BufferSize, *Buffer->GetName().ToString());

	FD3D12LockedResource& LockedData = Buffer->LockedData;
	check(LockedData.bLocked == false);

	FD3D12Adapter& Adapter = GetAdapter();

	void* Data = nullptr;

	// Determine whether the buffer is dynamic or not.
	if (EnumHasAnyFlags(BufferUsage, EBufferUsageFlags::AnyDynamic))
	{
		check(LockMode == RLM_WriteOnly || LockMode == RLM_WriteOnly_NoOverwrite);

		if (LockedData.bHasNeverBeenLocked || LockMode == RLM_WriteOnly_NoOverwrite)
		{
			// Buffers on upload heap are mapped right after creation
			Data = Buffer->ResourceLocation.GetMappedBaseAddress();
			check(!!Data);
		}
		else
		{
			FD3D12Device* Device = Buffer->GetParentDevice();

			FD3D12ResourceLocation NewLocation(Device);
			Data = Adapter.GetUploadHeapAllocator(Device->GetGPUIndex()).AllocUploadResource(BufferSize, Buffer->BufferAlignment, NewLocation);

			RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FD3D12DynamicRHI::LockBuffer"),
			[
				Resource = Buffer,
				NewLocation = MoveTemp(NewLocation)
			](FD3D12ContextArray const& Contexts) mutable
			{
				const static FLazyName ExecuteName(TEXT("FRHICommandRenameUploadBuffer::Execute"));
				UE_TRACE_METADATA_SCOPE_ASSET_FNAME(Resource->GetName(), ExecuteName, Resource->GetOwnerName());

				for (FD3D12Buffer& DeviceBuffer : *Resource)
				{
					for (FD3D12CommandContextBase* ContextBase : Contexts)
					{
						if (FD3D12CommandContext* Context = ContextBase ? ContextBase->GetSingleDeviceContext(DeviceBuffer.GetParentDevice()->GetGPUIndex()) : nullptr)
						{
							// Clear the resource if still bound to make sure the SRVs are rebound again on next operation. This needs to happen
							// on the RHI timeline when this command runs at the top of the pipe (which can happen when locking buffers in
							// RLM_WriteOnly_NoOverwrite mode).
							Context->ConditionalClearShaderResource(&DeviceBuffer.ResourceLocation, EShaderParameterTypeMask::SRVMask);
						}
					}
				}

#if UE_MEMORY_TRACE_ENABLED
				// This memory trace happens before RenameLDAChain so the old & new GPU addresses are correct
				MemoryTrace_ReallocFree(Resource->ResourceLocation.GetGPUVirtualAddress(), EMemoryTraceRootHeap::VideoMemory);
				MemoryTrace_ReallocAlloc(NewLocation.GetGPUVirtualAddress(), Resource->ResourceLocation.GetSize(), Resource->BufferAlignment, EMemoryTraceRootHeap::VideoMemory);
#endif
				Resource->RenameLDAChain(Contexts, NewLocation);
			});
		}
	}
	else
	{
		// Static and read only buffers only have one version of the content. Use the first related device.
		FD3D12Device* Device = Buffer->GetParentDevice();
		FD3D12Resource* pResource = Buffer->ResourceLocation.GetResource();

		// Locking for read must occur immediately so we can't queue up the operations later.
		if (LockMode == RLM_ReadOnly)
		{
			LockedData.bLockedForReadOnly = true;
			// If the static buffer is being locked for reading, create a staging buffer.
			FD3D12Resource* StagingBuffer = nullptr;

			const FRHIGPUMask Node = Device->GetGPUMask();
			VERIFYD3D12RESULT(Adapter.CreateBuffer(D3D12_HEAP_TYPE_READBACK, Node, Node, Offset + Size, &StagingBuffer, nullptr));

			// Copy the contents of the buffer to the staging buffer.
			RHICmdList.EnqueueLambda([Node, StagingBuffer, pResource, Buffer, Offset, Size](FRHICommandListBase& ExecutingCmdList)
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Node.GetFirstIndex());
				uint64 SubAllocOffset = Buffer->ResourceLocation.GetOffsetFromBaseOfResource();

				FScopedResourceBarrier ScopeResourceBarrierSource(
					Context,
					pResource,
					ED3D12Access::CopySrc,
					0);

				// Don't need to transition upload heaps
				Context.FlushResourceBarriers(); // Must flush so the desired state is actually set.

				Context.UpdateResidency(StagingBuffer);
				Context.UpdateResidency(pResource);

				Context.CopyBufferRegionChecked(
					StagingBuffer->GetResource(), StagingBuffer->GetName(),
					0,
					pResource->GetResource(), pResource->GetName(),
					SubAllocOffset + Offset, Size);
			});
			
			RHICmdList.GetAsImmediate().SubmitAndBlockUntilGPUIdle();

			LockedData.ResourceLocation.AsStandAlone(StagingBuffer, Size);
			Data = LockedData.ResourceLocation.GetMappedBaseAddress();
		}
		else
		{
			// If the static buffer is being locked for writing, allocate memory for the contents to be written to.
			Data = Adapter.GetUploadHeapAllocator(Device->GetGPUIndex()).AllocUploadResource(Size, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, LockedData.ResourceLocation);
		}
	}

	LockedData.LockOffset = Offset;
	LockedData.LockSize = Size;
	LockedData.bLocked = true;
	LockedData.bHasNeverBeenLocked = false;

	// Return the offset pointer
	check(Data != nullptr);
	return Data;
}

void FD3D12DynamicRHI::UnlockBuffer(FRHICommandListBase& RHICmdList, FD3D12Buffer* Buffer, EBufferUsageFlags BufferUsage)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12UnlockBufferTime);

	FD3D12LockedResource& LockedData = Buffer->LockedData;
	check(LockedData.bLocked == true);

	// Determine whether the buffer is dynamic or not.
	if (EnumHasAnyFlags(BufferUsage, EBufferUsageFlags::AnyDynamic))
	{
		// If the Buffer is dynamic, its upload heap memory can always stay mapped. Don't do anything.
	}
	else if (LockedData.bLockedForReadOnly)
	{
		// Nothing to do, just release the locked data at the end of the function
	}
	else
	{
		// Update all of the resources in the LDA chain
		check(Buffer->IsHeadLink());

		RHICmdList.EnqueueLambda([
#if EXECUTE_DEBUG_COMMAND_LISTS
			this,
#endif
			RootBuffer = Buffer,
			LockedData = MoveTemp(LockedData)
		](FRHICommandListBase& ExecutingCmdList)
		{
			for (FD3D12Buffer& Buffer : *RootBuffer)
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Buffer.GetParentDevice()->GetGPUIndex());

				FD3D12Resource* SourceResource = LockedData.ResourceLocation.GetResource();
				uint32 SourceFullOffset = LockedData.ResourceLocation.GetOffsetFromBaseOfResource();

				FD3D12Resource* DestResource = Buffer.ResourceLocation.GetResource();
				uint32 DestFullOffset = Buffer.ResourceLocation.GetOffsetFromBaseOfResource() + LockedData.LockOffset;

				// Clear the resource if still bound to make sure the SRVs are rebound again on next operation (and get correct resource transitions enqueued)
				Context.ConditionalClearShaderResource(&Buffer.ResourceLocation, EShaderParameterTypeMask::SRVMask);

				FScopedResourceBarrier ScopeResourceBarrierDest(
					Context,
					DestResource,
					ED3D12Access::CopyDest,
					0);

				// Don't need to transition upload heaps
				Context.FlushResourceBarriers();

				Context.UpdateResidency(DestResource);
				Context.UpdateResidency(SourceResource);

				Context.CopyBufferRegionChecked(
					DestResource->GetResource(), DestResource->GetName(),
					DestFullOffset,
					SourceResource->GetResource(), SourceResource->GetName(),
					SourceFullOffset,
					LockedData.LockSize
				);
				
				Context.ConditionalSplitCommandList();

				DEBUG_RHI_EXECUTE_COMMAND_LIST(this);
			}
		});
	}

	LockedData.Reset();
}

void* FD3D12DynamicRHI::RHILockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	// If you hit this assert, you should be using LockBufferMGPU and iterating over FRHIGPUMask::All() to initialize the resource separately for each GPU.
	// "MultiGPUAllocate" only makes sense if a buffer must vary per GPU, for example if it's a buffer that includes GPU specific virtual addresses for ray
	// tracing acceleration structures.
	check(!EnumHasAnyFlags(BufferRHI->GetUsage(), EBufferUsageFlags::MultiGPUAllocate));

	bool bNeedTransition = ( (!(EnumHasAnyFlags(BufferRHI->GetUsage(), BUF_AnyDynamic))) && (LockMode == EResourceLockMode::RLM_ReadOnly) );

	if (RHICmdList.NeedsExtraTransitions() && bNeedTransition)
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(BufferRHI, ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}

	FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(BufferRHI);
	void* retval = LockBuffer(RHICmdList, Buffer, Buffer->GetSize(), Buffer->GetUsage(), Offset, Size, LockMode);

	if (RHICmdList.NeedsExtraTransitions() && bNeedTransition)
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(BufferRHI, ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}

	return retval;
}

void* FD3D12DynamicRHI::RHILockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 GPUIndex, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	// If you hit this assert, you should be using LockBuffer to initialize the resource, rather than this function.  The MGPU version is only for resources
	// with the MultiGPUAllocate flag, where it's necessary for the caller to initialize the buffer for each GPU.  The other LockBuffer call initializes the
	// resource on all GPUs with one call, due to driver mirroring of the underlying resource.
	check(EnumHasAnyFlags(BufferRHI->GetUsage(), EBufferUsageFlags::MultiGPUAllocate));

	FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(BufferRHI, GPUIndex);
	return LockBuffer(RHICmdList, Buffer, Buffer->GetSize(), Buffer->GetUsage(), Offset, Size, LockMode);
}

void FD3D12DynamicRHI::RHIUnlockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	check(!EnumHasAnyFlags(BufferRHI->GetUsage(), EBufferUsageFlags::MultiGPUAllocate));

	FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(BufferRHI);
	FD3D12LockedResource& LockedData = Buffer->LockedData;

	const bool bNeedTransition = (!((EnumHasAnyFlags(Buffer->GetUsage(), BUF_AnyDynamic)) || LockedData.bLockedForReadOnly));

	if (RHICmdList.NeedsExtraTransitions() && bNeedTransition)
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(Buffer, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}

	UnlockBuffer(RHICmdList, Buffer, Buffer->GetUsage());

	if (RHICmdList.NeedsExtraTransitions() && bNeedTransition )
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(Buffer, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}
}

void FD3D12DynamicRHI::RHIUnlockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 GPUIndex)
{
	check(EnumHasAnyFlags(BufferRHI->GetUsage(), EBufferUsageFlags::MultiGPUAllocate));

	FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(BufferRHI, GPUIndex);
	UnlockBuffer(RHICmdList, Buffer, Buffer->GetUsage());
}

void FD3D12DynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, const TCHAR* Name)
{
	if (BufferRHI == nullptr || !GD3D12BindResourceLabels)
	{
		return;
	}

#if RHI_USE_RESOURCE_DEBUG_NAME
	// Also set on RHI object
	BufferRHI->SetName(Name);

	FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(BufferRHI);

	// only rename the underlying d3d12 resource if it's not sub allocated (requires resource state tracking or stand alone allocated)
	if (ShouldSetD3D12ResourceName(Buffer->ResourceLocation))
	{
		if (GNumExplicitGPUsForRendering > 1)
		{
			// Generate string of the form "Name (GPU #)" -- assumes GPU index is a single digit.  This is called many times
			// a frame, so we want to avoid any string functions which dynamically allocate, to reduce perf overhead.
			static_assert(MAX_NUM_GPUS <= 10);

			static const TCHAR NameSuffix[] = TEXT(" (GPU #)");
			constexpr int32 NameSuffixLengthWithTerminator = (int32)UE_ARRAY_COUNT(NameSuffix);
			constexpr int32 NameBufferLength = 256;
			constexpr int32 GPUIndexSuffixOffset = 6;		// Offset of '#' character

			// Combine Name and suffix in our string buffer (clamping the length for bounds checking).  We'll replace the GPU index
			// with the appropriate digit in the loop.
			int32 NameLength = FMath::Min(FCString::Strlen(Name), NameBufferLength - NameSuffixLengthWithTerminator);
			int32 GPUIndexOffset = NameLength + GPUIndexSuffixOffset;

			TCHAR DebugName[NameBufferLength];
			FMemory::Memcpy(&DebugName[0], Name, NameLength * sizeof(TCHAR));
			FMemory::Memcpy(&DebugName[NameLength], NameSuffix, NameSuffixLengthWithTerminator * sizeof(TCHAR));

			for (FD3D12Buffer::FLinkedObjectIterator BufferIt(Buffer); BufferIt; ++BufferIt)
			{
				FD3D12Resource* Resource = BufferIt->GetResource();

				DebugName[GPUIndexOffset] = TEXT('0') + BufferIt->GetParentDevice()->GetGPUIndex();

				SetD3D12ResourceName(BufferIt->GetResource(), DebugName);
			}
		}
		else
		{
			SetD3D12ResourceName(Buffer->GetResource(), Name);
		}
	}
#endif
}

void FD3D12CommandContext::RHICopyBufferRegion(FRHIBuffer* DestBufferRHI, uint64 DstOffset, FRHIBuffer* SourceBufferRHI, uint64 SrcOffset, uint64 NumBytes)
{
	FD3D12Buffer* SourceBuffer = RetrieveObject<FD3D12Buffer>(SourceBufferRHI);
	FD3D12Buffer* DestBuffer = RetrieveObject<FD3D12Buffer>(DestBufferRHI);

	FD3D12Device* BufferDevice = SourceBuffer->GetParentDevice();
	check(BufferDevice == DestBuffer->GetParentDevice());
	check(BufferDevice == GetParentDevice());

	FD3D12Resource* pSourceResource = SourceBuffer->ResourceLocation.GetResource();
	D3D12_RESOURCE_DESC const& SourceBufferDesc = pSourceResource->GetDesc();

	FD3D12Resource* pDestResource = DestBuffer->ResourceLocation.GetResource();
	D3D12_RESOURCE_DESC const& DestBufferDesc = pDestResource->GetDesc();

	checkf(pSourceResource != pDestResource, TEXT("CopyBufferRegion cannot be used on the same resource. This can happen when both the source and the dest are suballocated from the same resource."));

	check(DstOffset + NumBytes <= DestBufferDesc.Width);
	check(SrcOffset + NumBytes <= SourceBufferDesc.Width);

	FScopedResourceBarrier ScopeResourceBarrierSrc(
		*this,
		pSourceResource,
		ED3D12Access::CopySrc,
		0);
	FScopedResourceBarrier ScopeResourceBarrierDst(
		*this,
		pDestResource,
		ED3D12Access::CopyDest,
		0);
	FlushResourceBarriers();

	CopyBufferRegionChecked(
		pDestResource->GetResource(), pDestResource->GetName(),
		DestBuffer->ResourceLocation.GetOffsetFromBaseOfResource() + DstOffset,
		pSourceResource->GetResource(), pSourceResource->GetName(),
		SourceBuffer->ResourceLocation.GetOffsetFromBaseOfResource() + SrcOffset,
		NumBytes
	);

	UpdateResidency(pSourceResource);
	UpdateResidency(pDestResource);

	ConditionalSplitCommandList();
}
