// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12CommandContext.cpp: RHI  Command Context implementation.
=============================================================================*/

#include "D3D12CommandContext.h"
#include "D3D12RHIPrivate.h"

#include "D3D12AmdExtensions.h"
#include "D3D12RayTracing.h"

#include "D3D12Tracing.h"

int32 GD3D12MaxCommandsPerCommandList = 10000;
static FAutoConsoleVariableRef CVarMaxCommandsPerCommandList(
	TEXT("D3D12.MaxCommandsPerCommandList"),
	GD3D12MaxCommandsPerCommandList,
	TEXT("Flush command list to GPU after certain amount of enqueued commands (draw, dispatch, copy, ...) (default value 10000)"),
	ECVF_RenderThreadSafe
);

// We don't yet have a way to auto-detect that the Radeon Developer Panel is running
// with profiling enabled, so for now, we have to manually toggle this console var.
// It needs to be set before device creation, so it's read only.
int32 GEmitRgpFrameMarkers = 0;
static FAutoConsoleVariableRef CVarEmitRgpFrameMarkers(
	TEXT("D3D12.EmitRgpFrameMarkers"),
	GEmitRgpFrameMarkers,
	TEXT("Enables/Disables frame markers for AMD's RGP tool."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// jhoerner TODO 10/4/2022:  This setting is a hack to improve performance by reverting cross GPU transfer synchronization behavior to
// what it was in 5.0, at a cost in validation correctness (D3D debug errors related to using a cross GPU transferred resource in an
// incorrect transition state, or when possibly still being written).  In practice, these errors haven't caused artifacts or stability
// issues, but if you run into an artifact suspected to be related to a cross GPU transfer, or want to run with validation for
// debugging, you can disable the hack.  A future refactor in 5.2 will clean this up and provide validation correctness without any
// performance loss.
//
bool GD3D12UnsafeCrossGPUTransfers = true;
static FAutoConsoleVariableRef CVarD3D12UnsafeCrossGPUTransfers(
	TEXT("D3D12.UnsafeCrossGPUTransfers"),
	GD3D12UnsafeCrossGPUTransfers,
	TEXT("Disables cross GPU synchronization correctness, for a gain in performance (Default: true)."),
	ECVF_RenderThreadSafe
);

FD3D12CommandContextBase::FD3D12CommandContextBase(class FD3D12Adapter* InParentAdapter, FRHIGPUMask InGPUMask)
	: FD3D12AdapterChild(InParentAdapter)
	, GPUMask(InGPUMask)
	, PhysicalGPUMask(InGPUMask)
{
}

FD3D12CommandContext::FD3D12CommandContext(FD3D12Device* InParent, ED3D12QueueType QueueType, bool InIsDefaultContext)
	: FD3D12ContextCommon(InParent, QueueType, InIsDefaultContext)
	, FD3D12CommandContextBase(InParent->GetParentAdapter(), InParent->GetGPUMask())
	, FD3D12DeviceChild(InParent)
	, ConstantsAllocator(InParent, InParent->GetGPUMask())
	, StateCache(*this, InParent->GetGPUMask())
	, StageConstantBuffers{
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
		FD3D12ConstantBuffer(InParent, ConstantsAllocator),
	}
{
	StaticUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());
	ClearState();

	// All the interesting stuff (sync points, payloads etc.) happens in FD3D12ContextCommon, but the tracing code associates
	// translate jobs with IRHIComputeContext pointers, and FD3D12ContextCommon does not derive from that interface. Instead,
	// this class derives from both, so this is the place where we can map FD3D12ContextCommon pointers to IRHIComputeContext
	// pointers, to allow the tracing code to reconstruct the translate job -> RHI context -> D3D12 context dependency chains.
	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::AssociateContext(this, this));
}

FD3D12CommandContext::~FD3D12CommandContext()
{
	ClearState();
}

void FD3D12ContextCommon::WriteMarker(D3D12_GPU_VIRTUAL_ADDRESS Address, uint32 Value, EMarkerType Type)
{
	if (!GraphicsCommandList2())
		return;

	D3D12_WRITEBUFFERIMMEDIATE_PARAMETER Parameter;
	Parameter.Dest = Address;
	Parameter.Value = Value;

	D3D12_WRITEBUFFERIMMEDIATE_MODE Mode = Type == EMarkerType::In
		? D3D12_WRITEBUFFERIMMEDIATE_MODE_MARKER_IN
		: D3D12_WRITEBUFFERIMMEDIATE_MODE_MARKER_OUT;

	GraphicsCommandList2()->WriteBufferImmediate(1, &Parameter, &Mode);
}

void FD3D12ContextCommon::BindDiagnosticBuffer(FD3D12RootSignature const* RootSignature, ED3D12PipelineType PipelineType)
{
	int8 const Slot = RootSignature->GetDiagnosticBufferSlot();
	if (Slot < 0)
		return;

	FD3D12DiagnosticBuffer* DiagBuffer = Device->GetQueue(QueueType).DiagnosticBuffer.Get();
	if (DiagBuffer && DiagBuffer->IsValid())
	{
		D3D12_GPU_VIRTUAL_ADDRESS DataAddress = DiagBuffer->GetGPUQueueData();

		switch (PipelineType)
		{
		default: checkNoEntry(); [[fallthrough]];
		case ED3D12PipelineType::Graphics: GraphicsCommandList()->SetGraphicsRootUnorderedAccessView(Slot, DataAddress); break;
		case ED3D12PipelineType::Compute : GraphicsCommandList()->SetComputeRootUnorderedAccessView (Slot, DataAddress); break;
		}
	}
}

#if WITH_RHI_BREADCRUMBS
	void FD3D12CommandContext::RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		FD3D12DiagnosticBuffer* DiagBuffer = Device->GetQueue(QueueType).DiagnosticBuffer.Get();
		if (DiagBuffer && DiagBuffer->IsValid() && UE::RHI::UseGPUCrashBreadcrumbs())
		{
			D3D12_GPU_VIRTUAL_ADDRESS Marker = DiagBuffer->GetGPUQueueMarkerIn();
			WriteMarker(Marker, Breadcrumb->ID, EMarkerType::In);
		}

	#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::D3D12::BeginBreadcrumb(AftermathHandle(), Breadcrumb);
	#endif
	#if INTEL_GPU_CRASH_DUMPS
		UE::RHICore::Intel::GPUCrashDumps::D3D12::BeginBreadcrumb(GraphicsCommandList().Get(), Breadcrumb);
	#endif
	
		const TCHAR* NameStr = nullptr;
		FRHIBreadcrumb::FBuffer Buffer;
		auto GetNameStr = [&]()
		{
			if (!NameStr)
			{
				NameStr = Breadcrumb->GetTCHAR(Buffer);
			}
			return NameStr;
		};

		// Only emit formatted strings to platform APIs when requested.
		if (ShouldEmitBreadcrumbs())
		{
		#if WITH_AMD_AGS
			if (AGSContext* const AmdAgsContext = FD3D12DynamicRHI::GetD3DRHI()->GetAmdAgsContext())
			{
				if (GEmitRgpFrameMarkers)
				{
					agsDriverExtensionsDX12_PushMarker(AmdAgsContext, GraphicsCommandList().Get(), TCHAR_TO_ANSI(GetNameStr()));
				}
			}
		#endif

		#if USE_PIX
			if (FD3D12DynamicRHI::GetD3DRHI()->IsPixEventEnabled())
			{
				uint32 Color = PIX_COLOR(Breadcrumb->Data.Color.R, Breadcrumb->Data.Color.G, Breadcrumb->Data.Color.B);
				PIXBeginEvent(GraphicsCommandList().Get(), Color, TEXT("%s"), GetNameStr());
			}
		#endif
		}

		{
			FlushProfilerStats();

			auto& Event = GetCommandList().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb>(Breadcrumb);
			FD3D12QueryLocation TimestampQuery = AllocateQuery(ED3D12QueryType::ProfilerTimestampTOP, &Event.GPUTimestampTOP);
			EndQuery(TimestampQuery);
		}
	}

	void FD3D12CommandContext::RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		{
			FlushProfilerStats();

			auto& Event = GetCommandList().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb>(Breadcrumb);
			FD3D12QueryLocation TimestampQuery = AllocateQuery(ED3D12QueryType::ProfilerTimestampBOP, &Event.GPUTimestampBOP);
			EndQuery(TimestampQuery);
		}

		// Only emit formatted strings to platform APIs when requested.
		if (ShouldEmitBreadcrumbs())
		{
		#if USE_PIX
			if (FD3D12DynamicRHI::GetD3DRHI()->IsPixEventEnabled())
			{
				PIXEndEvent(GraphicsCommandList().Get());
			}
		#endif

		#if WITH_AMD_AGS
			if (AGSContext* const AmdAgsContext = FD3D12DynamicRHI::GetD3DRHI()->GetAmdAgsContext())
			{
				if (GEmitRgpFrameMarkers)
				{
					agsDriverExtensionsDX12_PopMarker(AmdAgsContext, GraphicsCommandList().Get());
				}
			}
		#endif
		}

	#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::D3D12::EndBreadcrumb(AftermathHandle(), Breadcrumb);
	#endif
	#if INTEL_GPU_CRASH_DUMPS
		UE::RHICore::Intel::GPUCrashDumps::D3D12::EndBreadcrumb(GraphicsCommandList().Get(), Breadcrumb);
	#endif

		FD3D12DiagnosticBuffer* DiagBuffer = Device->GetQueue(QueueType).DiagnosticBuffer.Get();
		if (DiagBuffer && DiagBuffer->IsValid() && UE::RHI::UseGPUCrashBreadcrumbs())
		{
			D3D12_GPU_VIRTUAL_ADDRESS Marker = DiagBuffer->GetGPUQueueMarkerOut();
			WriteMarker(Marker, Breadcrumb->ID, EMarkerType::Out);
		}
	}
#endif // WITH_RHI_BREADCRUMBS

FD3D12ContextCommon::FD3D12ContextCommon(FD3D12Device* Device, ED3D12QueueType QueueType, bool bIsDefaultContext)
	: Device(Device)
	, QueueType(QueueType)
	, bIsDefaultContext(bIsDefaultContext)
	, TimestampQueries(Device, QueueType, D3D12_QUERY_TYPE_TIMESTAMP)
	, OcclusionQueries(Device, QueueType, D3D12_QUERY_TYPE_OCCLUSION)
	, PipelineStatsQueries(Device, QueueType, D3D12_QUERY_TYPE_PIPELINE_STATISTICS)
	, Barriers(Device->GetParentAdapter()->CreateBarriersForContext())
{
}

void FD3D12ContextCommon::WaitSyncPoint(FD3D12SyncPoint* SyncPoint)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12Payload* Payload = GetPayload(EPhase::Wait);
	Payload->SyncPointsToWait.Add(SyncPoint);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::WaitSyncPoint(Payload, SyncPoint));
}

void FD3D12ContextCommon::SignalSyncPoint(FD3D12SyncPoint* SyncPoint)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12Payload* Payload = GetPayload(EPhase::Signal);
	Payload->SyncPointsToSignal.Add(SyncPoint);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SignalSyncPoint(Payload, SyncPoint));
}

void FD3D12ContextCommon::SignalManualFence(ID3D12Fence* Fence, uint64 Value)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12Payload* Payload = GetPayload(EPhase::Signal);
	Payload->ManualFencesToSignal.Emplace(Fence, Value, nullptr);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SignalManualFence(Payload, Fence, Value));
}

void FD3D12ContextCommon::WaitManualFence(ID3D12Fence* Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12Payload* Payload = GetPayload(EPhase::Wait);
	Payload->ManualFencesToWait.Emplace(Fence, Value, bHasSignalled);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::WaitManualFence(Payload, Fence, Value));
}

FD3D12QueryLocation FD3D12ContextCommon::AllocateQuery(ED3D12QueryType Type, void* Target)
{
	switch (Type)
	{
	default:
		checkNoEntry();
		[[fallthrough]];

	case ED3D12QueryType::TimestampRaw:
	case ED3D12QueryType::TimestampMicroseconds:
	case ED3D12QueryType::ProfilerTimestampTOP:
	case ED3D12QueryType::ProfilerTimestampBOP:
		return TimestampQueries.Allocate(Type, Target);

	case ED3D12QueryType::Occlusion:
		return OcclusionQueries.Allocate(Type, Target);

	case ED3D12QueryType::PipelineStats:
		return PipelineStatsQueries.Allocate(Type, Target);
	}
}

FD3D12QueryLocation FD3D12ContextCommon::InsertTimestamp(ED3D12Units Units, uint64* Target)
{
	ED3D12QueryType Type;
	switch (Units)
	{
	default:
		checkNoEntry();
		[[fallthrough]];

	case ED3D12Units::Microseconds: Type = ED3D12QueryType::TimestampMicroseconds; break;
	case ED3D12Units::Raw:          Type = ED3D12QueryType::TimestampRaw;          break;
	}

	FD3D12QueryLocation Location = AllocateQuery(Type, Target);
	EndQuery(Location);

	return Location;
}

void FD3D12ContextCommon::SetReservedBufferCommitSize(FD3D12Buffer* Buffer, uint64 CommitSizeInBytes)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12CommitReservedResourceDesc CommitDesc;
	CommitDesc.Resource = Buffer->GetResource();
	CommitDesc.CommitSizeInBytes = CommitSizeInBytes;

	checkf(CommitDesc.Resource, TEXT("FD3D12CommitReservedResourceDesc::Resource must be set"));

	GetPayload(EPhase::UpdateReservedResources)->ReservedResourcesToCommit.Add(CommitDesc);
}

void FD3D12ContextCommon::OpenCommandList()
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/OpenCommandList"));
	checkf(!IsOpen(), TEXT("Command list is already open."));

	if (CommandAllocator == nullptr)
	{
		// Obtain a command allocator if the context doesn't already have one.
		CommandAllocator = Device->ObtainCommandAllocator(QueueType);
	}

	// Get a new command list
	CommandList = Device->ObtainCommandList(CommandAllocator, &TimestampQueries, &PipelineStatsQueries);
	FD3D12Payload* Payload = GetPayload(EPhase::Execute);
	Payload->CommandListsToExecute.Add(CommandList);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::OpenCommandList(Payload, CommandList->BaseCommandList().GetNoRefCount()));

	check(ActiveQueries == 0);
}

void FD3D12CommandContext::OpenCommandList()
{
	FD3D12ContextCommon::OpenCommandList();

	// Notify the descriptor cache about the new command list
	// This will set the descriptor cache's current heaps on the new command list.
	StateCache.GetDescriptorCache()->OpenCommandList();
}

void FD3D12ContextCommon::CloseCommandList()
{
	checkf(IsPendingCommands(), TEXT("The command list is empty."));
	// Do this before we insert the final timestamp to ensure we're timing all the work on the command list.
	// If the command list only has barrier work to do, this will open the command list for the first time
	FlushResourceBarriers();

	checkf(IsOpen(), TEXT("Command list is not open."));
	checkf(Payloads.Num() && CurrentPhase == EPhase::Execute, TEXT("Expected the current payload to be in the execute phase."));
	
	checkf(ActiveQueries == 0, TEXT("All queries must be completed before the command list is closed."));

	FD3D12Payload* Payload = GetPayload(EPhase::Execute);

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::CloseCommandList(Payload, CommandList->BaseCommandList().GetNoRefCount()));

	CommandList->Close();
	CommandList = nullptr;

	TimestampQueries    .CloseAndReset(Payload->BatchedObjects.QueryRanges);
	OcclusionQueries    .CloseAndReset(Payload->BatchedObjects.QueryRanges);
	PipelineStatsQueries.CloseAndReset(Payload->BatchedObjects.QueryRanges);
}

void FD3D12CommandContext::CloseCommandList()
{
	// FD3D12ContextCommon::CloseCommandList call FlushResourceBarriers that might actually Open the command list
	FD3D12ContextCommon::CloseCommandList();

	StateCache.GetDescriptorCache()->CloseCommandList();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Always call the Bindless Manager CloseCommandList, it will determine when it needs to do anything.
	GetParentDevice()->GetBindlessDescriptorManager().CloseCommandList(*this);
#endif

	// Mark state as dirty now, because ApplyState may be called before OpenCommandList(), and it needs to know that the state has
	// become invalid, so it can set it up again (which opens a new command list if necessary).
	StateCache.DirtyStateForNewCommandList();

#if RHI_RAYTRACING
	RayTracingShaderTables.Empty();
#endif
}

void FD3D12ContextCommon::Finalize(TArray<FD3D12Payload*>& OutPayloads)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	// Collect the context's batch of sync points to wait/signal
	if (BatchedSyncPoints.ToWait.Num())
	{
		FD3D12Payload* Payload = Payloads.Num()
			? Payloads[0]
			: GetPayload(EPhase::Wait);

#if UE_RENDER_TRACING_ENABLED
		for (const FD3D12SyncPointRef& SyncPoint : BatchedSyncPoints.ToWait)
		{
			RenderTracing::D3D12::WaitSyncPoint(Payload, SyncPoint);
		}
#endif

		Payload->SyncPointsToWait.Append(BatchedSyncPoints.ToWait);
		BatchedSyncPoints.ToWait.Reset();
	}

	if (BatchedSyncPoints.ToSignal.Num())
	{
		FD3D12Payload* Payload = GetPayload(EPhase::Signal);

#if UE_RENDER_TRACING_ENABLED
		for (const FD3D12SyncPointRef& SyncPoint : BatchedSyncPoints.ToSignal)
		{
			RenderTracing::D3D12::SignalSyncPoint(Payload, SyncPoint);
		}
#endif

		Payload->SyncPointsToSignal.Append(BatchedSyncPoints.ToSignal);
		BatchedSyncPoints.ToSignal.Reset();
	}

	// Attach the command allocator and query heaps to the last payload.
	// The interrupt thread will release these back to the device object pool.
	if (CommandAllocator)
	{
		GetPayload(EPhase::Signal)->AllocatorsToRelease.Add(CommandAllocator);
		CommandAllocator = nullptr;
	}

	check(!TimestampQueries.HasQueries());
	check(!OcclusionQueries.HasQueries());
	check(!PipelineStatsQueries.HasQueries());

	ContextSyncPoint = nullptr;

	// Move the list of payloads out of this context
	OutPayloads.Append(MoveTemp(Payloads));
}

ERHIPipeline FD3D12ContextCommon::GetRHIPipeline() const
{
	switch (QueueType)
	{
	case ED3D12QueueType::Direct:
		return ERHIPipeline::Graphics;
	case ED3D12QueueType::Async:
		return ERHIPipeline::AsyncCompute;
	default:
		checkNoEntry();
		return ERHIPipeline::None;
	}
	static_assert(static_cast<uint32>(ERHIPipeline::Num) == 2, 
		"Above logic only handles two types of pipes");
}

uint32 FD3D12CommandContext::GetFrameFenceCounter() const
{
	return GetParentDevice()->GetParentAdapter()->GetFrameFence().GetNextFenceToSignal();
}

void FD3D12CommandContext::Finalize(TArray<FD3D12Payload*>& OutPayloads)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	GetParentDevice()->GetBindlessDescriptorManager().FinalizeContext(*this);
#endif

	FlushProfilerStats();

	FD3D12ContextCommon::Finalize(OutPayloads);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FD3D12DescriptorHeap* FD3D12CommandContext::GetBindlessResourcesHeap()
{
	// We require the descriptor cache to be setup correctly before it can have a valid bindless heap.
	OpenIfNotAlready();

	return StateCache.GetDescriptorCache()->GetBindlessResourcesHeap();
}
#endif

FD3D12QueryLocation FD3D12QueryAllocator::Allocate(ED3D12QueryType Type, void* Target)
{
	check(Type != ED3D12QueryType::None);

	// Allocate a new heap if needed
	if (!CurrentRange || CurrentRange->IsFull(CurrentHeap))
	{
		TRefCountPtr<FD3D12QueryHeap> Heap = Device->ObtainQueryHeap(QueueType, QueryType);
		if (!Heap)
		{
			// Unsupported query type
			return {};
		}

		CurrentHeap = Heap;
		CurrentRange = &Heaps.FindOrAdd(MoveTemp(Heap));
	}

	return FD3D12QueryLocation(
		CurrentHeap,
		CurrentRange->End++,
		Type,
		Target
	);
}

void FD3D12QueryAllocator::CloseAndReset(TMap<TRefCountPtr<FD3D12QueryHeap>, TArray<FD3D12QueryRange>>& OutRanges)
{
	if (HasQueries())
	{
		for (auto const& Pair : Heaps)
		{
			OutRanges.FindOrAdd(Pair.Key).Emplace(Pair.Value);
		}

		if (CurrentRange->IsFull(CurrentHeap))
		{
			// No space in any heap. Reset the whole array.
			Heaps.Reset();

			CurrentRange = nullptr;
			CurrentHeap = nullptr;
		}
		else
		{
			// The last heap still has space. Reuse it for the next batch of command lists.
			FD3D12QueryRange LastRange = *CurrentRange;
			LastRange.Start = LastRange.End;

			Heaps.Reset();
			CurrentRange = &Heaps.FindOrAdd(CurrentHeap);
			*CurrentRange = LastRange;
		}
	}
}

FD3D12CopyScope::FD3D12CopyScope(FD3D12Device* Device, ED3D12SyncPointType SyncPointType, FD3D12SyncPointRef const& WaitSyncPoint)
	: Device(Device)
	, SyncPoint(FD3D12SyncPoint::Create(SyncPointType, UE_D3D12_STATIC_FNAME("CopyScope")))
	, Context(*Device->ObtainContextCopy())
{
	if (WaitSyncPoint)
	{
		Context.BatchedSyncPoints.ToWait.Add(WaitSyncPoint);
	}
}

FD3D12CopyScope::~FD3D12CopyScope()
{
#if DO_CHECK
	checkf(bSyncPointRetrieved, TEXT("The copy sync point must be retrieved before the end of the scope."));
#endif

	Context.SignalSyncPoint(SyncPoint);

	TArray<FD3D12Payload*> Payloads;
	Context.Finalize(Payloads);

	Context.ClearState();
	Device->ReleaseContext(&Context);

	FD3D12DynamicRHI::GetD3DRHI()->SubmitPayloads(MoveTemp(Payloads));
}

FD3D12SyncPoint* FD3D12CopyScope::GetSyncPoint() const
{
#if DO_CHECK
	bSyncPointRetrieved = true;
#endif

	return SyncPoint;
}

void FD3D12ContextCommon::NewPayload()
{
	FD3D12Payload* Payload = new FD3D12Payload(Device->GetQueue(QueueType));
	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::AddPayload(this, Payload));
	Payloads.Add(Payload);
}

// Set the Present Params in the context for the Submission Thread to call the native Present
void FD3D12ContextCommon::SetPresent(FD3D12PresentArgs& PresentArgs)
{
	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12Payload* PresentPayload = GetPayload(EPhase::Signal);
	check(PresentPayload->PresentArgs.IsValid() == false);
	PresentPayload->PresentArgs = PresentArgs;
}

void FD3D12ContextCommon::FlushCommands(ED3D12FlushFlags FlushFlags)
{
#if D3D12RHI_IDLE_AFTER_EVERY_GPU_EVENT
	if (!IsDefaultContext())
	{
		return;
	}
#endif

	// We should only be flushing the default context
	check(IsDefaultContext());

	if (IsPendingCommands())
	{
		CloseCommandList();
	}

	FD3D12SyncPointRef SyncPoint;
	FGraphEventRef SubmissionEvent;

	if (EnumHasAnyFlags(FlushFlags, ED3D12FlushFlags::WaitForCompletion))
	{
		SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, UE_D3D12_STATIC_FNAME("WaitForCompletion"));
		SignalSyncPoint(SyncPoint);
	}

	if (EnumHasAnyFlags(FlushFlags, ED3D12FlushFlags::WaitForSubmission))
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		GetPayload(EPhase::Signal)->SubmissionEvent = SubmissionEvent;
	}

	{
		TArray<FD3D12Payload*> LocalPayloads;
		Finalize(LocalPayloads);
		FD3D12DynamicRHI::GetD3DRHI()->SubmitPayloads(MoveTemp(LocalPayloads));
	}

	if (SyncPoint)
	{
		SyncPoint->Wait();
	}

	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SCOPED_NAMED_EVENT_TEXT("Submission_Wait", FColor::Turquoise);
		SubmissionEvent->Wait();
	}
}

void FD3D12ContextCommon::ConditionalSplitCommandList()
{
	// Start a new command list if the total number of commands exceeds the threshold. Too many commands in a single command list can cause TDRs.
	if (IsOpen() && ActiveQueries == 0 && GD3D12MaxCommandsPerCommandList > 0 && CommandList->State.NumCommands > (uint32)GD3D12MaxCommandsPerCommandList)
	{
		UE_LOGF(LogD3D12RHI, Verbose, "Splitting command lists because too many commands have been enqueued already (%d commands)", CommandList->State.NumCommands);
		CloseCommandList();
	}
}

void FD3D12CommandContext::ClearState(EClearStateMode Mode)
{
	StateCache.ClearState();

	bDiscardSharedGraphicsConstants = false;
	bDiscardSharedComputeConstants = false;

	FMemory::Memzero(BoundUniformBuffers, sizeof(BoundUniformBuffers));
	FMemory::Memzero(DirtyUniformBuffers, sizeof(DirtyUniformBuffers));

	if (Mode == EClearStateMode::All)
	{
		FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));
	}
}

void FD3D12CommandContext::ConditionalClearShaderResource(FD3D12ResourceLocation* Resource, EShaderParameterTypeMask ShaderParameterTypeMask)
{
	check(Resource);

	for (int32 Index = 0; Index < SF_NumStandardFrequencies; Index++)
	{
		StateCache.ClearResourceViewCaches(static_cast<EShaderFrequency>(Index), Resource, ShaderParameterTypeMask);
	}
}

void FD3D12CommandContext::ClearShaderResources(FD3D12UnorderedAccessView* UAV, EShaderParameterTypeMask ShaderParameterTypeMask)
{
	if (UAV)
	{
		ConditionalClearShaderResource(UAV->GetResourceLocation(), ShaderParameterTypeMask);
	}
}

void FD3D12CommandContext::ClearShaderResources(FD3D12BaseShaderResource* Resource, EShaderParameterTypeMask ShaderParameterTypeMask)
{
	if (Resource)
	{
		ConditionalClearShaderResource(&Resource->ResourceLocation, ShaderParameterTypeMask);
	}
}

void FD3D12CommandContext::ClearAllShaderResources()
{
	StateCache.ClearSRVs();
}

void FD3D12DynamicRHI::UpdateMemoryStats()
{
#if PLATFORM_WINDOWS && (STATS || CSV_PROFILER_STATS)
	SCOPE_CYCLE_COUNTER(STAT_D3DUpdateVideoMemoryStats);

	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		// Refresh captured memory stats.
		MemoryStats = Adapter->CollectMemoryStats();
		UpdateD3DMemoryStatsAndCSV(MemoryStats, true);
	
#if STATS
		uint64 MaxTexAllocWastage = 0;
		for (FD3D12Device* Device : Adapter->GetDevices())
		{
			Device->GetDefaultBufferAllocator().UpdateMemoryStats();
			Adapter->GetUploadHeapAllocator(Device->GetGPUIndex()).UpdateMemoryStats();
		}
#endif // STATS
	}
#endif // PLATFORM_WINDOWS && (STATS || CSV_PROFILER_STATS)
}

IRHIComputeContext* FD3D12DynamicRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	FRHIGPUMask GPUMask = FRHIGPUMask::All();

	if (GPUMask.HasSingleIndex())
	{
		FD3D12Device* Device = GetAdapter().GetDevice(GPUMask.ToIndex());

		FD3D12CommandContext* CmdContext;
		switch (Pipeline)
		{
		default: checkNoEntry(); // fallthrough
		case ERHIPipeline::Graphics    : CmdContext = Device->ObtainContextGraphics(); break;
		case ERHIPipeline::AsyncCompute: CmdContext = Device->ObtainContextCompute();  break;
		}

		check(CmdContext->GetPhysicalGPUMask() == GPUMask);

		return CmdContext;
	}
	else
	{
		FD3D12CommandContextRedirector* CmdContextRedirector = new FD3D12CommandContextRedirector(&GetAdapter(), GetD3DCommandQueueType(Pipeline), false);
		CmdContextRedirector->SetPhysicalGPUMask(GPUMask);

		for (uint32 GPUIndex : GPUMask)
		{
			FD3D12Device* Device = GetAdapter().GetDevice(GPUIndex);

			FD3D12CommandContext* CmdContext;
			switch (Pipeline)
			{
			default: checkNoEntry(); // fallthrough
			case ERHIPipeline::Graphics    : CmdContext = Device->ObtainContextGraphics(); break;
			case ERHIPipeline::AsyncCompute: CmdContext = Device->ObtainContextCompute();  break;
			}

			CmdContextRedirector->SetPhysicalContext(CmdContext);
		}

		return CmdContextRedirector;
	}
}

void FD3D12DynamicRHI::RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
{
	GetAdapter().CreateTransition(Transition, CreateInfo);
}

void FD3D12DynamicRHI::RHIReleaseTransition(FRHITransition* Transition)
{
	GetAdapter().ReleaseTransition(Transition);
}

IRHITransientResourceAllocator* FD3D12DynamicRHI::RHICreateTransientResourceAllocator()
{
	return new FD3D12TransientResourceHeapAllocator(GetAdapter().GetOrCreateTransientHeapCache());
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FD3D12CommandContextRedirector
//
//////////////////////////////////////////////////////////////////////////////////////////////////////////

FD3D12CommandContextRedirector::FD3D12CommandContextRedirector(class FD3D12Adapter* InParent, ED3D12QueueType QueueType, bool InIsDefaultContext)
	: FD3D12CommandContextBase(InParent, FRHIGPUMask::All())
	, QueueType(QueueType)
	, bIsDefaultContext(InIsDefaultContext)
{
	for (FD3D12CommandContext*& Context : PhysicalContexts)
		Context = nullptr;
}

#if WITH_MGPU
void FD3D12CommandContextRedirector::RHITransferResources(TConstArrayView<FTransferResourceParams> Params)
{
	if (Params.Num() == 0)
		return;

	auto MGPUSync = [this](FRHIGPUMask SignalMask, TOptional<FRHIGPUMask> WaitMask = {})
	{
		FRHIGPUMask CombinedMask = SignalMask;
		if (WaitMask.IsSet())
		{
			CombinedMask |= WaitMask.GetValue();
		}

		// Signal a sync point on each source GPU
		TStaticArray<FD3D12SyncPointRef, MAX_NUM_GPUS> SyncPoints;
		for (uint32 GPUIndex : SignalMask)
		{
			SyncPoints[GPUIndex] = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("TransferResources"));
			PhysicalContexts[GPUIndex]->SignalSyncPoint(SyncPoints[GPUIndex]);
		}

		// Wait for sync points
		if (WaitMask.IsSet())
		{
			for (uint32 WaitGPUIndex : WaitMask.GetValue())
			{
				for (uint32 SignalGPUIndex : SignalMask)
				{
					PhysicalContexts[WaitGPUIndex]->WaitSyncPoint(SyncPoints[SignalGPUIndex]);
				}
			}
		}

		return SyncPoints;
	};

	// Note that by default it is not empty, but GPU0
	FRHIGPUMask SrcMask, DstMask;
	bool bLockstep = GD3D12UnsafeCrossGPUTransfers == false; // @todo mgpu - fix synchronization
	bool bDelayFence = false;

	{
		bool bFirst = true;
		for (const FTransferResourceParams& Param : Params)
		{
			FD3D12CommandContext* SrcContext = PhysicalContexts[Param.SrcGPUIndex];
			FD3D12CommandContext* DstContext = PhysicalContexts[Param.DestGPUIndex];
			if (!ensure(SrcContext && DstContext))
			{
				continue;
			}

			// @todo mgpu - fix synchronization
			bLockstep |= Param.bLockStepGPUs;

			// If it's the first time we set the mask.
			if (bFirst)
			{
				SrcMask = FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				DstMask = FRHIGPUMask::FromIndex(Param.DestGPUIndex);
				bDelayFence = Param.DelayedFence != nullptr;
				bFirst = false;
			}
			else
			{
				SrcMask |= FRHIGPUMask::FromIndex(Param.SrcGPUIndex);
				DstMask |= FRHIGPUMask::FromIndex(Param.DestGPUIndex);
				check(bDelayFence == (Param.DelayedFence != nullptr));
			}

			FD3D12Resource* SrcResource;
			FD3D12Resource* DstResource;

			if (Param.Texture)
			{
				check(Param.Buffer == nullptr);

				SrcResource = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.SrcGPUIndex )->GetResource();
				DstResource = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.DestGPUIndex)->GetResource();
			}
			else
			{
				check(Param.Buffer != nullptr);

				SrcResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.SrcGPUIndex )->GetResource();
				DstResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.DestGPUIndex)->GetResource();
			}
		}
	}

	// Wait on any pre-transfer fences first
	for (const FTransferResourceParams& Param : Params)
	{
		if (Param.PreTransferFence)
		{
			FTransferResourceFenceData* FenceData = Param.PreTransferFence;
			for (uint32 GPUIndex : FenceData->Mask)
			{
				FD3D12SyncPoint* SyncPoint = static_cast<FD3D12SyncPoint*>(FenceData->SyncPoints[GPUIndex]);

				PhysicalContexts[GPUIndex]->WaitSyncPoint(SyncPoint);

				SyncPoint->Release();
			}

			delete FenceData;
		}
	}
	
	// Pre-copy synchronization
	if (bLockstep)
	{
		// Everyone waits for completion of everyone one else.
		MGPUSync(SrcMask | DstMask, SrcMask | DstMask);
	}
	else
	{
		for (const FTransferResourceParams& Param : Params)
		{
			if (Param.bPullData)
			{
				// Destination GPUs wait for source GPUs
				MGPUSync(SrcMask, DstMask);
				break;
			}
		}
	}

	// Enqueue the copy work
	for (const FTransferResourceParams& Param : Params)
	{
		FD3D12CommandContext* SrcContext = PhysicalContexts[Param.SrcGPUIndex];
		FD3D12CommandContext* DstContext = PhysicalContexts[Param.DestGPUIndex];
		if (!ensure(SrcContext && DstContext))
		{
			continue;
		}

		FD3D12CommandContext* CopyContext = Param.bPullData ? DstContext : SrcContext;

		if (Param.Texture)
		{
			FD3D12Texture* SrcTexture = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.SrcGPUIndex);
			FD3D12Texture* DstTexture = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.DestGPUIndex);

			// If the texture size is zero (Max.Z == 0, set in the constructor), copy the whole resource
			if (Param.Max.Z == 0)
			{
				CopyContext->GraphicsCommandList()->CopyResource(DstTexture->GetResource()->GetResource(), SrcTexture->GetResource()->GetResource());

				CopyContext->UpdateResidency(SrcTexture->GetResource());
				CopyContext->UpdateResidency(DstTexture->GetResource());
			}
			else
			{
				// Must be a 2D texture for this code path
				check(Param.Texture->GetTexture2D() != nullptr);

				ensureMsgf(
					Param.Min.X >= 0 && Param.Min.Y >= 0 && Param.Min.Z >= 0 &&
					Param.Max.X >= 0 && Param.Max.Y >= 0 && Param.Max.Z >= 0,
					TEXT("Invalid rect for texture transfer: %i, %i, %i, %i, %i, %i"), Param.Min.X, Param.Min.Y, Param.Min.Z, Param.Max.X, Param.Max.Y, Param.Max.Z);

				D3D12_BOX Box = { (UINT)Param.Min.X, (UINT)Param.Min.Y, (UINT)Param.Min.Z, (UINT)Param.Max.X, (UINT)Param.Max.Y, (UINT)Param.Max.Z };

				CD3DX12_TEXTURE_COPY_LOCATION SrcLocation(SrcTexture->GetResource()->GetResource(), 0);
				CD3DX12_TEXTURE_COPY_LOCATION DstLocation(DstTexture->GetResource()->GetResource(), 0);

				CopyContext->CopyTextureRegionChecked(&DstLocation, Box.left, Box.top, Box.front, DstTexture->GetFormat(), &SrcLocation, &Box, SrcTexture->GetFormat(), DstTexture->GetName(), SrcTexture->GetName());

				CopyContext->UpdateResidency(SrcTexture->GetResource());
				CopyContext->UpdateResidency(DstTexture->GetResource());
			}
		}
		else
		{
			FD3D12Resource* SrcResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.SrcGPUIndex)->GetResource();
			FD3D12Resource* DstResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.DestGPUIndex)->GetResource();

			CopyContext->GraphicsCommandList()->CopyResource(DstResource->GetResource(), SrcResource->GetResource());

			CopyContext->UpdateResidency(SrcResource);
			CopyContext->UpdateResidency(DstResource);
		}
	}

	// Post-copy synchronization
	if (bLockstep)
	{
		// Complete the lockstep by ensuring the GPUs don't start doing something else before the copy completes.
		MGPUSync(SrcMask | DstMask, SrcMask | DstMask);
	}
	else if (bDelayFence)
	{
		auto SyncPoints = MGPUSync(SrcMask | DstMask);

		for (const FTransferResourceParams& Param : Params)
		{
			check(Param.DelayedFence);
			Param.DelayedFence->Mask = SrcMask | DstMask;

			// Copy the sync points into the delayed fence struct. These will be awaited later in RHITransferResourceWait().
			for (int32 Index = 0; Index < SyncPoints.Num(); ++Index)
			{
				FD3D12SyncPointRef& SyncPoint = SyncPoints[Index];

				if (SyncPoint)
				{
					SyncPoint->AddRef();
					Param.DelayedFence->SyncPoints[Index] = SyncPoint.GetReference();
				}
				else
				{
					Param.DelayedFence->SyncPoints[Index] = nullptr;
				}
			}
		}
	}
	else
	{
		// The dest waits for the src to be at this place in the frame before using the data.
		MGPUSync(SrcMask, DstMask);
	}
}

void FD3D12CommandContextRedirector::RHITransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> FenceDatas, FRHIGPUMask SrcGPUMask)
{
	check(FenceDatas.Num() == SrcGPUMask.GetNumActive());

	uint32 FenceIndex = 0;
	for (uint32 SrcGPUIndex : SrcGPUMask)
	{
		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("TransferResourceSignal"));
		SyncPoint->AddRef();

		PhysicalContexts[SrcGPUIndex]->SignalSyncPoint(SyncPoint);

		FTransferResourceFenceData* FenceData = FenceDatas[FenceIndex++];
		FenceData->Mask = FRHIGPUMask::FromIndex(SrcGPUIndex);
		FenceData->SyncPoints[SrcGPUIndex] = SyncPoint;
	}
}

void FD3D12CommandContextRedirector::RHITransferResourceWait(TConstArrayView<FTransferResourceFenceData*> FenceDatas)
{
	FRHIGPUMask AllMasks;
	for (int32 Index = 0; Index < FenceDatas.Num(); ++Index)
	{
		AllMasks = Index == 0
			? FenceDatas[Index]->Mask
			: FenceDatas[Index]->Mask | AllMasks;
	}

	for (FTransferResourceFenceData* FenceData : FenceDatas)
	{
		// Wait for sync points
		for (uint32 WaitGPUIndex : FenceData->Mask)
		{
			for (void* SyncPointPtr : FenceData->SyncPoints)
			{
				if (SyncPointPtr)
				{	
					FD3D12SyncPoint* SyncPoint = static_cast<FD3D12SyncPoint*>(SyncPointPtr);
					PhysicalContexts[WaitGPUIndex]->WaitSyncPoint(SyncPoint);
				}
			}
		}

		// Release sync points
		for (void* SyncPointPtr : FenceData->SyncPoints)
		{
			if (SyncPointPtr)
			{
				static_cast<FD3D12SyncPoint*>(SyncPointPtr)->Release();
			}
		}

		delete FenceData;
	}
}

void FD3D12CommandContextRedirector::RHICrossGPUTransfer(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer, TConstArrayView<FCrossGPUTransferFence*> PostTransfer)
{
	if (Params.Num() == 0)
		return;

	// Wait on any pre-transfer fences first
	for (FCrossGPUTransferFence* PreTransferSyncPoint : PreTransfer)
	{
		FD3D12SyncPoint* SyncPoint = static_cast<FD3D12SyncPoint*>(PreTransferSyncPoint->SyncPoint);

		PhysicalContexts[PreTransferSyncPoint->WaitGPUIndex]->WaitSyncPoint(SyncPoint);

		SyncPoint->Release();

		delete PreTransferSyncPoint;
	}
	
	// Enqueue the copy work
	for (const FTransferResourceParams& Param : Params)
	{
		FD3D12CommandContext* SrcContext = PhysicalContexts[Param.SrcGPUIndex];

		if (Param.Texture)
		{
			FD3D12Texture* SrcTexture = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.SrcGPUIndex);
			FD3D12Texture* DstTexture = FD3D12CommandContext::RetrieveTexture(Param.Texture, Param.DestGPUIndex);

			// If the texture size is zero (Max.Z == 0, set in the constructor), copy the whole resource
			if (Param.Max.Z == 0)
			{
				SrcContext->GraphicsCommandList()->CopyResource(DstTexture->GetResource()->GetResource(), SrcTexture->GetResource()->GetResource());

				SrcContext->UpdateResidency(SrcTexture->GetResource());
				SrcContext->UpdateResidency(DstTexture->GetResource());
			}
			else
			{
				// Must be a 2D texture for this code path
				check(Param.Texture->GetTexture2D() != nullptr);

				ensureMsgf(
					Param.Min.X >= 0 && Param.Min.Y >= 0 && Param.Min.Z >= 0 &&
					Param.Max.X >= 0 && Param.Max.Y >= 0 && Param.Max.Z >= 0,
					TEXT("Invalid rect for texture transfer: %i, %i, %i, %i, %i, %i"), Param.Min.X, Param.Min.Y, Param.Min.Z, Param.Max.X, Param.Max.Y, Param.Max.Z);

				D3D12_BOX Box = { (UINT)Param.Min.X, (UINT)Param.Min.Y, (UINT)Param.Min.Z, (UINT)Param.Max.X, (UINT)Param.Max.Y, (UINT)Param.Max.Z };

				CD3DX12_TEXTURE_COPY_LOCATION SrcLocation(SrcTexture->GetResource()->GetResource(), 0);
				CD3DX12_TEXTURE_COPY_LOCATION DstLocation(DstTexture->GetResource()->GetResource(), 0);

				SrcContext->CopyTextureRegionChecked(&DstLocation, Box.left, Box.top, Box.front, DstTexture->GetFormat(), &SrcLocation, &Box, SrcTexture->GetFormat(), DstTexture->GetName(), SrcTexture->GetName());
				
				SrcContext->UpdateResidency(SrcTexture->GetResource());
				SrcContext->UpdateResidency(DstTexture->GetResource());
			}
		}
		else
		{
			FD3D12Resource* SrcResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.SrcGPUIndex)->GetResource();
			FD3D12Resource* DstResource = FD3D12DynamicRHI::ResourceCast(Param.Buffer.GetReference(), Param.DestGPUIndex)->GetResource();

			SrcContext->GraphicsCommandList()->CopyResource(DstResource->GetResource(), SrcResource->GetResource());

			SrcContext->UpdateResidency(SrcResource);
			SrcContext->UpdateResidency(DstResource);

		}
	}

	// Post-copy synchronization
	FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("RHICrossGPUTransfer"));
	PhysicalContexts[Params[0].SrcGPUIndex]->SignalSyncPoint(SyncPoint);

	for (FCrossGPUTransferFence* PostTransferSyncPoint : PostTransfer)
	{
		// Copy the sync points into the delayed fence struct. These will be awaited later in RHITransferResourceWait().
		SyncPoint->AddRef();
		PostTransferSyncPoint->SyncPoint = SyncPoint.GetReference();
	}
}

void FD3D12CommandContextRedirector::RHICrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer)
{
	for (FCrossGPUTransferFence* TransferSyncPoint : PreTransfer)
	{
		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("RHICrossGPUTransferSignal"));
		SyncPoint->AddRef();

		PhysicalContexts[TransferSyncPoint->SignalGPUIndex]->SignalSyncPoint(SyncPoint);

		TransferSyncPoint->SyncPoint = SyncPoint;
	}
}

void FD3D12CommandContextRedirector::RHICrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> PostTransfer)
{
	for (FCrossGPUTransferFence* TransferSyncPoint : PostTransfer)
	{
		if (TransferSyncPoint->SyncPoint)
		{
			FD3D12SyncPoint* SyncPoint = static_cast<FD3D12SyncPoint*>(TransferSyncPoint->SyncPoint);
			PhysicalContexts[TransferSyncPoint->WaitGPUIndex]->WaitSyncPoint(SyncPoint);

			SyncPoint->Release();
		}

		delete TransferSyncPoint;
	}
}

#endif // WITH_MGPU
