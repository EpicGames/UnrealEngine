// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12CommandContext.h: D3D12 Command Context Interfaces
=============================================================================*/

#pragma once

#include "D3D12Allocation.h"
#include "D3D12BindlessDescriptors.h"
#include "D3D12CommandList.h"
#include "D3D12Queue.h"
#include "D3D12Query.h"
#include "D3D12Resources.h"
#include "D3D12StateCachePrivate.h"
#include "D3D12Submission.h"
#include "D3D12Texture.h"
#include COMPILED_PLATFORM_HEADER(D3D12BarriersFactory.h)
#include "Experimental/Containers/RobinHoodHashTable.h"

#include "RHICoreShader.h"
#include "RHICore.h"
#include "RHIShaderBindingLayout.h"

#include "GPUProfiler.h"

#if USE_PIX
	#include "Windows/AllowWindowsPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START
		#include <pix3.h>
	#include "Windows/HideWindowsPlatformTypes.h"
	THIRD_PARTY_INCLUDES_END
#endif

enum class ED3D12PipelineType : uint8;

struct FD3D12DescriptorHeap;
struct FRayTracingShaderBindings;
struct FD3D12ViewSubset;
class FD3D12Device;
class FD3D12Resource;
class FD3D12Heap;
struct FD3D12DescriptorHeap;
class FD3D12ResourceLocation;
class FD3D12RootSignature;
class FD3D12ExplicitDescriptorCache;

struct FD3D12DeferredDeleteObject
{
	enum class EType
	{
		RHIObject,
		D3DObject,
		Heap,
		DescriptorHeap,
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		DescriptorAllocation,
		BindlessDescriptor,
		BindlessDescriptorHeap,
#endif
		CPUAllocation,
		DescriptorBlock,
		VirtualAllocation,
		Func,
		TextureStagingBuffer
	} Type;

	union
	{
		FD3D12Resource* RHIObject;
		FD3D12Heap* Heap;
		FD3D12DescriptorHeap* DescriptorHeap;
		ID3D12Object* D3DObject;

		TUniqueFunction<void()>* Func;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		struct
		{
			FRHIDescriptorAllocation Allocation;
			ERHIDescriptorTypeMask TypeMask;
			FD3D12Device* Device;
		} DescriptorAllocation;

		struct
		{
			FRHIDescriptorHandle Handle;
			FD3D12Device* Device;
		} BindlessDescriptor;
#endif

		void* CPUAllocation;

		struct
		{
			FD3D12OnlineDescriptorBlock* Block;
			FD3D12OnlineDescriptorManager* Manager;
		} DescriptorBlock;

		struct
		{
			FPlatformMemory::FPlatformVirtualMemoryBlock VirtualBlock;
			ETextureCreateFlags Flags;
			uint64 CommittedTextureSize;
			void* RawMemory;
		} VirtualAllocDescriptor;

		struct
		{
			FD3D12Texture* Texture;
			alignas(alignof(TUniquePtr<FD3D12LockedResource>)) uint8 LockedResourceStorage[sizeof(TUniquePtr<FD3D12LockedResource>)];
			uint32 Subresource;
		} TextureStagingBufferData;
	};

	explicit FD3D12DeferredDeleteObject(FD3D12Resource* RHIObject)
		: Type(EType::RHIObject)
		, RHIObject(RHIObject)
	{}

	explicit FD3D12DeferredDeleteObject(FD3D12Heap* InHeap)
		: Type(EType::Heap)
		, Heap(InHeap)
	{}

	explicit FD3D12DeferredDeleteObject(FD3D12DescriptorHeap* InDescriptorHeap, EType Type)
		: Type(Type)
		, DescriptorHeap(InDescriptorHeap)
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		check(Type == EType::BindlessDescriptorHeap || Type == EType::DescriptorHeap);
#else
		check(Type == EType::DescriptorHeap);
#endif
	}

	explicit FD3D12DeferredDeleteObject(ID3D12Object* D3DObject)
		: Type(EType::D3DObject)
		, D3DObject(D3DObject)
	{}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	explicit FD3D12DeferredDeleteObject(FRHIDescriptorAllocation Allocation, ERHIDescriptorTypeMask TypeMask, FD3D12Device* Device)
		: Type(EType::DescriptorAllocation)
		, DescriptorAllocation({ Allocation, TypeMask, Device })
	{}

	explicit FD3D12DeferredDeleteObject(FRHIDescriptorHandle Handle, FD3D12Device* Device)
		: Type(EType::BindlessDescriptor)
		, BindlessDescriptor({ Handle, Device })
	{}
#endif

	explicit FD3D12DeferredDeleteObject(void* Ptr, EType Type)
		: Type(Type)
		, CPUAllocation(Ptr)
	{
		check(Type == EType::CPUAllocation);
	}

	explicit FD3D12DeferredDeleteObject(FD3D12OnlineDescriptorBlock* Block, FD3D12OnlineDescriptorManager* Manager)
		: Type(EType::DescriptorBlock)
		, DescriptorBlock({ Block, Manager })
	{}

	explicit FD3D12DeferredDeleteObject(FPlatformMemory::FPlatformVirtualMemoryBlock& VirtualBlock, ETextureCreateFlags Flags, uint64 CommittedTextureSize, void* RawMemory)
		: Type(EType::VirtualAllocation)
		, VirtualAllocDescriptor({ VirtualBlock, Flags, CommittedTextureSize, RawMemory })
	{}

	explicit FD3D12DeferredDeleteObject(TUniqueFunction<void()>&& Func)
		: Type(EType::Func)
		, Func(new TUniqueFunction<void()>(MoveTemp(Func)))
	{}

	explicit FD3D12DeferredDeleteObject(FD3D12Texture* InTexture, TUniquePtr<FD3D12LockedResource>&& InLockedResource, uint32 InSubresource)
		: Type(EType::TextureStagingBuffer)
	{
		// Add a ref, in case texture gets destroyed while this deferred delete is in flight.  We can't use TRefCountPtr in the union.
		TextureStagingBufferData.Texture = InTexture;
		InTexture->AddRef();

		// We can't use TUniquePtr in the union, so do placement new.
		TUniquePtr<FD3D12LockedResource>* LockedResource = new (TextureStagingBufferData.LockedResourceStorage) TUniquePtr<FD3D12LockedResource>();
		*LockedResource = MoveTemp(InLockedResource);

		TextureStagingBufferData.Subresource = InSubresource;
	}
};

enum class ED3D12Units
{
	Raw,
	Microseconds
};

enum class ED3D12FlushFlags
{
	None = 0,

	// Block the calling thread until the submission thread has dispatched all work.
	WaitForSubmission = 1,

	// Both the calling thread until the GPU has signaled completion of all dispatched work.
	WaitForCompletion = 2
};
ENUM_CLASS_FLAGS(ED3D12FlushFlags)

//
// Base class that manages the recording of FD3D12FinalizedCommands instances.
// Manages the logic for creating and recycling command lists and allocators.
//
class FD3D12ContextCommon
{
	friend class FScopedResourceBarrier;

protected:
	FD3D12ContextCommon(FD3D12Device* Device, ED3D12QueueType QueueType, bool bIsDefaultContext);

public:
	virtual ~FD3D12ContextCommon() = default;

protected:
	virtual void OpenCommandList();
	virtual void CloseCommandList();

public:
	enum class EClearStateMode
	{
		TransientOnly,
		All
	};

	virtual void ClearState(EClearStateMode ClearStateMode = EClearStateMode::All) {}
	virtual void ConditionalClearShaderResource(FD3D12ResourceLocation* Resource, EShaderParameterTypeMask ShaderParameterTypeMask) {}


	// Inserts a command to signal the specified sync point
	void SignalSyncPoint(FD3D12SyncPoint* SyncPoint);

	// Inserts a command that blocks the GPU queue until the specified sync point is signaled.
	void WaitSyncPoint(FD3D12SyncPoint* SyncPoint);

	// Inserts a command that signals the specified D3D12 fence object.
	void SignalManualFence(ID3D12Fence* Fence, uint64 Value);

	// Inserts a command that waits the specified D3D12 fence object.
	void WaitManualFence(ID3D12Fence* Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled);

	// Inserts a timestamp query command. "Target" specifies the optional 
	// location the result will be written to by the interrupt handler thread.
	FD3D12QueryLocation InsertTimestamp(ED3D12Units Units, uint64* Target);

	// Allocates a query of the specified type, returning its location.
	FD3D12QueryLocation AllocateQuery(ED3D12QueryType Type, void* Target);

	// Resizes physical memory allocation for a buffer. Allocates new backing heaps as necessary.
	// Causes the command list to be split, as reserved resource update operations are performed on the D3D12 queue.
	// The actual work is deferred via FD3D12Payload.
	void SetReservedBufferCommitSize(FD3D12Buffer* Buffer, uint64 CommitSizeInBytes);

	// Complete recording of the current command list set, and appends the resulting
	// payloads to the given array. Resets the context so new commands can be recorded.
	virtual void Finalize(TArray<FD3D12Payload*>& OutPayloads);

	// The owner device of this context
	FD3D12Device* const Device;

	// The type of command lists this context records.
	ED3D12QueueType const QueueType;
	bool IsAsyncComputeContext() const { return QueueType == ED3D12QueueType::Async; }
	ERHIPipeline GetRHIPipeline() const;

	// True for the immediate context (@todo remove this)
	bool const bIsDefaultContext;
	bool IsDefaultContext() const { return bIsDefaultContext; }

	bool IsOpen() const { return CommandList != nullptr; }
	bool IsPendingCommands() const { return IsOpen() || Barriers->GetNumPendingBarriers(); }

	FD3D12SyncPoint* GetContextSyncPoint()
	{
		if (!ContextSyncPoint)
		{
			ContextSyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, UE_D3D12_STATIC_FNAME("ContextSyncPoint"));
			BatchedSyncPoints.ToSignal.Add(ContextSyncPoint);
		}

		return ContextSyncPoint;
	}

	// Sync points which are waited at the start / signaled at the end 
	// of the whole batch of command lists this context recorded.
	struct
	{
		TArray<FD3D12SyncPointRef> ToWait;
		TArray<FD3D12SyncPointRef> ToSignal;
	} BatchedSyncPoints;

	void BindDiagnosticBuffer(FD3D12RootSignature const* RootSignature, ED3D12PipelineType PipelineType);

	FD3D12QueryAllocator& GetTimestampQueries()
	{
		return TimestampQueries;
	}

private:
	// Allocators to manage query heaps
	FD3D12QueryAllocator TimestampQueries;
	FD3D12QueryAllocator OcclusionQueries;
	FD3D12QueryAllocator PipelineStatsQueries;

	// The active D3D12 command list where recorded D3D commands are directed.
	// This is swapped when command lists are split (e.g. when signalling a fence).
	FD3D12CommandList* CommandList = nullptr;

	// The command allocator used to open command lists within this context.
	// The allocator is reused for each new command list until the context is finalized.
	FD3D12CommandAllocator* CommandAllocator = nullptr;

	// The array of recorded payloads the submission thread will process.
	// These are returned when the context is finalized.
	TArray<FD3D12Payload*> Payloads;

	// A sync point signaled when all payloads in this context have completed.
	FD3D12SyncPointRef ContextSyncPoint;

public:
	void BeginRecursiveCommand()
	{
		// Nothing to do
	}

	// Returns the current command list (or creates a new one if the command list was not open).
	FD3D12CommandList& GetCommandList()
	{
		OpenIfNotAlready();

		return *CommandList;
	}

protected:
	enum class EMarkerType { In, Out };
	void WriteMarker(D3D12_GPU_VIRTUAL_ADDRESS Address, uint32 Value, EMarkerType Type);

	enum class EPhase
	{
		Wait,
		UpdateReservedResources,
		Execute,
		Signal
	} CurrentPhase = EPhase::Wait;

	FD3D12Payload* GetPayload(EPhase Phase)
	{
		if (Payloads.Num() == 0 || Phase < CurrentPhase)
		{
			NewPayload();
		}

		CurrentPhase = Phase;
		return Payloads.Last();
	}

	void NewPayload();

	uint32 ActiveQueries = 0;

	const TUniquePtr<FD3D12BarriersFactory::BarriersForContextType> Barriers;

public:
	// Open the command list if it's not already open.
	void OpenIfNotAlready()
	{
		if (!CommandList)
		{
			OpenCommandList();
		}
	}

	// Set the Present Params in the context for the Submission Thread to call the native Present
	void SetPresent(FD3D12PresentArgs& PresentArgs);

	// Flushes any pending commands in this context to the GPU.
	void FlushCommands(ED3D12FlushFlags FlushFlags = ED3D12FlushFlags::None);

	// Closes the current command list if the number of enqueued commands exceeds
	// the threshold defined by the "D3D12.MaxCommandsPerCommandList" cvar.
	void ConditionalSplitCommandList();

	auto BaseCommandList      () { return GetCommandList().BaseCommandList(); }
	auto CopyCommandList      () { return GetCommandList().CopyCommandList(); }
	auto GraphicsCommandList  () { return GetCommandList().GraphicsCommandList(); }
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 1
	auto GraphicsCommandList1 () { return GetCommandList().GraphicsCommandList1(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 2
	auto GraphicsCommandList2 () { return GetCommandList().GraphicsCommandList2(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 3
	auto GraphicsCommandList3 () { return GetCommandList().GraphicsCommandList3(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 4
	auto GraphicsCommandList4 () { return GetCommandList().GraphicsCommandList4(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 5
	auto GraphicsCommandList5 () { return GetCommandList().GraphicsCommandList5(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 6
	auto GraphicsCommandList6 () { return GetCommandList().GraphicsCommandList6(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 7
	auto GraphicsCommandList7 () { return GetCommandList().GraphicsCommandList7(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 8
	auto GraphicsCommandList8 () { return GetCommandList().GraphicsCommandList8(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 9
	auto GraphicsCommandList9 () { return GetCommandList().GraphicsCommandList9(); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 10
	auto GraphicsCommandList10() { return GetCommandList().GraphicsCommandList10(); }
#endif

#if D3D12_SUPPORTS_DEBUG_COMMAND_LIST			    
	auto DebugCommandList     () { return GetCommandList().DebugCommandList(); }
#endif
#if D3D12_RHI_RAYTRACING
	auto RayTracingCommandList() { return GetCommandList().RayTracingCommandList(); }
#endif
#if NV_AFTERMATH
	auto AftermathHandle      () { return GetCommandList().AftermathHandle(); }
#endif

	void BeginQuery(FD3D12QueryLocation const& Location) { GetCommandList().BeginQuery(Location); }
	void EndQuery  (FD3D12QueryLocation const& Location) { GetCommandList().EndQuery  (Location); }

#if ENABLE_RESIDENCY_MANAGEMENT
	void UpdateResidency(const FD3D12Resource* Resource) { check(Resource); GetCommandList().UpdateResidency(Resource); }
#else
	void UpdateResidency(const FD3D12Resource* Resource) { }
#endif

	// Resource transition / barrier functions. These get batched and recorded into the command list when FlushResourceBarriers() is called.
	void AddGlobalBarrier(
		ED3D12Access InD3D12AccessBefore,
		ED3D12Access InD3D12AccessAfter);

	void AddBarrier(
		const FD3D12Resource* pResource,
		ED3D12Access InD3D12AccessBefore,
		ED3D12Access InD3D12AccessAfter,
		uint32 Subresource);

	// Flushes the batched resource barriers to the current command list
	void FlushResourceBarriers();
};

//
// Context for the copy queue. Doesn't implement an RHI interface 
// since the copy queue is not directly exposed to the renderer.
//
class FD3D12ContextCopy final : public FD3D12ContextCommon
{
public:
	FD3D12ContextCopy(FD3D12Device* Device)
		: FD3D12ContextCommon(Device, ED3D12QueueType::Copy, false)
	{}
};

//
// Helper for recording and submitting copy queue work.
// Used for buffer / texture data upload etc.
//
class FD3D12CopyScope final
{
private:
	FD3D12Device* const Device;
	FD3D12SyncPointRef SyncPoint;

#if DO_CHECK
	mutable bool bSyncPointRetrieved = false;
#endif

public:
	FD3D12ContextCopy& Context;

	FD3D12SyncPoint* GetSyncPoint() const;

	FD3D12CopyScope(FD3D12Device* Device, ED3D12SyncPointType SyncPointType, FD3D12SyncPointRef const& WaitSyncPoint = {});
	~FD3D12CopyScope();
};

// Base class used to define commands that are not device specific, or that broadcast to all devices.
// @todo mgpu - try to remove this class
class FD3D12CommandContextBase : public IRHICommandContext, public FD3D12AdapterChild
{
public:
	FD3D12CommandContextBase(FD3D12Adapter* InParent, FRHIGPUMask InGPUMask);

	FRHIGPUMask GetGPUMask() const { return GPUMask; }
	FRHIGPUMask GetPhysicalGPUMask() const { return PhysicalGPUMask; }

	virtual void RHISetAsyncComputeBudget(EAsyncComputeBudget Budget) {}

	virtual class FD3D12CommandContextRedirector* AsRedirector() { return nullptr; }

	static FD3D12CommandContextBase& Get(FRHICommandListBase& RHICmdList)
	{
		return static_cast<FD3D12CommandContextBase&>(RHICmdList.GetComputeContext().GetLowestLevelContext());
	}

	static FD3D12CommandContextBase* Get(IRHIComputeContext* RHIContext)
	{
		return RHIContext ? static_cast<FD3D12CommandContextBase*>(&RHIContext->GetLowestLevelContext()) : nullptr;
	}

	virtual FD3D12CommandContext* GetSingleDeviceContext(uint32 InGPUIndex) = 0;

protected:
	friend class FD3D12CommandContext;

	FRHIGPUMask GPUMask;
	FRHIGPUMask PhysicalGPUMask;
};

// RHI Context type used for graphics and async compute command lists.
class FD3D12CommandContext : public FD3D12ContextCommon, public FD3D12CommandContextBase, public FD3D12DeviceChild
{
public:
	FD3D12CommandContext(class FD3D12Device* InParent, ED3D12QueueType QueueType, bool InIsDefaultContext);
	virtual ~FD3D12CommandContext();

	static FD3D12CommandContext& Get(FRHICommandListBase& RHICmdList, uint32 GPUIndex)
	{
		FD3D12CommandContextBase& Base = FD3D12CommandContextBase::Get(RHICmdList);
#if WITH_MGPU
		return *Base.GetSingleDeviceContext(GPUIndex);
#else
		return static_cast<FD3D12CommandContext&>(Base);
#endif
	}

	virtual void OpenCommandList() override;
	virtual void CloseCommandList() override final;

	virtual ERHIPipeline GetPipeline() const override
	{
		return QueueType == ED3D12QueueType::Direct
			? ERHIPipeline::Graphics
			: ERHIPipeline::AsyncCompute;
	}

	virtual void ClearState(EClearStateMode ClearStateMode = EClearStateMode::All) override final;
	virtual void ConditionalClearShaderResource(FD3D12ResourceLocation* Resource, EShaderParameterTypeMask ShaderParameterTypeMask) override final;
	void ClearShaderResources(FD3D12UnorderedAccessView* UAV, EShaderParameterTypeMask ShaderParameterTypeMask);
	void ClearShaderResources(FD3D12BaseShaderResource* Resource, EShaderParameterTypeMask ShaderParameterTypeMask);
	void ClearAllShaderResources();

	void FlushProfilerStats()
	{
		// Flush accumulated draw stats
		if (StatEvent)
		{
			GetCommandList().EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FStats>() = StatEvent;
			StatEvent = {};
		}
	}

	FD3D12FastConstantAllocator ConstantsAllocator;

	// Current GPU event stack
	TArray<uint32> GPUEventStack;

	FD3D12StateCache StateCache;

	/** Track the currently bound uniform buffers. */
	FD3D12UniformBuffer* BoundUniformBuffers[SF_NumStandardFrequencies][MAX_CBS] = {};

	/** Bit array to track which uniform buffers have changed since the last draw call. */
	uint16 DirtyUniformBuffers[SF_NumStandardFrequencies] = {};

	/** Handle for the dummy outer occlusion query we optionally insert for performance reasons */
	FRenderQueryRHIRef OuterOcclusionQuery;
	bool bOuterOcclusionQuerySubmitted = false;

	/** When a new graphics PSO is set, we discard all old constants set for the previous shader. */
	bool bDiscardSharedGraphicsConstants = false;

	/** When a new compute PSO is set, we discard all old constants set for the previous shader. */
	bool bDiscardSharedComputeConstants = false;

	/** Used by variable rate shading to cache the current state of the combiners and the constant shading rate*/
#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
	static_assert(D3D12_RS_SET_SHADING_RATE_COMBINER_COUNT == ED3D12VRSCombinerStages::Num);
	D3D12_SHADING_RATE_COMBINER		VRSCombiners[D3D12_RS_SET_SHADING_RATE_COMBINER_COUNT] = { D3D12_SHADING_RATE_COMBINER_PASSTHROUGH, D3D12_SHADING_RATE_COMBINER_PASSTHROUGH };
	D3D12_SHADING_RATE				VRSShadingRate = D3D12_SHADING_RATE_1X1;
#endif

	/** Constant buffers for Set*ShaderParameter calls. */
	FD3D12ConstantBuffer StageConstantBuffers[SF_NumStandardFrequencies];

#if PLATFORM_SUPPORTS_VIRTUAL_TEXTURES
	bool bNeedFlushTextureCache = false;
	void InvalidateTextureCache() { bNeedFlushTextureCache = true; }
	inline void FlushTextureCacheIfNeeded()
	{
		if (bNeedFlushTextureCache)
		{
			FlushTextureCache();

			bNeedFlushTextureCache = false;
		}
	}
	virtual void FlushTextureCache() {};
#endif

#if RHI_RAYTRACING
	// Used to deduplicate work done by the shader table on this context.
	Experimental::TRobinHoodHashSet<uint64> RayTracingShaderTables;
#endif

	/** needs to be called before each draw call */
	void CommitNonComputeShaderConstants();

	/** needs to be called before each dispatch call */
	void CommitComputeShaderConstants();

	template <class ShaderType> void SetResourcesFromTables(const ShaderType* RESTRICT);

	void SetSRVParameter(EShaderFrequency Frequency, uint32 SRVIndex, FD3D12ShaderResourceView* SRV);
	void SetUAVParameter(EShaderFrequency Frequency, uint32 UAVIndex, FD3D12UnorderedAccessView* UAV);

	void CommitGraphicsResourceTables();
	void CommitComputeResourceTables();

	template<typename TPixelShader>
	void ResolveTextureUsingShader(
		FD3D12Texture* SourceTexture,
		FD3D12Texture* DestTexture,
		FD3D12RenderTargetView* DestSurfaceRTV,
		FD3D12DepthStencilView* DestSurfaceDSV,
		const D3D12_RESOURCE_DESC& ResolveTargetDesc,
		const FResolveRect& SourceRect,
		const FResolveRect& DestRect,
		typename TPixelShader::FParameter PixelShaderParameter
		);

	virtual void SetDepthBounds(float MinDepth, float MaxDepth);
	virtual void SetShadingRate(EVRSShadingRate ShadingRate, FD3D12Resource* ShadingRateImage, const TStaticArray<EVRSRateCombiner, ED3D12VRSCombinerStages::Num>& Combiners);

	virtual void SetAsyncComputeBudgetInternal(EAsyncComputeBudget Budget) {}

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) final override;
	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) final override;

	// IRHIComputeContext interface
	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) final override;
	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override;
	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override;
	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
	virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
	virtual void RHIEndBreadcrumbGPU  (FRHIBreadcrumbNode* Breadcrumb) final override;
#endif

	// IRHICommandContext interface
	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override;
	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override;
	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override;
	virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) final override;
	virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) final override;
	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBuffer, FRHIStagingBuffer* DestinationStagingBuffer, uint32 Offset, uint32 NumBytes) final override;
	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override;
	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override;
	virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) override;
	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override;
	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsPipelineState, uint32 StencilRef, bool bApplyAdditionalState) final override;
	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	virtual void RHISetStencilRef(uint32 StencilRef) final override;
	virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) final override;

	void SetRenderTargets(uint32 NumSimultaneousRenderTargets, const FRHIRenderTargetView* NewRenderTargets, const FRHIDepthRenderTargetView* NewDepthStencilTarget);
	void SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo);

	virtual void RHISetShaderRootConstants(
		const FUint32Vector4& Constants) override;

	virtual void RHIDispatchComputeShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
		ERHIShaderBundleMethod Method) override;

	virtual void RHIDispatchGraphicsShaderBundle(
		FRHIShaderBundle* ShaderBundle,
		FRHIBuffer* RecordArgBuffer,
		const FRHIShaderBundleGraphicsState& BundleState,
		TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
		TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
		ERHIShaderBundleMethod Method) override;

	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBufferOffset, uint32 MaxDrawArguments) final override;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
#endif
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override;
    virtual void RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner) final override;

	virtual void RHIClearMRTImpl(bool* bClearColorArray, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);

	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
	{
		FRHISetRenderTargetsInfo RTInfo;
		InInfo.ConvertToRenderTargetsInfo(RTInfo);
		SetRenderTargetsAndClear(RTInfo);

		RenderPassInfo = InInfo;
	}

	virtual void RHIEndRenderPass()
	{
		UE::RHICore::ResolveRenderPassTargets(RenderPassInfo, [this](UE::RHICore::FResolveTextureInfo Info)
		{
			ResolveTexture(Info);
		});

		// Buffers used as vertex streams in this pass can be transitioned to other states after the pass ends. If such a buffer
		// was bound to a slot that's not used in a subsequent draw, the cache will apply the stale binding, resulting in validation
		// layer errors if the new state doesn't include D3D12_RESOURCE_STATE_VERTEX_AND_CONSTANT_BUFFER (despite the shader not actually
		// using the buffer - the validation layer can't know that). We clear the VB cache to prevent this case.
		StateCache.ClearVertexBuffers();
	}

	void ResolveTexture(UE::RHICore::FResolveTextureInfo Info);

#if D3D12_RHI_RAYTRACING
	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) final override;
	virtual void BuildAccelerationStructuresInternal(TConstArrayView<D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC> BuildDesc);
#if WITH_NVAPI
	virtual void BuildAccelerationStructuresInternalEx(const TArrayView<NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX> ParamsArray);
#endif
#if WITH_MGPU
	// Should be called before RHIBuildAccelerationStructures when multiple GPU support is present (for example, from FD3D12CommandContextRedirector::RHIBuildAccelerationStructures)
	static void UnregisterAccelerationStructuresInternalMGPU(TConstArrayView<FRayTracingGeometryBuildParams> Params, FRHIGPUMask GPUMask);
#endif
	virtual void RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) final override;
	virtual void RHICommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer) final override;

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height) final override;
	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;

	virtual void RHISetBindingsOnShaderBindingTable(
		FRHIShaderBindingTable* InSBT, FRHIRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType) final override;
#endif // D3D12_RHI_RAYTRACING

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE TReturnType* ResourceCast(TRHIType* Resource)
	{
		return static_cast<TReturnType*>(Resource);
	}

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE_DEBUGGABLE TReturnType* ResourceCast(TRHIType* Resource, uint32 GPUIndex)
	{
		TReturnType* Object = ResourceCast<TRHIType, TReturnType>(Resource);
		return Object ? static_cast<TReturnType*>(Object->GetLinkedObject(GPUIndex)) : nullptr;
	}

	template<typename ObjectType, typename RHIType>
	static FORCEINLINE_DEBUGGABLE ObjectType* RetrieveObject(RHIType* RHIObject, uint32 GPUIndex)
	{
		return ResourceCast<RHIType, ObjectType>(RHIObject, GPUIndex);
	}

	template<typename ObjectType, typename RHIType>
	FORCEINLINE_DEBUGGABLE ObjectType* RetrieveObject(RHIType* RHIObject)
	{
		return RetrieveObject<ObjectType, RHIType>(RHIObject, GetGPUIndex());
	}

	static inline FD3D12Texture* RetrieveTexture(FRHITexture* Texture, uint32 GPUIndex)
	{
		FD3D12Texture* RHITexture = GetD3D12TextureFromRHITexture(Texture);
		return RHITexture ? RHITexture->GetLinkedObject(GPUIndex) : nullptr;
	}

	FORCEINLINE_DEBUGGABLE FD3D12Texture* RetrieveTexture(FRHITexture* Texture)
	{
		return RetrieveTexture(Texture, GetGPUIndex());
	}

	FORCEINLINE_DEBUGGABLE const FD3D12Texture* RetrieveTexture(const FRHITexture* Texture)
	{
		return RetrieveTexture(const_cast<FRHITexture*>(Texture), GetGPUIndex());
	}

	uint32 GetFrameFenceCounter() const;

	uint32 GetGPUIndex() const { return GPUMask.ToIndex(); }

	virtual void RHISetGPUMask(FRHIGPUMask InGPUMask) final override
	{
		// This is a single-GPU context so it doesn't make sense to ever change its GPU
		// mask. If multiple GPUs are supported we should be using the redirector context.
		ensure(InGPUMask == GPUMask);
	}

	inline const TArray<FRHIUniformBuffer*>& GetStaticUniformBuffers() const
	{
		return StaticUniformBuffers;
	}

	void FlushPendingDescriptorUpdates();

	void SetExplicitDescriptorCache(FD3D12ExplicitDescriptorCache& ExplicitDescriptorCache);
	void UnsetExplicitDescriptorCache();

	virtual void Finalize(TArray<FD3D12Payload*>& OutPayloads) override;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12DescriptorHeap* GetBindlessResourcesHeap();
	FD3D12ContextBindlessState& GetBindlessState() { return BindlessState; }
#endif

	const FRHIShaderBindingLayout& GetShaderBindingLayout() const
	{
		static const FRHIShaderBindingLayout Default;
		return ShaderBindinglayout ? *ShaderBindinglayout : Default;
	}

	FORCENOINLINE void CopyBufferRegionChecked(
		ID3D12Resource* DestResource, const FName& DestName, uint64 DestOffset,
		ID3D12Resource* SourceResource, const FName& SourceName, uint64 SourceOffset,
		uint32 ByteCount
	);
	
	FORCENOINLINE void CopyTextureRegionChecked(
		const D3D12_TEXTURE_COPY_LOCATION* DestCopyLocation, int DestX, int DestY, int DestZ, EPixelFormat DestPixelFormat,
		const D3D12_TEXTURE_COPY_LOCATION* SourceCopyLocation, const D3D12_BOX* SourceBox, EPixelFormat SourcePixelFormat,
		const FName& DestName, const FName& SourceName
	);

protected:

	FD3D12CommandContext* GetSingleDeviceContext(uint32 InGPUIndex) final override
	{  
		return InGPUIndex == GetGPUIndex() ? this : nullptr; 
	}

private:
	void SetupDispatch(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ);
	void SetupDraw(FRHIBuffer* IndexBufferRHI, uint32 NumPrimitives = 0, uint32 NumVertices = 0);
	void SetupDispatchDraw(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ);
	FD3D12ResourceLocation& SetupIndirectArgument(FRHIBuffer* ArgumentBufferRHI, D3D12_RESOURCE_STATES ExtraStates = static_cast<D3D12_RESOURCE_STATES>(0));
	void PostGpuEvent();

	static void ClearUAV(TRHICommandList_RecursiveHazardous<FD3D12CommandContext>& RHICmdList, FD3D12UnorderedAccessView_RHI* UAV, const void* ClearValues, bool bFloat);

	void DispatchWorkGraphShaderBundle(FRHIShaderBundle* ShaderBundle, FRHIBuffer* RecordArgBuffer, TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters, TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches);
	void DispatchWorkGraphShaderBundle(FRHIShaderBundle* ShaderBundle, FRHIBuffer* RecordArgBuffer, const FRHIShaderBundleGraphicsState& BundleState, TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters, TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches);

	TArray<FRHIUniformBuffer*> StaticUniformBuffers;
	const FRHIShaderBindingLayout* ShaderBindinglayout = nullptr;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12ContextBindlessState BindlessState;
#endif
};

// Version of command context to handle multi-GPU.  Because IRHICommandContext is pure virtual we can return the normal
// FD3D12CommandContext when not using mGPU, thus there is no additional overhead for the common case i.e. 1 GPU.
class FD3D12CommandContextRedirector final : public FD3D12CommandContextBase
{
public:
	// The type of command lists this context records.
	ED3D12QueueType const QueueType;
	bool const bIsDefaultContext;

	FD3D12CommandContextRedirector(class FD3D12Adapter* InParent, ED3D12QueueType QueueType, bool InIsDefaultContext);

	virtual FD3D12CommandContextRedirector* AsRedirector() override { return this; }

#define ContextRedirect(Call) { for (uint32 GPUIndex : GPUMask) PhysicalContexts[GPUIndex]-> Call; }
#define ContextGPU0(Call) { PhysicalContexts[0]-> Call; }

	FORCEINLINE virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) final override
	{
		ContextRedirect(RHISetComputePipelineState(ComputePipelineState));
	}
	FORCEINLINE virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override
	{
		ContextRedirect(RHIDispatchComputeShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ));
	}
	FORCEINLINE virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ContextRedirect(RHIDispatchIndirectComputeShader(ArgumentBuffer, ArgumentOffset));
	}

	FORCEINLINE virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) final override
	{
		ContextRedirect(RHIBeginTransitions(Transitions));
	}
	FORCEINLINE virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) final override
	{
		ContextRedirect(RHIEndTransitions(Transitions));
	}

#if WITH_MGPU
	virtual void RHITransferResources(TConstArrayView<FTransferResourceParams> Params) final override;
	virtual void RHITransferResourceSignal(TConstArrayView<FTransferResourceFenceData*> FenceDatas, FRHIGPUMask SrcGPUMask) final override;
	virtual void RHITransferResourceWait(TConstArrayView<FTransferResourceFenceData*> FenceDatas) final override;

	// New and improved cross GPU transfer API
	virtual void RHICrossGPUTransfer(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer, TConstArrayView<FCrossGPUTransferFence*> PostTransfer) final override;
	virtual void RHICrossGPUTransferSignal(TConstArrayView<FTransferResourceParams> Params, TConstArrayView<FCrossGPUTransferFence*> PreTransfer) final override;
	virtual void RHICrossGPUTransferWait(TConstArrayView<FCrossGPUTransferFence*> PostTransfer) final override;
#endif // WITH_MGPU

	FORCEINLINE virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBuffer, FRHIStagingBuffer* DestinationStagingBuffer, uint32 Offset, uint32 NumBytes) final override
	{
		ContextRedirect(RHICopyToStagingBuffer(SourceBuffer, DestinationStagingBuffer, Offset, NumBytes));
	}
	FORCEINLINE virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
		ContextRedirect(RHISetShaderParameters(Shader, InParametersData, InParameters, InResourceParameters, InBindlessParameters));
	}
	FORCEINLINE virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override
	{
		ContextRedirect(RHISetShaderUnbinds(Shader, InUnbinds));
	}

#if WITH_RHI_BREADCRUMBS
	FORCEINLINE virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
		// Always forward to all sub-contexts, regardless of mask
		for (uint32 GPUIndex : PhysicalGPUMask)
		{
			PhysicalContexts[GPUIndex]->RHIBeginBreadcrumbGPU(Breadcrumb);
		}
	}
	FORCEINLINE virtual void RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
		// Always forward to all sub-contexts, regardless of mask
		for (uint32 GPUIndex : PhysicalGPUMask)
		{
			PhysicalContexts[GPUIndex]->RHIEndBreadcrumbGPU(Breadcrumb);
		}
	}
#endif // WITH_RHI_BREADCRUMBS

	// IRHICommandContext interface
	FORCEINLINE virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override
	{
		ContextRedirect(RHISetMultipleViewports(Count, Data));
	}
	FORCEINLINE virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override
	{
		ContextRedirect(RHIClearUAVFloat(UnorderedAccessViewRHI, Values));
	}
	FORCEINLINE virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override
	{
		ContextRedirect(RHIClearUAVUint(UnorderedAccessViewRHI, Values));
	}
	FORCEINLINE virtual void RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo) final override
	{
		ContextRedirect(RHICopyTexture(SourceTextureRHI, DestTextureRHI, CopyInfo));
	}
	FORCEINLINE virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) final override
	{
		ContextRedirect(RHICopyBufferRegion(DestBuffer, DstOffset, SourceBuffer, SrcOffset, NumBytes));
	}
	FORCEINLINE virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override
	{
		ContextRedirect(RHIBeginRenderQuery(RenderQuery));
	}
	FORCEINLINE virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override
	{
		ContextRedirect(RHIEndRenderQuery(RenderQuery));
	}
	FORCEINLINE virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override
	{
		ContextRedirect(RHISetStreamSource(StreamIndex, VertexBuffer, Offset));
	}
	FORCEINLINE virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override
	{
		ContextRedirect(RHISetViewport(MinX, MinY, MinZ, MaxX, MaxY, MaxZ));
	}
	FORCEINLINE virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) override
	{
		ContextRedirect(RHISetStereoViewport(LeftMinX, RightMinX, LeftMinY, RightMinY, MinZ, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, MaxZ));
	}

	FORCEINLINE virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override
	{
		ContextRedirect(RHISetScissorRect(bEnable, MinX, MinY, MaxX, MaxY));
	}
	FORCEINLINE virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsPipelineState, uint32 StencilRef, bool bApplyAdditionalState) final override
	{
		ContextRedirect(RHISetGraphicsPipelineState(GraphicsPipelineState, StencilRef, bApplyAdditionalState));
	}
	FORCEINLINE virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override
	{
		ContextRedirect(RHISetStaticUniformBuffers(InUniformBuffers));
	}
	FORCEINLINE virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override
	{
		ContextRedirect(RHISetStaticUniformBuffer(Slot, Buffer));
	}
	FORCEINLINE virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
		ContextRedirect(RHISetShaderParameters(Shader, InParametersData, InParameters, InResourceParameters, InBindlessParameters));
	}
	FORCEINLINE virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override
	{
		ContextRedirect(RHISetShaderUnbinds(Shader, InUnbinds));
	}
	FORCEINLINE virtual void RHISetStencilRef(uint32 StencilRef) final override
	{
		ContextRedirect(RHISetStencilRef(StencilRef));
	}
	FORCEINLINE void RHISetBlendFactor(const FLinearColor& BlendFactor) final override
	{
		ContextRedirect(RHISetBlendFactor(BlendFactor));
	}
	FORCEINLINE void RHISetShaderRootConstants(const FUint32Vector4& Constants) final override
	{
		ContextRedirect(RHISetShaderRootConstants(Constants));
	}
	FORCEINLINE virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override
	{
		ContextRedirect(RHIDrawPrimitive(BaseVertexIndex, NumPrimitives, NumInstances));
	}
	FORCEINLINE virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ContextRedirect(RHIDrawPrimitiveIndirect(ArgumentBuffer, ArgumentOffset));
	}
	FORCEINLINE virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override
	{
		ContextRedirect(RHIDrawIndexedPrimitive(IndexBuffer, BaseVertexIndex, FirstInstance, NumVertices, StartIndex, NumPrimitives, NumInstances));
	}
	FORCEINLINE virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ContextRedirect(RHIDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentBuffer, ArgumentOffset));
	}
	FORCEINLINE virtual void RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBufferOffset, uint32 MaxDrawArguments) final override
	{
		ContextRedirect(RHIMultiDrawIndexedPrimitiveIndirect(IndexBuffer, ArgumentBuffer, ArgumentOffset, CountBuffer, CountBufferOffset, MaxDrawArguments));
	}
#if PLATFORM_SUPPORTS_MESH_SHADERS
	FORCEINLINE virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override
	{
		ContextRedirect(RHIDispatchMeshShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ));
	}
	FORCEINLINE virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ContextRedirect(RHIDispatchIndirectMeshShader(ArgumentBuffer, ArgumentOffset));
	}
#endif
	FORCEINLINE virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override
	{
		ContextRedirect(RHISetDepthBounds(MinDepth, MaxDepth));
	}
	
	FORCEINLINE virtual void RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner) final override
	{
		ContextRedirect(RHISetShadingRate(ShadingRate, Combiner));
	}

	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) final override
	{
		ContextRedirect(RHIBeginRenderPass(InInfo, InName));
	}

	virtual void RHIEndRenderPass() final override
	{
		ContextRedirect(RHIEndRenderPass());
	}

#if D3D12_RHI_RAYTRACING
	virtual void RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params) final override
	{
		ContextRedirect(RHIExecuteMultiIndirectClusterOperation(Params));
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) final override
	{
#if WITH_MGPU
		FD3D12CommandContext::UnregisterAccelerationStructuresInternalMGPU(Params, GPUMask);
#endif 

		ContextRedirect(RHIBuildAccelerationStructures(Params, ScratchBufferRange));
	}

	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) final override
	{
		ContextRedirect(RHIBuildAccelerationStructures(Params));
	}

	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		uint32 Width, uint32 Height) final override
	{
		ContextRedirect(RHIRayTraceDispatch(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, Width, Height));
	}

	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
		FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
		FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
		ContextRedirect(RHIRayTraceDispatchIndirect(RayTracingPipelineState, RayGenShader, SBT, GlobalResourceBindings, ArgumentBuffer, ArgumentOffset));
	}

	virtual void RHISetBindingsOnShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIRayTracingPipelineState* Pipeline, uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings, ERayTracingBindingType BindingType) final override
	{
		ContextRedirect(RHISetBindingsOnShaderBindingTable(SBT, Pipeline, NumBindings, Bindings, BindingType));
	}

	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) final override
	{
		ContextRedirect(RHIBindAccelerationStructureMemory(Scene, Buffer, BufferOffset));
	}

	virtual void RHICommitShaderBindingTable(FRHIShaderBindingTable* SBT, FRHIBuffer* InlineBindingDataBuffer) final override
	{
		ContextRedirect(RHICommitShaderBindingTable(SBT, InlineBindingDataBuffer));
	}

#endif // D3D12_RHI_RAYTRACING

	virtual void RHISetGPUMask(FRHIGPUMask InGPUMask) final override
	{
		GPUMask = InGPUMask;
		check(PhysicalGPUMask.ContainsAll(GPUMask));
	}

	virtual FRHIGPUMask RHIGetGPUMask() const final override
	{
		return GPUMask;
	}

	// Sets the mask of which GPUs can be supported, as opposed to the currently active
	// set. RHISetGPUMask checks that the active mask is a subset of the physical mask.
	FORCEINLINE void SetPhysicalGPUMask(FRHIGPUMask InGPUMask)
	{
		PhysicalGPUMask = InGPUMask;
	}

	FORCEINLINE void SetPhysicalContext(FD3D12CommandContext* Context)
	{
		check(Context);
		const uint32 GPUIndex = Context->GetGPUIndex();
		check(PhysicalGPUMask.Contains(GPUIndex));
		PhysicalContexts[GPUIndex] = Context;
	}

	FORCEINLINE FD3D12CommandContext* GetSingleDeviceContext(uint32 GPUIndex) final override
	{
		return PhysicalContexts[GPUIndex];
	}

	virtual void SetExecutingCommandList(FRHICommandListBase* InCmdList) final override
	{
		FD3D12CommandContextBase::SetExecutingCommandList(InCmdList);
		for (uint32 Index : PhysicalGPUMask)
		{
			PhysicalContexts[Index]->SetExecutingCommandList(InCmdList);
		}
	}

private:
	TStaticArray<FD3D12CommandContext*, MAX_NUM_GPUS> PhysicalContexts;
};

class FD3D12ContextArray : public TRHIPipelineArray<FD3D12CommandContextBase*>
{
public:
	FD3D12ContextArray(FRHIContextArray const& Contexts)
	{
		for (int32 Index = 0; Index < int32(ERHIPipeline::Num); ++Index)
		{
			(*this)[Index] = FD3D12CommandContextBase::Get(Contexts[Index]);
		}
	}

	operator FRHIContextArray() const
	{
		FRHIContextArray Result;
		for (int32 Index = 0; Index < int32(ERHIPipeline::Num); ++Index)
		{
			FD3D12CommandContextBase* Base = (*this)[Index];
			Result[Index] = Base ? &Base->GetHighestLevelContext() : nullptr;
		}
		return Result;
	}
};
