// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12CommandList.h"
#include "D3D12RHIPrivate.h"
#include "RHIValidation.h"

#if RHI_USE_RESOURCE_DEBUG_NAME
static std::atomic_int32_t GNextCommandListID{1};
#endif

void FD3D12CommandList::UpdateResidency(const FD3D12Resource* Resource)
{
#if ENABLE_RESIDENCY_MANAGEMENT
	if (Resource->NeedsDeferredResidencyUpdate())
	{
		State.DeferredResidencyUpdateSet.Add(Resource);
	}
	else
	{
		AddToResidencySet(Resource->GetResidencyHandles());
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

#if ENABLE_RESIDENCY_MANAGEMENT
FD3D12ResidencySet* FD3D12CommandList::CloseResidencySet()
{
	for (const FD3D12Resource* Resource : State.DeferredResidencyUpdateSet)
	{
		AddToResidencySet(Resource->GetResidencyHandles());
	}

	if (State.DeferredResidencyUpdateSet.Num() > 0)
	{
		D3DX12Residency::Close(ResidencySet);
	}

	return ResidencySet;
}

void FD3D12CommandList::AddToResidencySet(TConstArrayView<FD3D12ResidencyHandle*> ResidencyHandles)
{
	for (FD3D12ResidencyHandle* Handle : ResidencyHandles)
	{
		if (D3DX12Residency::IsInitialized(Handle))
		{
#if DO_CHECK
			check(Device->GetGPUMask() == Handle->GPUObject->GetGPUMask());
#endif
			D3DX12Residency::Insert(*ResidencySet, *Handle);
		}
	}
}
#endif // ENABLE_RESIDENCY_MANAGEMENT

void FD3D12ContextCommon::AddGlobalBarrier(
	ED3D12Access InD3D12AccessBefore,
	ED3D12Access InD3D12AccessAfter)
{
	Barriers->AddGlobalBarrier(*this, InD3D12AccessBefore, InD3D12AccessAfter);
}

void FD3D12ContextCommon::AddBarrier(
	const FD3D12Resource* pResource,
	ED3D12Access InD3D12AccessBefore,
	ED3D12Access InD3D12AccessAfter,
	uint32 Subresource)
{
	check(pResource);
	Barriers->AddBarrier(*this, pResource, InD3D12AccessBefore, InD3D12AccessAfter, Subresource);
	UpdateResidency(pResource);
}

void FD3D12ContextCommon::FlushResourceBarriers()
{
	Barriers->FlushIntoCommandList(GetCommandList(), GetTimestampQueries());
}

FD3D12CommandAllocator::FD3D12CommandAllocator(FD3D12Device* Device, ED3D12QueueType QueueType)
	: Device(Device)
	, QueueType(QueueType)
{
	VERIFYD3D12RESULT(Device->GetDevice()->CreateCommandAllocator(GetD3DCommandListType(QueueType), IID_PPV_ARGS(CommandAllocator.GetInitReference())));
	INC_DWORD_STAT(STAT_D3D12NumCommandAllocators);
}

FD3D12CommandAllocator::~FD3D12CommandAllocator()
{
	CommandAllocator.SafeRelease();
	DEC_DWORD_STAT(STAT_D3D12NumCommandAllocators);
}

void FD3D12CommandAllocator::Reset()
{
	VERIFYD3D12RESULT(CommandAllocator->Reset());
}

FD3D12CommandList::FD3D12CommandList(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator)
	: Device(CommandAllocator->Device)
	, QueueType(CommandAllocator->QueueType)
	, ResidencySet(D3DX12Residency::CreateResidencySet(Device->GetResidencyManager()))
	, State(CommandAllocator, TimestampAllocator, PipelineStatsAllocator)
{
	switch (QueueType)
	{
	case ED3D12QueueType::Direct:
	case ED3D12QueueType::Async:
		VERIFYD3D12RESULT(Device->CreateCommandList(
			Device->GetGPUMask().GetNative(),
			GetD3DCommandListType(QueueType),
			*CommandAllocator,
			nullptr,
			IID_PPV_ARGS(Interfaces.GraphicsCommandList.GetInitReference())
		));
		Interfaces.CommandList = Interfaces.GraphicsCommandList;

		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.CopyCommandList.GetInitReference()));

		// Optionally obtain the versioned ID3D12GraphicsCommandList[0-9]+ interfaces, we don't check the HRESULT.
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 1
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList1.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 2
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList2.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 3
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList3.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 4
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList4.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 5
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList5.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 6
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList6.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 7
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList7.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 8
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList8.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 9
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList9.GetInitReference()));
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 10
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.GraphicsCommandList10.GetInitReference()));
#endif

#if D3D12_SUPPORTS_DEBUG_COMMAND_LIST
		Interfaces.CommandList->QueryInterface(IID_PPV_ARGS(Interfaces.DebugCommandList.GetInitReference()));
#endif
		break;

	case ED3D12QueueType::Copy:
		VERIFYD3D12RESULT(Device->GetDevice()->CreateCommandList(
			Device->GetGPUMask().GetNative(),
			GetD3DCommandListType(QueueType),
			*CommandAllocator,
			nullptr,
			IID_PPV_ARGS(Interfaces.CopyCommandList.GetInitReference())
		));
		Interfaces.CommandList = Interfaces.CopyCommandList;

		break;

	default:
		checkNoEntry();
		return;
	}

	INC_DWORD_STAT(STAT_D3D12NumCommandLists);

#if NV_AFTERMATH
	Interfaces.AftermathHandle = UE::RHICore::Nvidia::Aftermath::D3D12::RegisterCommandList(Interfaces.CommandList);
#endif

#if INTEL_GPU_CRASH_DUMPS
	Interfaces.IntelCommandListHandle = UE::RHICore::Intel::GPUCrashDumps::D3D12::RegisterCommandList( Interfaces.GraphicsCommandList );
#endif

#if RHI_USE_RESOURCE_DEBUG_NAME
	const int32 ID = GNextCommandListID++;
	const FString Name = FString::Printf(TEXT("FD3D12CommandList %d (GPU %d)"), ID, Device->GetGPUIndex());
	SetD3D12ObjectName(Interfaces.CommandList, GetData(Name));
#endif

	D3DX12Residency::Open(ResidencySet);
	BeginLocalQueries();
}

FD3D12CommandList::~FD3D12CommandList()
{
	D3DX12Residency::DestroyResidencySet(Device->GetResidencyManager(), ResidencySet);

#if NV_AFTERMATH
	UE::RHICore::Nvidia::Aftermath::D3D12::UnregisterCommandList(Interfaces.AftermathHandle);
#endif

	DEC_DWORD_STAT(STAT_D3D12NumCommandLists);
}

void FD3D12CommandList::Reset(FD3D12CommandAllocator* NewCommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator)
{
	check(IsClosed());
	check(NewCommandAllocator->Device == Device && NewCommandAllocator->QueueType == QueueType);
	if (Interfaces.CopyCommandList)
	{
		VERIFYD3D12RESULT(Interfaces.CopyCommandList->Reset(*NewCommandAllocator, nullptr));
	}
	else
	{
		VERIFYD3D12RESULT(Interfaces.GraphicsCommandList->Reset(*NewCommandAllocator, nullptr));
	}
	D3DX12Residency::Open(ResidencySet);

	(&State)->~FState();
	new (&State) FState(NewCommandAllocator, TimestampAllocator, PipelineStatsAllocator);

	BeginLocalQueries();
}

void FD3D12CommandList::Close()
{
	check(IsOpen());
	EndLocalQueries();

	HRESULT hr;
	if (Interfaces.CopyCommandList)
	{
		hr = Interfaces.CopyCommandList->Close();
	}
	else
	{
		hr = Interfaces.GraphicsCommandList->Close();
	}

#if DEBUG_RESOURCE_STATES
	if (hr != S_OK)
		LogResourceBarriers(State.ResourceBarriers, Interfaces.CommandList.GetReference(), ED3D12QueueType::Direct , FString(DX12_RESOURCE_NAME_TO_LOG));
#endif

	VERIFYD3D12RESULT(hr);

	if (State.DeferredResidencyUpdateSet.Num() == 0)
	{
		D3DX12Residency::Close(ResidencySet);
	}

	State.IsClosed = true;
}

void FD3D12CommandList::BeginLocalQueries()
{
#if DO_CHECK
	check(!State.bLocalQueriesBegun);
	State.bLocalQueriesBegun = true;
#endif

	if (State.BeginTimestamp)
	{
		// CPUTimestamp is filled in at submission time in FlushProfilerEvents
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0);
		State.BeginTimestamp.Target = &Event.GPUTimestampTOP;

		EndQuery(State.BeginTimestamp);
	}

	if (State.PipelineStats)
	{
		BeginQuery(State.PipelineStats);
	}
}

void FD3D12CommandList::EndLocalQueries()
{
#if DO_CHECK
	check(!State.bLocalQueriesEnded);
	State.bLocalQueriesEnded = true;
#endif

	if (State.PipelineStats)
	{
		EndQuery(State.PipelineStats);
	}

	if (State.EndTimestamp)
	{
		auto& Event = EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
		State.EndTimestamp.Target = &Event.GPUTimestampBOP;

		EndQuery(State.EndTimestamp);
	}
}

void FD3D12CommandList::BeginQuery(FD3D12QueryLocation const& Location)
{
	check(Location);
	check(Location.Heap->QueryType == D3D12_QUERY_TYPE_OCCLUSION || Location.Heap->QueryType == D3D12_QUERY_TYPE_PIPELINE_STATISTICS);

	GraphicsCommandList()->BeginQuery(
		Location.Heap->GetD3DQueryHeap(),
		Location.Heap->QueryType,
		Location.Index
	);
}

void FD3D12CommandList::EndQuery(FD3D12QueryLocation const& Location)
{
	check(Location);
	switch (Location.Heap->QueryType)
	{
	default:
		checkNoEntry();
		break;

	case D3D12_QUERY_TYPE_PIPELINE_STATISTICS:
		GraphicsCommandList()->EndQuery(
			Location.Heap->GetD3DQueryHeap(),
			Location.Heap->QueryType,
			Location.Index
		);
		State.PipelineStatsQueries.Add(Location);
		break;

	case D3D12_QUERY_TYPE_OCCLUSION:
		GraphicsCommandList()->EndQuery(
			Location.Heap->GetD3DQueryHeap(),
			Location.Heap->QueryType,
			Location.Index
		);
		State.OcclusionQueries.Add(Location);
		break;

	case D3D12_QUERY_TYPE_TIMESTAMP:
		{
			ED3D12QueryPosition Position;
			switch (Location.Type)
			{
			default:
				checkf(false, TEXT("Query location type is not a top or bottom of pipe timestamp."));
				Position = ED3D12QueryPosition::BottomOfPipe;
				break;

			case ED3D12QueryType::ProfilerTimestampTOP:
				Position = ED3D12QueryPosition::TopOfPipe;
				break;

			case ED3D12QueryType::TimestampMicroseconds:
			case ED3D12QueryType::TimestampRaw:
			case ED3D12QueryType::ProfilerTimestampBOP:
				Position = ED3D12QueryPosition::BottomOfPipe;
				break;
			}

			WriteTimestamp(Location, Position);
			
			State.TimestampQueries.Add(Location);
		}
		break;
	}
}

#if D3D12RHI_PLATFORM_USES_TIMESTAMP_QUERIES
void FD3D12CommandList::WriteTimestamp(FD3D12QueryLocation const& Location, ED3D12QueryPosition Position)
{
	GraphicsCommandList()->EndQuery(
		Location.Heap->GetD3DQueryHeap(),
		Location.Heap->QueryType,
		Location.Index
	);
}
#endif // D3D12RHI_PLATFORM_USES_TIMESTAMP_QUERIES

FD3D12CommandList::FState::FState(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator)
	: CommandAllocator(CommandAllocator)
	, EventStream(CommandAllocator->Device->GetQueue(CommandAllocator->QueueType).GetProfilerQueue())
{
	if (TimestampAllocator)
	{
		BeginTimestamp = TimestampAllocator->Allocate(ED3D12QueryType::ProfilerTimestampTOP, nullptr);
		EndTimestamp   = TimestampAllocator->Allocate(ED3D12QueryType::ProfilerTimestampBOP, nullptr);
	}

	if (PipelineStatsAllocator)
	{
		PipelineStats = PipelineStatsAllocator->Allocate(ED3D12QueryType::PipelineStats, nullptr);
	}
}


namespace D3D12RHI
{
	void GetGfxCommandListAndQueue(FRHICommandList& RHICmdList, void*& OutGfxCmdList, void*& OutCommandQueue)
	{
		IRHICommandContext& RHICmdContext = RHICmdList.GetContext();
		FD3D12CommandContext& CmdContext = static_cast<FD3D12CommandContext&>(RHICmdContext);
		check(CmdContext.IsDefaultContext());

		OutGfxCmdList = CmdContext.GraphicsCommandList().Get();
		OutCommandQueue = CmdContext.Device->GetQueue(CmdContext.QueueType).D3DCommandQueue;
	}
}
