// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12CommandList.h: Implementation of D3D12 Command List functions
=============================================================================*/
#pragma once

#include "D3D12RHICommon.h"
#include "RHICommandList.h"
#include "D3D12NvidiaExtensions.h"
#include "D3D12Resources.h"
#include "D3D12Submission.h"
#include "D3D12Util.h"

class FD3D12ContextCommon;
class FD3D12Device;
class FD3D12DynamicRHI;
class FD3D12QueryAllocator;
class FD3D12Queue;

struct FD3D12ResidencyHandle;

enum class ED3D12QueueType;

namespace D3DX12Residency
{
	class ResidencySet;
}
typedef D3DX12Residency::ResidencySet FD3D12ResidencySet;

//
// Wraps a D3D command list allocator object.
// Allocators are obtained from the parent device, and recycled in that device's object pool.
//
class FD3D12CommandAllocator final
{
private:
	friend FD3D12Device;
	FD3D12CommandAllocator(FD3D12CommandAllocator const&) = delete;
	FD3D12CommandAllocator(FD3D12CommandAllocator&&) = delete;

	FD3D12CommandAllocator(FD3D12Device* Device, ED3D12QueueType QueueType);

public:
	~FD3D12CommandAllocator();

	FD3D12Device* const Device;
	ED3D12QueueType const QueueType;

	void Reset();

	operator ID3D12CommandAllocator*() { return CommandAllocator.GetReference(); }

private:
	TRefCountPtr<ID3D12CommandAllocator> CommandAllocator;
};

//
// Wraps a D3D command list object. Includes additional data required by the command context and submission thread.
// Command lists are obtained from the parent device, and recycled in that device's object pool.
//
class FD3D12CommandList final
{
private:
	friend FD3D12Device;
	friend FD3D12ContextCommon;
	friend FD3D12DynamicRHI;
	friend FD3D12Queue;

	FD3D12CommandList(FD3D12CommandList const&) = delete;
	FD3D12CommandList(FD3D12CommandList&&) = delete;

	FD3D12CommandList(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator);

	void BeginLocalQueries();
	void EndLocalQueries();

public:
	~FD3D12CommandList();

	void BeginQuery(FD3D12QueryLocation const& Location);
	void EndQuery  (FD3D12QueryLocation const& Location);

	void Reset(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator);
	void Close();

	bool IsOpen  () const { return !State.IsClosed; }
	bool IsClosed() const { return  State.IsClosed; }

	uint32 GetNumCommands() const { return State.NumCommands; }

	FD3D12Device*       const Device;
	ED3D12QueueType     const QueueType;

private:
	FD3D12ResidencySet* const ResidencySet;

public:
	// Indicate that a resource must be made resident before execution on GPU.
	// Either immediately adds residency handle for this resource to the residency set
	// or defers it until submission time (residency handles may not be known until then).
	void UpdateResidency(const FD3D12Resource* Resource);

#if ENABLE_RESIDENCY_MANAGEMENT
	// Immediately add residency handles to the residency set for this command list.
	void AddToResidencySet(TConstArrayView<FD3D12ResidencyHandle*> Handles);

	// Closes and returns the residency set. Used only during submission.
	FD3D12ResidencySet* CloseResidencySet();
#endif // ENABLE_RESIDENCY_MANAGEMENT

	template <typename TEventType, typename... TArgs>
	TEventType& EmplaceProfilerEvent(TArgs&&... Args)
	{
		TEventType& Data = State.EventStream.Emplace<TEventType>(Forward<TArgs>(Args)...);

		if constexpr (std::is_same_v<UE::RHI::GPUProfiler::FEvent::FBeginWork, TEventType>)
		{
			// Store BeginEvents in a separate array as the CPUTimestamp field needs updating at submit time.
			State.BeginEvents.Add(&Data);
		}

		return Data;
	}

	void FlushProfilerEvents(UE::RHI::GPUProfiler::FEventStream& Destination, uint64 CPUTimestamp)
	{
		for (UE::RHI::GPUProfiler::FEvent::FBeginWork* BeginEvent : State.BeginEvents)
		{
			BeginEvent->CPUTimestamp = CPUTimestamp;
		}

		Destination.Append(MoveTemp(State.EventStream));
	}

private:
	struct FInterfaces
	{
		TRefCountPtr<ID3D12CommandList>          CommandList;
		TRefCountPtr<ID3D12CopyCommandList>      CopyCommandList;
		TRefCountPtr<ID3D12GraphicsCommandList>  GraphicsCommandList;

#if D3D12_MAX_COMMANDLIST_INTERFACE >= 1
		TRefCountPtr<ID3D12GraphicsCommandList1> GraphicsCommandList1;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 2
		TRefCountPtr<ID3D12GraphicsCommandList2> GraphicsCommandList2;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 3
		TRefCountPtr<ID3D12GraphicsCommandList3> GraphicsCommandList3;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 4
		TRefCountPtr<ID3D12GraphicsCommandList4> GraphicsCommandList4;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 5
		TRefCountPtr<ID3D12GraphicsCommandList5> GraphicsCommandList5;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 6
		TRefCountPtr<ID3D12GraphicsCommandList6> GraphicsCommandList6;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 7
		TRefCountPtr<ID3D12GraphicsCommandList7> GraphicsCommandList7;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 8
		TRefCountPtr<ID3D12GraphicsCommandList8> GraphicsCommandList8;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 9
		TRefCountPtr<ID3D12GraphicsCommandList9> GraphicsCommandList9;
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 10
		TRefCountPtr<ID3D12GraphicsCommandList10> GraphicsCommandList10;
#endif

#if D3D12_SUPPORTS_DEBUG_COMMAND_LIST
		TRefCountPtr<ID3D12DebugCommandList>     DebugCommandList;
#endif
#if NV_AFTERMATH
		UE::RHICore::Nvidia::Aftermath::D3D12::FCommandList AftermathHandle = nullptr;
#endif
#if INTEL_GPU_CRASH_DUMPS
		uint64 IntelCommandListHandle = 0;
#endif
	} Interfaces;

public:
	//
	// Wrapper type to prevent l-value use of the returned command list interfaces.
	// A context's command list may be swapped out during recording. Users should access the command
	// list via the context itself, to ensure they always have the correct command list instance.
	//
	template <typename T>
	class TRValuePtr
	{
		friend FD3D12CommandList;

		FD3D12CommandList& CommandList;
		T* Ptr;

		TRValuePtr(FD3D12CommandList& CommandList, T* Ptr)
			: CommandList(CommandList)
			, Ptr(Ptr)
		{}

	public:
		TRValuePtr() = delete;

		TRValuePtr(TRValuePtr const&) = delete;
		TRValuePtr(TRValuePtr&&)      = delete;

		TRValuePtr& operator= (TRValuePtr const&) = delete;
		TRValuePtr& operator= (TRValuePtr&&)      = delete;

		operator bool () const&& { return !!Ptr; }
		bool operator!() const&& { return  !Ptr; }

		// These accessor functions count useful work on command lists
		T* operator ->  () && { CommandList.State.NumCommands++; return Ptr; }
		T* Get          () && { CommandList.State.NumCommands++; return Ptr; }

		T* GetNoRefCount() && { return Ptr; }
	};

private:
	template <typename FInterfaceType>
	auto BuildRValuePtr(TRefCountPtr<FInterfaceType> FInterfaces::* Member)
	{
		return TRValuePtr<FInterfaceType>(*this, Interfaces.*Member);
	}

public:
	auto BaseCommandList      () { return BuildRValuePtr(&FInterfaces::CommandList         ); }
	auto CopyCommandList      () { return BuildRValuePtr(&FInterfaces::CopyCommandList     ); }
	auto GraphicsCommandList  () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList ); }
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 1
	auto GraphicsCommandList1 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList1); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 2
	auto GraphicsCommandList2 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList2); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 3
	auto GraphicsCommandList3 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList3); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 4
	auto GraphicsCommandList4 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList4); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 5
	auto GraphicsCommandList5 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList5); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 6
	auto GraphicsCommandList6 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList6); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 7
	auto GraphicsCommandList7 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList7); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 8
	auto GraphicsCommandList8 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList8); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 9
	auto GraphicsCommandList9 () { return BuildRValuePtr(&FInterfaces::GraphicsCommandList9); }
#endif
#if D3D12_MAX_COMMANDLIST_INTERFACE >= 10
	auto GraphicsCommandList10() { return BuildRValuePtr(&FInterfaces::GraphicsCommandList10); }
#endif

#if D3D12_SUPPORTS_DEBUG_COMMAND_LIST
	auto DebugCommandList     () { return BuildRValuePtr(&FInterfaces::DebugCommandList    ); }
#endif
#if D3D12_RHI_RAYTRACING
	auto RayTracingCommandList() { return BuildRValuePtr(&FInterfaces::GraphicsCommandList4); }
#endif
#if NV_AFTERMATH
	auto AftermathHandle      () { return Interfaces.AftermathHandle; }
#endif

private:
	void WriteTimestamp(FD3D12QueryLocation const& Location, ED3D12QueryPosition Position);

	// Contents of the state struct are reset when the command list is recycled
	struct FState
	{
		FState(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator);

		// The allocator currently assigned to this command list.
		FD3D12CommandAllocator* CommandAllocator;

#if DEBUG_RESOURCE_STATES
		// Tracks all the resources barriers being issued on this command list in order
		TArray<D3D12_RESOURCE_BARRIER> ResourceBarriers;
#endif
		
		FD3D12QueryLocation BeginTimestamp;
		FD3D12QueryLocation EndTimestamp;

		FD3D12QueryLocation PipelineStats;

		TArray<FD3D12QueryLocation> TimestampQueries;
		TArray<FD3D12QueryLocation> OcclusionQueries;
		TArray<FD3D12QueryLocation> PipelineStatsQueries;

		// Resources whose residency must be updated on the submission thread, as their residency handles are not known during translation.
		// This includes reserved resources that may refer to different heaps at different points on the submission timeline.
		TSet<const FD3D12Resource*> DeferredResidencyUpdateSet;

		uint32 NumCommands = 0;

		bool IsClosed = false;

	#if DO_CHECK
		bool bLocalQueriesBegun = false;
		bool bLocalQueriesEnded = false;
	#endif

		UE::RHI::GPUProfiler::FEventStream EventStream;
		TArray<UE::RHI::GPUProfiler::FEvent::FBeginWork*, TInlineAllocator<8>> BeginEvents;
		
	} State;
};