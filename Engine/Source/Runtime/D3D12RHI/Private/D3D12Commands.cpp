// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Commands.cpp: D3D RHI commands implementation.
=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "D3D12DescriptorRange.h"
#include "D3D12ResourceCollection.h"
#include "D3D12TextureReference.h"
#include "RHIUniformBufferUtilities.h"
#include "RHICoreTransitions.h"

using namespace D3D12RHI;

inline void ValidateBoundShader(FD3D12StateCache& InStateCache, FRHIShader* InShaderRHI)
{
#if DO_CHECK
	const EShaderFrequency ShaderFrequency = InShaderRHI->GetFrequency();
	FRHIShader* CachedShader = InStateCache.GetShader(ShaderFrequency);
	ensureMsgf(CachedShader == InShaderRHI, TEXT("Parameters are being set for a %sShader which is not currently bound"), GetShaderFrequencyString(ShaderFrequency, false));
#endif
}

inline FD3D12ShaderData* GetShaderData(FRHIShader* InShaderRHI)
{
	switch (InShaderRHI->GetFrequency())
	{
	case SF_Vertex:               return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIVertexShader*>(InShaderRHI));
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:                 return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIMeshShader*>(InShaderRHI));
	case SF_Amplification:        return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIAmplificationShader*>(InShaderRHI));
#endif					          
	case SF_Pixel:                return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIPixelShader*>(InShaderRHI));
	case SF_Geometry:             return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIGeometryShader*>(InShaderRHI));
	case SF_Compute:              return FD3D12DynamicRHI::ResourceCast(static_cast<FRHIComputeShader*>(InShaderRHI));
	}
	return nullptr;
}

inline void ValidateBoundUniformBuffer(FD3D12UniformBuffer* InUniformBuffer, FRHIShader* InShaderRHI, uint32 InBufferIndex)
{
#if DO_CHECK
	auto const& LayoutHashes = InShaderRHI->GetShaderResourceTable().ResourceTableLayoutHashes;

	if (InBufferIndex < (uint32)LayoutHashes.Num())
	{
		uint32 UniformBufferHash = InUniformBuffer->GetLayout().GetHash();
		uint32 ShaderTableHash = LayoutHashes[InBufferIndex];
		ensureMsgf(ShaderTableHash == 0 || UniformBufferHash == ShaderTableHash,
			TEXT("Invalid uniform buffer %s bound on %sShader at index %d."),
			*(InUniformBuffer->GetLayout().GetDebugName()),
			GetShaderFrequencyString(InShaderRHI->GetFrequency(), false),
			InBufferIndex);
	}
#endif
}

static void BindUniformBuffer(FD3D12CommandContext& Context, FRHIShader* Shader, EShaderFrequency ShaderFrequency, uint32 BufferIndex, FD3D12UniformBuffer* InBuffer)
{
	ValidateBoundUniformBuffer(InBuffer, Shader, BufferIndex);

	Context.StateCache.SetConstantsFromUniformBuffer(ShaderFrequency, BufferIndex, InBuffer);

	Context.BoundUniformBuffers[ShaderFrequency][BufferIndex] = InBuffer;
	Context.DirtyUniformBuffers[ShaderFrequency] |= (1 << BufferIndex);
}

void FD3D12CommandContext::FlushPendingDescriptorUpdates()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Make sure the graphics command list is valid and open before trying to flush pending descriptor updates
	OpenIfNotAlready();
	GetParentDevice()->GetBindlessDescriptorManager().FlushPendingDescriptorUpdates(*this);
#endif
}

void FD3D12CommandContext::SetExplicitDescriptorCache(FD3D12ExplicitDescriptorCache& ExplicitDescriptorCache)
{
	StateCache.GetDescriptorCache()->SetExplicitDescriptorCache(ExplicitDescriptorCache);
}

void FD3D12CommandContext::UnsetExplicitDescriptorCache()
{
	StateCache.GetDescriptorCache()->UnsetExplicitDescriptorCache();
}

// Vertex state.
void FD3D12CommandContext::RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBufferRHI, uint32 Offset)
{
	FD3D12Buffer* VertexBuffer = RetrieveObject<FD3D12Buffer>(VertexBufferRHI);

	StateCache.SetStreamSource(VertexBuffer ? &VertexBuffer->ResourceLocation : nullptr, StreamIndex, Offset);
}

void FD3D12CommandContext::SetupDispatch(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	FlushPendingDescriptorUpdates();

	CommitComputeShaderConstants();
	CommitComputeResourceTables();

	StateCache.ApplyState(GetPipeline(), ED3D12PipelineType::Compute);
}

FD3D12ResourceLocation& FD3D12CommandContext::SetupIndirectArgument(FRHIBuffer* ArgumentBufferRHI, D3D12_RESOURCE_STATES ExtraStates)
{
	FD3D12ResourceLocation& ArgumentBufferLocation = RetrieveObject<FD3D12Buffer>(ArgumentBufferRHI)->ResourceLocation;

	// Must flush so the desired state is actually set.
	FlushResourceBarriers();

	UpdateResidency(ArgumentBufferLocation.GetResource());

	return ArgumentBufferLocation;
}

void FD3D12CommandContext::PostGpuEvent()
{
#if D3D12RHI_IDLE_AFTER_EVERY_GPU_EVENT
	AddGlobalBarrier(
		ED3D12Access::Common,
		ED3D12Access::Common);
#endif

	UnsetExplicitDescriptorCache();

	ConditionalSplitCommandList();
	DEBUG_EXECUTE_COMMAND_LIST(this);
}

void FD3D12CommandContext::RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	RHI_DISPATCH_CALL_INC();

	SetupDispatch(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);

	GraphicsCommandList()->Dispatch(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
	
	PostGpuEvent();
}

void FD3D12CommandContext::RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DISPATCH_CALL_INC();

	SetupDispatch(1, 1, 1);
	
	FD3D12ResourceLocation& ArgumentBufferLocation = SetupIndirectArgument(ArgumentBufferRHI, D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE);

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	ID3D12CommandSignature* CommandSignature = IsAsyncComputeContext()
		? Adapter->GetDispatchIndirectComputeCommandSignature()
		: Adapter->GetDispatchIndirectGraphicsCommandSignature();
	
	GraphicsCommandList()->ExecuteIndirect(
		CommandSignature,
		1,
		ArgumentBufferLocation.GetResource()->GetResource(),
		ArgumentBufferLocation.GetOffsetFromBaseOfResource() + ArgumentOffset,
		NULL,
		0
	);
	
	PostGpuEvent();
}

void FD3D12CommandContext::RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions)
{
	Barriers->BeginTransitions(*this, Transitions);
}

void FD3D12CommandContext::RHIEndTransitions(TArrayView<const FRHITransition*> Transitions)
{
	Barriers->EndTransitions(*this, Transitions);
}

void FD3D12CommandContext::RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers)
{
	FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));

	if (const FRHIShaderBindingLayout* Layout = InUniformBuffers.GetShaderBindingLayout())
	{
		check(InUniformBuffers.GetUniformBufferCount() == Layout->GetNumUniformBufferEntries());

		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			StaticUniformBuffers[Index] = InUniformBuffers.GetUniformBuffer(Index);
			checkf(StaticUniformBuffers[Index], TEXT("Static uniform buffer at index %d is referenced in the shader binding layout but is not provided"), Index);
		}

		ShaderBindinglayout = Layout;
	}
	else
	{
		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			FUniformBufferStaticSlot Slot = InUniformBuffers.GetSlot(Index);
			StaticUniformBuffers[Slot] = InUniformBuffers.GetUniformBuffer(Index);
		}

		ShaderBindinglayout = nullptr;
	}

}

void FD3D12CommandContext::RHISetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
{
	StaticUniformBuffers[InSlot] = InBuffer;
}

void FD3D12CommandContext::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* StagingBufferRHI, uint32 Offset, uint32 NumBytes)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12CopyToStagingBufferTime);

	const static FLazyName RHIStagingBufferName(TEXT("FRHIStagingBuffer"));
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(SourceBufferRHI->GetName(), RHIStagingBufferName, SourceBufferRHI->GetOwnerName());

	FD3D12StagingBuffer* StagingBuffer = FD3D12DynamicRHI::ResourceCast(StagingBufferRHI);
	check(StagingBuffer);
	ensureMsgf(!StagingBuffer->bIsLocked, TEXT("Attempting to Copy to a locked staging buffer. This may have undefined behavior"));

	FD3D12Buffer* VertexBuffer = RetrieveObject<FD3D12Buffer>(SourceBufferRHI);
	check(VertexBuffer);

	// Ensure our shadow buffer is large enough to hold the readback.
	if (!StagingBuffer->ResourceLocation.IsValid() || StagingBuffer->ShadowBufferSize < NumBytes)
	{
		StagingBuffer->SafeRelease();

		// Unknown aligment requirement for sub allocated read back buffer data
		uint32 AllocationAlignment = 16;
		const D3D12_RESOURCE_DESC BufferDesc = CD3DX12_RESOURCE_DESC::Buffer(NumBytes, D3D12_RESOURCE_FLAG_NONE);		
		GetParentDevice()->GetDefaultBufferAllocator().AllocDefaultResource(
			D3D12_HEAP_TYPE_READBACK,
			BufferDesc,
			BUF_None,
			ED3D12ResourceStateMode::SingleState,
			ED3D12Access::CopyDest,
			StagingBuffer->ResourceLocation,
			AllocationAlignment,
			TEXT("StagedRead"));
		check(StagingBuffer->ResourceLocation.GetSize() == NumBytes);
		StagingBuffer->ShadowBufferSize = NumBytes;
	}

	// No need to check the GPU mask as staging buffers are in CPU memory and visible to all GPUs.
	
	{
		FD3D12Resource* pSourceResource = VertexBuffer->ResourceLocation.GetResource();
		D3D12_RESOURCE_DESC const& SourceBufferDesc = pSourceResource->GetDesc();
		uint32 SourceOffset = VertexBuffer->ResourceLocation.GetOffsetFromBaseOfResource();

		FD3D12Resource* pDestResource = StagingBuffer->ResourceLocation.GetResource();
		D3D12_RESOURCE_DESC const& DestBufferDesc = pDestResource->GetDesc();
		uint32 DestOffset = StagingBuffer->ResourceLocation.GetOffsetFromBaseOfResource();

		FlushResourceBarriers();	// Must flush so the desired state is actually set.

#if D3D12_RHI_RAYTRACING
		// TODO: Buffer in D3D12_RESOURCE_STATE_RAYTRACING_ACCELERATION_STRUCTURE needs CopyRaytracingAccelerationStructure
		bool bRayTracingAccellerationStruct = ((pSourceResource->RequiresResourceStateTracking() == false) && (pSourceResource->GetDesc().Flags & D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE));
#else
		bool bRayTracingAccellerationStruct = false;
#endif

		if (bRayTracingAccellerationStruct)
		{
			UE_LOGF(LogD3D12RHI, Warning, "RHICopyToStagingBuffer cannot be used on the RayTracing Accelleration structure %ls", *SourceBufferRHI->GetName().GetPlainNameString());
		}
		else
		{
			CopyBufferRegionChecked(
				pDestResource->GetResource(), pDestResource->GetName(),
				DestOffset,
				pSourceResource->GetResource(), pSourceResource->GetName(),
				Offset + SourceOffset,
				NumBytes
			);
		}
		
		UpdateResidency(pDestResource);
		UpdateResidency(pSourceResource);

		ConditionalSplitCommandList();
	}
}

void FD3D12CommandContext::RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ)
{
	// These are the maximum viewport extents for D3D12. Exceeding them leads to badness.
	check(MinX <= (uint32)D3D12_VIEWPORT_BOUNDS_MAX);
	check(MinY <= (uint32)D3D12_VIEWPORT_BOUNDS_MAX);
	check(MaxX <= (uint32)D3D12_VIEWPORT_BOUNDS_MAX);
	check(MaxY <= (uint32)D3D12_VIEWPORT_BOUNDS_MAX);

	D3D12_VIEWPORT Viewport = { MinX, MinY, (MaxX - MinX), (MaxY - MinY), MinZ, MaxZ };
	//avoid setting a 0 extent viewport, which the debug runtime doesn't like
	if (Viewport.Width > 0 && Viewport.Height > 0)
	{
		// Setting a viewport will also set the scissor rect appropriately.
		StateCache.SetViewport(Viewport);
		RHISetScissorRect(true, FMath::RoundToZero(MinX), FMath::RoundToZero(MinY), FMath::RoundFromZero(MaxX), FMath::RoundFromZero(MaxY));
	}
}

void FD3D12CommandContext::RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
{
	// Set up both viewports
	D3D12_VIEWPORT Viewports[2] = {};

	Viewports[0].TopLeftX = FMath::FloorToInt(LeftMinX);
	Viewports[0].TopLeftY = FMath::FloorToInt(LeftMinY);
	Viewports[0].Width = FMath::CeilToInt(LeftMaxX - LeftMinX);
	Viewports[0].Height = FMath::CeilToInt(LeftMaxY - LeftMinY);
	Viewports[0].MinDepth = MinZ;
	Viewports[0].MaxDepth = MaxZ;

	Viewports[1].TopLeftX = FMath::FloorToInt(RightMinX);
	Viewports[1].TopLeftY = FMath::FloorToInt(RightMinY);
	Viewports[1].Width = FMath::CeilToInt(RightMaxX - RightMinX);
	Viewports[1].Height = FMath::CeilToInt(RightMaxY - RightMinY);
	Viewports[1].MinDepth = MinZ;
	Viewports[1].MaxDepth = MaxZ;

	D3D12_RECT ScissorRects[2] =
	{
		{ Viewports[0].TopLeftX, Viewports[0].TopLeftY, Viewports[0].TopLeftX + Viewports[0].Width, Viewports[0].TopLeftY + Viewports[0].Height },
		{ Viewports[1].TopLeftX, Viewports[1].TopLeftY, Viewports[1].TopLeftX + Viewports[1].Width, Viewports[1].TopLeftY + Viewports[1].Height }
	};

	StateCache.SetViewports(2, Viewports);
	// Set the scissor rects appropriately.
	StateCache.SetScissorRects(2, ScissorRects);
}

void FD3D12CommandContext::RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY)
{
	CD3DX12_RECT ScissorRect;

	if (bEnable)
	{
		ScissorRect.left = MinX;
		ScissorRect.top = MinY;
		ScissorRect.right = MaxX;
		ScissorRect.bottom = MaxY;
	}
	else
	{ 
		const D3D12_VIEWPORT& Viewport = StateCache.GetViewport();
		ScissorRect.left = (LONG)Viewport.TopLeftX;
		ScissorRect.top = (LONG)Viewport.TopLeftY;
		ScissorRect.right = (LONG)Viewport.TopLeftX + (LONG)Viewport.Width;
		ScissorRect.bottom = (LONG)Viewport.TopLeftY + (LONG)Viewport.Height;
	}

	// Ensure left / top are clamped to 0. Negative values leads to badness
	ScissorRect.left = FMath::Max(0, ScissorRect.left);
	ScissorRect.top = FMath::Max(0, ScissorRect.top);

	StateCache.SetScissorRect(ScissorRect);
}

static void ApplyStaticUniformBuffersOnContext(FD3D12CommandContext& Context, FRHIShader* Shader)
{
	if (Shader)
	{
		const uint32 GpuIndex = Context.GetGPUIndex();
		const EShaderFrequency ShaderFrequency = Shader->GetFrequency();

		UE::RHICore::ApplyStaticUniformBuffers(
			Shader,
			Context.GetStaticUniformBuffers(),
			[&Context, Shader, ShaderFrequency, GpuIndex](int32 BufferIndex, FRHIUniformBuffer* Buffer)
			{
				BindUniformBuffer(Context, Shader, ShaderFrequency, BufferIndex, FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(Buffer, GpuIndex));
			}
		);
	}
}

void FD3D12CommandContext::RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState)
{
	FD3D12GraphicsPipelineState* GraphicsPipelineState = FD3D12DynamicRHI::ResourceCast(GraphicsState);

	// TODO: [PSO API] Every thing inside this scope is only necessary to keep the PSO shadow in sync while we convert the high level to only use PSOs
	// Ensure the command buffers are reset to reduce the amount of data that needs to be versioned.
	for (int32 Index = 0; Index < SF_NumGraphicsFrequencies; Index++)
	{
		StageConstantBuffers[Index].Reset();
	}
	
	// @TODO : really should only discard the constants if the shader state has actually changed.
	bDiscardSharedGraphicsConstants = true;

	if (!GraphicsPipelineState->PipelineStateInitializer.bDepthBounds)
	{
		StateCache.SetDepthBounds(0.0f, 1.0f);
	}

	if (GRHISupportsPipelineVariableRateShading)
	{
		if (GraphicsPipelineState->PipelineStateInitializer.bAllowVariableRateShading)
		{
			StateCache.SetShadingRate(GraphicsPipelineState->PipelineStateInitializer.ShadingRate, VRSRB_Passthrough, VRSRB_Max);
		}
		else
		{
			// This also forces shading rate image attachment to be ignored, so no need to set it to nullptr in the state cache
			StateCache.SetShadingRate(EVRSShadingRate::VRSSR_1x1, VRSRB_Passthrough, VRSRB_Passthrough);
		}
	}

	GraphicsPipelineState->FrameCounter.Set(GetFrameFenceCounter());

	StateCache.SetGraphicsPipelineState(GraphicsPipelineState);
	StateCache.SetStencilRef(StencilRef);

	if (bApplyAdditionalState)
	{
		ApplyStaticUniformBuffersOnContext(*this, GraphicsPipelineState->GetVertexShader());
		ApplyStaticUniformBuffersOnContext(*this, GraphicsPipelineState->GetMeshShader());
		ApplyStaticUniformBuffersOnContext(*this, GraphicsPipelineState->GetAmplificationShader());
		ApplyStaticUniformBuffersOnContext(*this, GraphicsPipelineState->GetGeometryShader());
		ApplyStaticUniformBuffersOnContext(*this, GraphicsPipelineState->GetPixelShader());
	}
}

void FD3D12CommandContext::RHISetComputePipelineState(FRHIComputePipelineState* ComputeState)
{
#if D3D12_RHI_RAYTRACING
	StateCache.TransitionComputeState(ED3D12PipelineType::Compute);
#endif

	FD3D12ComputePipelineState* ComputePipelineState = FD3D12DynamicRHI::ResourceCast(ComputeState);

	StageConstantBuffers[SF_Compute].Reset();
	bDiscardSharedComputeConstants = true;

	ComputePipelineState->FrameCounter.Set(GetFrameFenceCounter());

	StateCache.SetComputePipelineState(ComputePipelineState);

	ApplyStaticUniformBuffersOnContext(*this, FD3D12DynamicRHI::ResourceCast(ComputePipelineState->GetComputeShader()));
}

void FD3D12CommandContext::SetUAVParameter(EShaderFrequency Frequency, uint32 UAVIndex, FD3D12UnorderedAccessView* UAV)
{
	ClearShaderResources(UAV, EShaderParameterTypeMask::SRVMask);
	StateCache.SetUAV(Frequency, UAVIndex, UAV);
}

void FD3D12CommandContext::SetSRVParameter(EShaderFrequency Frequency, uint32 SRVIndex, FD3D12ShaderResourceView* SRV)
{
	StateCache.SetShaderResourceView(Frequency, SRV, SRVIndex);
}

struct FD3D12ResourceBinder
{
	FD3D12CommandContext& Context;
	FD3D12ConstantBuffer& ConstantBuffer;
	const uint32 GpuIndex;
	const EShaderFrequency Frequency;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	const bool bBindlessResources;
	const bool bBindlessSamplers;
#endif

	FD3D12ResourceBinder(FD3D12CommandContext& InContext, EShaderFrequency InFrequency, const FD3D12ShaderData* ShaderData)
		: Context(InContext)
		, ConstantBuffer(InContext.StageConstantBuffers[InFrequency])
		, GpuIndex(InContext.GetGPUIndex())
		, Frequency(InFrequency)
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		, bBindlessResources(EnumHasAnyFlags(ShaderData->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources))
		, bBindlessSamplers(EnumHasAnyFlags(ShaderData->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessSamplers))
#endif
	{
	}

	void SetParameter(uint32 BufferIndex, uint32 BaseIndex, uint32 NumBytes, const uint8* NewValue)
	{
		checkSlow(BufferIndex == 0);
		ConstantBuffer.UpdateConstant(NewValue, BaseIndex, NumBytes);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetBindlessHandle(const FRHIDescriptorHandle& Handle, uint32 Offset)
	{
		if (Handle.IsValid())
		{
			const uint32 BindlessIndex = Handle.GetIndex();
			SetParameter(0, Offset, 4, reinterpret_cast<const uint8*>(&BindlessIndex));
		}
	}
#endif

	void SetUAV(FRHIUnorderedAccessView* InUnorderedAccessView, uint32 Index, bool bClearResources = false)
	{
		FD3D12UnorderedAccessView_RHI* D3D12UnorderedAccessView = FD3D12CommandContext::RetrieveObject<FD3D12UnorderedAccessView_RHI>(InUnorderedAccessView, GpuIndex);
		if (bClearResources)
		{
			Context.ClearShaderResources(D3D12UnorderedAccessView, EShaderParameterTypeMask::SRVMask);
		}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			Context.StateCache.QueueBindlessUAV(Frequency, D3D12UnorderedAccessView);
		}
		else
#endif
		{
			Context.StateCache.SetUAV(Frequency, Index, D3D12UnorderedAccessView);
		}
	}

	void SetSRV(FRHIShaderResourceView* InShaderResourceView, uint32 Index)
	{
		FD3D12ShaderResourceView_RHI* D3D12ShaderResourceView = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(InShaderResourceView, GpuIndex);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
		}
		else
#endif
		{
			Context.StateCache.SetShaderResourceView(Frequency, D3D12ShaderResourceView, Index);
		}
	}

	void SetTexture(FRHITexture* InTexture, uint32 Index)
	{
		FD3D12Texture* D3D12Texture = FD3D12CommandContext::RetrieveTexture(InTexture, GpuIndex);
		FD3D12ShaderResourceView* D3D12ShaderResourceView = D3D12Texture ? D3D12Texture->GetShaderResourceView() : nullptr;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
		}
		else
#endif
		{
			Context.StateCache.SetShaderResourceView(Frequency, D3D12ShaderResourceView, Index);
		}
	}

	void SetSampler(FRHISamplerState* Sampler, uint32 Index)
	{
		FD3D12SamplerState* D3D12SamplerState = FD3D12CommandContext::RetrieveObject<FD3D12SamplerState>(Sampler, GpuIndex);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessSamplers)
		{
			// Nothing to do, only needs constants set
		}
		else
#endif
		{
			Context.StateCache.SetSamplerState(Frequency, D3D12SamplerState, Index);
		}
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		if (bBindlessResources)
		{
			if (FD3D12ResourceCollection* D3D12ResourceCollection = FD3D12CommandContext::RetrieveObject<FD3D12ResourceCollection>(ResourceCollection, GpuIndex))
			{
				FD3D12ShaderResourceView* D3D12ShaderResourceView = D3D12ResourceCollection->GetShaderResourceView();
				Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
				Context.StateCache.QueueBindlessSRVs(Frequency, D3D12ResourceCollection->AllSrvs);

				// We have to go through each TextureReference to get the most recent version.
				for (FD3D12RHITextureReference* TextureReference : D3D12ResourceCollection->AllTextureReferences)
				{
					if (FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureReference))
					{
						Context.StateCache.QueueBindlessSRV(Frequency, Texture->GetShaderResourceView());
					}
				}
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	void SetDescriptorRange(FRHIDescriptorRange* DescriptorRange, uint32 Offset)
	{
		if (bBindlessResources || bBindlessSamplers)
		{
			if (FD3D12DescriptorRange* D3D12DescriptorRange = FD3D12CommandContext::RetrieveObject<FD3D12DescriptorRange>(DescriptorRange, GpuIndex))
			{
				FRHIDescriptorRangeShaderData ShaderData = D3D12DescriptorRange->GetShaderData();
				SetParameter(0, Offset, sizeof(ShaderData), reinterpret_cast<const uint8*>(&ShaderData));

				// Make sure the referenced resources are resident before use.
				Context.StateCache.QueueBindlessSRVs(Frequency, D3D12DescriptorRange->AllSrvs);
				Context.StateCache.QueueBindlessUAVs(Frequency, D3D12DescriptorRange->AllUavs);
			}
		}
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
};

static void SetShaderParametersOnContext(
	FD3D12CommandContext& Context
	, FRHIShader* Shader
	, EShaderFrequency ShaderFrequency
	, TArrayView<const uint8> InParametersData
	, TArrayView<const FRHIShaderParameter> InParameters
	, TArrayView<const FRHIShaderParameterResource> InResourceParameters
	, TArrayView<const FRHIShaderParameterResource> InBindlessParameters)
{
	FD3D12ResourceBinder Binder(Context, ShaderFrequency, GetShaderData(Shader));

	for (const FRHIShaderParameter& Parameter : InParameters)
	{
		Binder.SetParameter(Parameter.BufferIndex, Parameter.BaseIndex, Parameter.ByteSize, &InParametersData[Parameter.ByteOffset]);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (const FRHIShaderParameterResource& Parameter : InBindlessParameters)
	{
		if (FRHIResource* Resource = Parameter.Resource)
		{
			if (Parameter.Type == FRHIShaderParameterResource::EType::UniformBuffer)
			{
				UE::RHI::Private::SetAllUniformBufferResourcesForBindless(Binder, static_cast<FRHIUniformBuffer*>(Resource));
			}
			else if (Parameter.Type == FRHIShaderParameterResource::EType::DescriptorRange)
			{
				Binder.SetDescriptorRange(static_cast<FRHIDescriptorRange*>(Resource), Parameter.Index);
			}
			else
			{
				FRHIDescriptorHandle Handle;

				switch (Parameter.Type)
				{
				case FRHIShaderParameterResource::EType::Texture:
					Handle = static_cast<FRHITexture*>(Resource)->GetDefaultBindlessHandle();
					Binder.SetTexture(static_cast<FRHITexture*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceView:
					Handle = static_cast<FRHIShaderResourceView*>(Resource)->GetBindlessHandle();
					Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::UnorderedAccessView:
					Handle = static_cast<FRHIUnorderedAccessView*>(Resource)->GetBindlessHandle();
					Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource), Parameter.Index, true);
					break;
				case FRHIShaderParameterResource::EType::Sampler:
					Handle = static_cast<FRHISamplerState*>(Resource)->GetBindlessHandle();
					Binder.SetSampler(static_cast<FRHISamplerState*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceCollection:
					Handle = static_cast<FRHIResourceCollection*>(Resource)->GetBindlessHandle();
					Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource), Parameter.Index);
					break;
				}

				checkf(Handle.IsValid(), TEXT("D3D12 resource did not provide a valid descriptor handle. Please validate that all D3D12 types can provide this or that the resource is still valid."));
				Binder.SetBindlessHandle(Handle, Parameter.Index);
			}
		}
	}
#endif

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		if (Parameter.Type == FRHIShaderParameterResource::EType::UnorderedAccessView)
		{
			if (D3D12DoesShaderFrequencySupportUAV(ShaderFrequency))
			{
				Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index, true);
			}
			else
			{
				checkf(false, TEXT("UAV set on unsupported shader frequency: %s"), GetShaderFrequencyString(ShaderFrequency));
			}
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			Binder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			Binder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BindUniformBuffer(Context, Shader, ShaderFrequency, Parameter.Index, FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(Parameter.Resource, Binder.GpuIndex));
			break;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		case FRHIShaderParameterResource::EType::ResourceCollection:
			Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Parameter.Resource), Parameter.Index);
			break;
#endif
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

void FD3D12CommandContext::RHISetShaderParameters(FRHIGraphicsShader* Shader, TArrayView<const uint8> InParametersData, TArrayView<const FRHIShaderParameter> InParameters, TArrayView<const FRHIShaderParameterResource> InResourceParameters, TArrayView<const FRHIShaderParameterResource> InBindlessParameters)
{
	const EShaderFrequency ShaderFrequency = Shader->GetFrequency();
	if (IsValidGraphicsFrequency(ShaderFrequency))
	{
		ValidateBoundShader(StateCache, Shader);

		SetShaderParametersOnContext(
			*this
			, Shader
			, ShaderFrequency
			, InParametersData
			, InParameters
			, InResourceParameters
			, InBindlessParameters
		);
	}
	else
	{
		checkf(0, TEXT("Unsupported FRHIGraphicsShader Type '%s'!"), GetShaderFrequencyString(ShaderFrequency, false));
	}
}

void FD3D12CommandContext::RHISetShaderParameters(FRHIComputeShader* Shader, TArrayView<const uint8> InParametersData, TArrayView<const FRHIShaderParameter> InParameters, TArrayView<const FRHIShaderParameterResource> InResourceParameters, TArrayView<const FRHIShaderParameterResource> InBindlessParameters)
{
	//ValidateBoundShader(StateCache, Shader);

	SetShaderParametersOnContext(
		*this
		, Shader
		, SF_Compute
		, InParametersData
		, InParameters
		, InResourceParameters
		, InBindlessParameters
	);
}

static void SetShaderUnbindsOnContext(
	FD3D12CommandContext& Context
	, FRHIShader* Shader
	, EShaderFrequency ShaderFrequency
	, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	for (const FRHIShaderParameterUnbind& Unbind : InUnbinds)
	{
		switch (Unbind.Type)
		{
		case FRHIShaderParameterUnbind::EType::ResourceView:
			Context.StateCache.SetShaderResourceView(ShaderFrequency, nullptr, Unbind.Index);
			break;
		case FRHIShaderParameterUnbind::EType::UnorderedAccessView:
			if (D3D12DoesShaderFrequencySupportUAV(ShaderFrequency))
			{
				Context.StateCache.SetUAV(ShaderFrequency, Unbind.Index, nullptr);
			}
			else
			{
				checkf(false, TEXT("UAV set on unsupported shader frequency: %s"), GetShaderFrequencyString(ShaderFrequency));
			}
			break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

void FD3D12CommandContext::RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	const EShaderFrequency ShaderFrequency = Shader->GetFrequency();
	if (IsValidGraphicsFrequency(ShaderFrequency))
	{
		ValidateBoundShader(StateCache, Shader);

		SetShaderUnbindsOnContext(*this, Shader, ShaderFrequency, InUnbinds);
	}
	else
	{
		checkf(0, TEXT("Unsupported FRHIGraphicsShader Type '%s'!"), GetShaderFrequencyString(ShaderFrequency, false));
	}
}

void FD3D12CommandContext::RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	//ValidateBoundShader(StateCache, Shader);

	SetShaderUnbindsOnContext(*this, Shader, SF_Compute, InUnbinds);
}

void FD3D12CommandContext::RHISetStencilRef(uint32 StencilRef)
{
	StateCache.SetStencilRef(StencilRef);
}

void FD3D12CommandContext::RHISetBlendFactor(const FLinearColor& BlendFactor)
{
	StateCache.SetBlendFactor((const float*)&BlendFactor);
}

struct FRTVDesc
{
	uint32 Width;
	uint32 Height;
	DXGI_SAMPLE_DESC SampleDesc;
};

// Return an FRTVDesc structure whose
// Width and height dimensions are adjusted for the RTV's miplevel.
FRTVDesc GetRenderTargetViewDesc(FD3D12RenderTargetView* RenderTargetView)
{
	const D3D12_RENDER_TARGET_VIEW_DESC &TargetDesc = RenderTargetView->GetD3DDesc();

	FD3D12Resource* BaseResource = RenderTargetView->GetResource();
	uint32 MipIndex = 0;
	FRTVDesc ret;
	memset(&ret, 0, sizeof(ret));

	switch (TargetDesc.ViewDimension)
	{
	case D3D12_RTV_DIMENSION_TEXTURE2D:
	case D3D12_RTV_DIMENSION_TEXTURE2DMS:
	case D3D12_RTV_DIMENSION_TEXTURE2DARRAY:
	case D3D12_RTV_DIMENSION_TEXTURE2DMSARRAY:
	{
		D3D12_RESOURCE_DESC const& Desc = BaseResource->GetDesc();
		ret.Width = (uint32)Desc.Width;
		ret.Height = Desc.Height;
		ret.SampleDesc = Desc.SampleDesc;
		if (TargetDesc.ViewDimension == D3D12_RTV_DIMENSION_TEXTURE2D || TargetDesc.ViewDimension == D3D12_RTV_DIMENSION_TEXTURE2DARRAY)
		{
			// All the non-multisampled texture types have their mip-slice in the same position.
			MipIndex = TargetDesc.Texture2D.MipSlice;
		}
		break;
	}
	case D3D12_RTV_DIMENSION_TEXTURE3D:
	{
		D3D12_RESOURCE_DESC const& Desc = BaseResource->GetDesc();
		ret.Width = (uint32)Desc.Width;
		ret.Height = Desc.Height;
		ret.SampleDesc.Count = 1;
		ret.SampleDesc.Quality = 0;
		MipIndex = TargetDesc.Texture3D.MipSlice;
		break;
	}
	default:
	{
		// not expecting 1D targets.
		checkNoEntry();
	}
	}
	ret.Width >>= MipIndex;
	ret.Height >>= MipIndex;
	return ret;
}

void FD3D12CommandContext::SetRenderTargets(
	uint32 NewNumSimultaneousRenderTargets,
	const FRHIRenderTargetView* NewRenderTargetsRHI,
	const FRHIDepthRenderTargetView* NewDepthStencilTargetRHI
	)
{
	FD3D12Texture* NewDepthStencilTarget = NewDepthStencilTargetRHI ? RetrieveTexture(NewDepthStencilTargetRHI->Texture) : nullptr;

	check(NewNumSimultaneousRenderTargets <= MaxSimultaneousRenderTargets);

	// Set the appropriate depth stencil view depending on whether depth writes are enabled or not
	FD3D12DepthStencilView* DepthStencilView = nullptr;
	if (NewDepthStencilTarget)
	{
		check(NewDepthStencilTargetRHI);	// Calm down static analysis
		DepthStencilView = NewDepthStencilTarget->GetDepthStencilView(NewDepthStencilTargetRHI->GetDepthStencilAccess());

		// Unbind any shader views of the depth stencil target that are bound.
		ClearShaderResources(NewDepthStencilTarget, EShaderParameterTypeMask::SRVMask | EShaderParameterTypeMask::UAVMask);
	}

	// Gather the render target views for the new render targets.
	FD3D12RenderTargetView* NewRenderTargetViews[MaxSimultaneousRenderTargets];
	for (uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets;++RenderTargetIndex)
	{
		FD3D12RenderTargetView* RenderTargetView = NULL;
		if (RenderTargetIndex < NewNumSimultaneousRenderTargets && NewRenderTargetsRHI[RenderTargetIndex].Texture != nullptr)
		{
			int32 RTMipIndex = NewRenderTargetsRHI[RenderTargetIndex].MipIndex;
			int32 RTSliceIndex = NewRenderTargetsRHI[RenderTargetIndex].ArraySliceIndex;
			FD3D12Texture* NewRenderTarget = RetrieveTexture(NewRenderTargetsRHI[RenderTargetIndex].Texture);
			RenderTargetView = NewRenderTarget->GetRenderTargetView(RTMipIndex, RTSliceIndex);

			ensureMsgf(RenderTargetView, TEXT("Texture being set as render target has no RTV"));

			// Unbind any shader views of the render target that are bound.
			ClearShaderResources(NewRenderTarget, EShaderParameterTypeMask::SRVMask | EShaderParameterTypeMask::UAVMask);
		}

		NewRenderTargetViews[RenderTargetIndex] = RenderTargetView;
	}

	StateCache.SetRenderTargets(NewNumSimultaneousRenderTargets, NewRenderTargetViews, DepthStencilView);
	StateCache.ClearUAVs(SF_Pixel);

	// Set the viewport to the full size of render target 0.
	if (NewRenderTargetViews[0])
	{
		// check target 0 is valid
		check(0 < NewNumSimultaneousRenderTargets && NewRenderTargetsRHI[0].Texture != nullptr);
		FRTVDesc RTTDesc = GetRenderTargetViewDesc(NewRenderTargetViews[0]);
		RHISetViewport(0.0f, 0.0f, 0.0f, (float)RTTDesc.Width, (float)RTTDesc.Height, 1.0f);
	}
	else if (DepthStencilView)
	{
		FD3D12Resource* DepthTargetTexture = DepthStencilView->GetResource();
		D3D12_RESOURCE_DESC const& DTTDesc = DepthTargetTexture->GetDesc();
		RHISetViewport(0.0f, 0.0f, 0.0f, (float)DTTDesc.Width, (float)DTTDesc.Height, 1.0f);
	}
}

#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
static D3D12_SHADING_RATE_COMBINER ConvertShadingRateCombiner(EVRSRateCombiner InCombiner)
{
	switch (InCombiner)
	{
	case VRSRB_Override:
		return D3D12_SHADING_RATE_COMBINER_OVERRIDE;
	case VRSRB_Min:
		return D3D12_SHADING_RATE_COMBINER_MIN;
	case VRSRB_Max:
		return D3D12_SHADING_RATE_COMBINER_MAX;
	case VRSRB_Sum:
		return D3D12_SHADING_RATE_COMBINER_SUM;
	case VRSRB_Passthrough:
	default:
		return D3D12_SHADING_RATE_COMBINER_PASSTHROUGH;
	}
}
#endif

void FD3D12CommandContext::SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo)
{
	this->SetRenderTargets(RenderTargetsInfo.NumColorRenderTargets,
		RenderTargetsInfo.ColorRenderTarget,
		&RenderTargetsInfo.DepthStencilRenderTarget);

	FD3D12RenderTargetView* RenderTargetViews[MaxSimultaneousRenderTargets];
	FD3D12DepthStencilView* DSView = nullptr;
	uint32 NumSimultaneousRTs = 0;
	StateCache.GetRenderTargets(RenderTargetViews, &NumSimultaneousRTs, &DSView);
	FD3D12BoundRenderTargets BoundRenderTargets(RenderTargetViews, NumSimultaneousRTs, DSView);
	FD3D12DepthStencilView* DepthStencilView = BoundRenderTargets.GetDepthStencilView();

	if (RenderTargetsInfo.bClearColor || RenderTargetsInfo.bClearStencil || RenderTargetsInfo.bClearDepth)
	{
		FLinearColor ClearColors[MaxSimultaneousRenderTargets];
		bool bClearColorArray[MaxSimultaneousRenderTargets];
		float DepthClear = 0.0;
		uint32 StencilClear = 0;

		if (RenderTargetsInfo.bClearColor)
		{
			for (int32 i = 0; i < RenderTargetsInfo.NumColorRenderTargets; ++i)
			{
				if (RenderTargetsInfo.ColorRenderTarget[i].Texture != nullptr)
				{
					const FClearValueBinding& ClearValue = RenderTargetsInfo.ColorRenderTarget[i].Texture->GetClearBinding();
					checkf(ClearValue.ColorBinding == EClearBinding::EColorBound, TEXT("Texture: %s does not have a color bound for fast clears"), *RenderTargetsInfo.ColorRenderTarget[i].Texture->GetName().GetPlainNameString());
					ClearColors[i] = ClearValue.GetClearColor();
				}
				else
				{
					ClearColors[i] = FLinearColor(ForceInitToZero);
				}
				bClearColorArray[i] = RenderTargetsInfo.ColorRenderTarget[i].LoadAction == ERenderTargetLoadAction::EClear;
			}
		}
		if (RenderTargetsInfo.bClearDepth || RenderTargetsInfo.bClearStencil)
		{
			const FClearValueBinding& ClearValue = RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetClearBinding();
			checkf(ClearValue.ColorBinding == EClearBinding::EDepthStencilBound, TEXT("Texture: %s does not have a DS value bound for fast clears"), *RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetName().GetPlainNameString());
			ClearValue.GetDepthStencil(DepthClear, StencilClear);
		}

		this->RHIClearMRTImpl(RenderTargetsInfo.bClearColor ? bClearColorArray : nullptr, RenderTargetsInfo.NumColorRenderTargets, ClearColors, RenderTargetsInfo.bClearDepth, DepthClear, RenderTargetsInfo.bClearStencil, StencilClear);
	}

#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
	if (GRHISupportsAttachmentVariableRateShading)
	{
		if (RenderTargetsInfo.ShadingRateTexture != nullptr)
		{
			FD3D12Resource* Resource = RetrieveTexture(RenderTargetsInfo.ShadingRateTexture)->GetResource();
			StateCache.SetShadingRateImage(Resource);
		}
		else
		{
			StateCache.SetShadingRateImage(nullptr);
		}
	}
#endif
}

// Primitive drawing.

void FD3D12CommandContext::CommitNonComputeShaderConstants()
{
	//SCOPE_CYCLE_COUNTER(STAT_D3D12CommitGraphicsConstants);

	const FD3D12GraphicsPipelineState* const RESTRICT GraphicPSO = StateCache.GetGraphicsPipelineState();

	check(GraphicPSO);

	// Only set the constant buffer if this shader needs the global constant buffer bound
	// Otherwise we will overwrite a different constant buffer

	for (int32 Index = 0; Index < SF_NumGraphicsFrequencies; Index++)
	{
		const EShaderFrequency ShaderFrequency = static_cast<EShaderFrequency>(Index);
		if (IsValidGraphicsFrequency(ShaderFrequency) && GraphicPSO->bShaderNeedsGlobalConstantBuffer[Index])
		{
			StateCache.SetConstantBuffer(ShaderFrequency, StageConstantBuffers[Index], bDiscardSharedGraphicsConstants);
		}
	}

	bDiscardSharedGraphicsConstants = false;
}

void FD3D12CommandContext::CommitComputeShaderConstants()
{
	const FD3D12ComputePipelineState* const RESTRICT ComputePSO = StateCache.GetComputePipelineState();

	check(ComputePSO);

	if (ComputePSO->bShaderNeedsGlobalConstantBuffer)
	{
		StateCache.SetConstantBuffer(SF_Compute, StageConstantBuffers[SF_Compute], bDiscardSharedComputeConstants);
	}

	bDiscardSharedComputeConstants = false;
}

template <class ShaderType>
void FD3D12CommandContext::SetResourcesFromTables(const ShaderType* RESTRICT Shader)
{
	checkSlow(Shader);

	static constexpr EShaderFrequency Frequency = static_cast<EShaderFrequency>(ShaderType::StaticFrequency);

	FD3D12ResourceBinder Binder(*this, Frequency, Shader);
	UE::RHI::Private::SetUniformBufferResourcesFromTables(
		  Binder
		, *Shader
		, DirtyUniformBuffers[Frequency]
		, BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
		, Tracker
#endif
	);
}

void FD3D12CommandContext::CommitGraphicsResourceTables()
{
	//SCOPE_CYCLE_COUNTER(STAT_D3D12CommitResourceTables);

	const FD3D12GraphicsPipelineState* const RESTRICT GraphicPSO = StateCache.GetGraphicsPipelineState();
	check(GraphicPSO);

	if (FD3D12PixelShader* Shader = GraphicPSO->GetPixelShader())
	{
		SetResourcesFromTables(Shader);
	}

	if (FD3D12VertexShader* Shader = GraphicPSO->GetVertexShader())
	{
		SetResourcesFromTables(Shader);
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (FD3D12MeshShader* Shader = GraphicPSO->GetMeshShader())
	{
		SetResourcesFromTables(Shader);
	}
	if (FD3D12AmplificationShader* Shader = GraphicPSO->GetAmplificationShader())
	{
		SetResourcesFromTables(Shader);
	}
#endif

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	if (FD3D12GeometryShader* Shader = GraphicPSO->GetGeometryShader())
	{
		SetResourcesFromTables(Shader);
	}
#endif
}

void FD3D12CommandContext::CommitComputeResourceTables()
{
	//SCOPE_CYCLE_COUNTER(STAT_D3D12CommitResourceTables);

	const FD3D12ComputePipelineState* const RESTRICT ComputePSO = StateCache.GetComputePipelineState();

	SetResourcesFromTables(FD3D12DynamicRHI::ResourceCast(ComputePSO->GetComputeShader()));
}

void FD3D12CommandContext::RHISetShaderRootConstants(const FUint32Vector4& Constants)
{
	StateCache.SetRootConstants(Constants);
}

void FD3D12CommandContext::RHIDispatchComputeShaderBundle(
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* RecordArgBuffer,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches,
	ERHIShaderBundleMethod Method
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHIDispatchShaderBundle);
	SCOPE_CYCLE_COUNTER(STAT_D3D12DispatchShaderBundle);

	check(ShaderBundle != nullptr && Dispatches.Num() > 0);

	if (Method == ERHIShaderBundleMethod::Emulation)
	{
		TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
		UE::RHICore::DispatchShaderBundleEmulation(RHICmdList, ShaderBundle, RecordArgBuffer, SharedBindlessParameters, Dispatches);
	}
	else
	{
		check(Method == ERHIShaderBundleMethod::Native);
		DispatchWorkGraphShaderBundle(ShaderBundle, RecordArgBuffer, SharedBindlessParameters, Dispatches);
	}
}

void FD3D12CommandContext::RHIDispatchGraphicsShaderBundle(
	FRHIShaderBundle* ShaderBundle,
	FRHIBuffer* RecordArgBuffer,
	const FRHIShaderBundleGraphicsState& BundleState,
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters,
	TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches,
	ERHIShaderBundleMethod Method
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHIDispatchShaderBundle);
	SCOPE_CYCLE_COUNTER(STAT_D3D12DispatchShaderBundle);

	check(ShaderBundle != nullptr && Dispatches.Num() > 0);

	if (Method == ERHIShaderBundleMethod::Emulation)
	{
		TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
		UE::RHICore::DispatchShaderBundleEmulation(RHICmdList, ShaderBundle, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches);
	}
	else
	{
		check(Method == ERHIShaderBundleMethod::Native);
		DispatchWorkGraphShaderBundle(ShaderBundle, RecordArgBuffer, BundleState, SharedBindlessParameters, Dispatches);
	}
}

void FD3D12CommandContext::SetupDraw(FRHIBuffer* IndexBufferRHI, uint32 NumPrimitives /* = 0 */, uint32 NumVertices /* = 0 */)
{
	FlushPendingDescriptorUpdates();

	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();

	if (IndexBufferRHI)
	{
		FD3D12Buffer* IndexBuffer = RetrieveObject<FD3D12Buffer>(IndexBufferRHI);

		// determine 16bit vs 32bit indices
		const DXGI_FORMAT Format = (IndexBuffer->GetStride() == sizeof(uint16) ? DXGI_FORMAT_R16_UINT : DXGI_FORMAT_R32_UINT);

		StateCache.SetIndexBuffer(IndexBuffer->ResourceLocation, Format, 0);
	}

	StateCache.ApplyState(GetPipeline(), ED3D12PipelineType::Graphics);
}

void FD3D12CommandContext::SetupDispatchDraw(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	FlushPendingDescriptorUpdates();

	CommitGraphicsResourceTables();
	CommitNonComputeShaderConstants();

	StateCache.ApplyState(GetPipeline(), ED3D12PipelineType::Graphics);
}

void FD3D12CommandContext::RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances)
{
	uint32 VertexCount = StateCache.GetVertexCount(NumPrimitives);
	NumInstances = FMath::Max<uint32>(1, NumInstances);

	RHI_DRAW_CALL_STATS(StateCache.GetGraphicsPipelinePrimitiveType(), VertexCount, NumPrimitives, NumInstances);

	SetupDraw(nullptr, NumPrimitives * NumInstances, VertexCount * NumInstances);

	GraphicsCommandList()->DrawInstanced(VertexCount, NumInstances, BaseVertexIndex, 0);

	PostGpuEvent();
}

void FD3D12CommandContext::RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DRAW_CALL_INC();

	SetupDraw(nullptr, 0, 0);

	FD3D12ResourceLocation& ArgumentBufferLocation = SetupIndirectArgument(ArgumentBufferRHI);

	GraphicsCommandList()->ExecuteIndirect(
		GetParentDevice()->GetParentAdapter()->GetDrawIndirectCommandSignature(),
		1,
		ArgumentBufferLocation.GetResource()->GetResource(),
		ArgumentBufferLocation.GetOffsetFromBaseOfResource() + ArgumentOffset,
		NULL,
		0
	);

	PostGpuEvent();
}

void FD3D12CommandContext::RHIDrawIndexedPrimitive(FRHIBuffer* IndexBufferRHI, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
{
	FD3D12Buffer* IndexBuffer = RetrieveObject<FD3D12Buffer>(IndexBufferRHI);

	// called should make sure the input is valid, this avoid hidden bugs
	ensure(NumPrimitives > 0);
	ensure(IndexBufferRHI->GetSize() > 0);
	ensure(IndexBuffer->ResourceLocation.GetResource() != nullptr);

	if (IndexBufferRHI->GetSize() == 0 || IndexBuffer->ResourceLocation.GetResource() == nullptr)
	{
		return;
	}

	RHI_DRAW_CALL_STATS(StateCache.GetGraphicsPipelinePrimitiveType(), NumVertices, NumPrimitives, NumInstances);

	NumInstances = FMath::Max<uint32>(1, NumInstances);

	uint32 IndexCount = StateCache.GetVertexCount(NumPrimitives);

	// Verify that we are not trying to read outside the index buffer range
	// test is an optimized version of: StartIndex + IndexCount <= IndexBuffer->GetSize() / IndexBuffer->GetStride() 
	checkf((StartIndex + IndexCount) * IndexBuffer->GetStride() <= IndexBuffer->GetSize(),
		TEXT("Start %u, Count %u, Type %u, Buffer Size %u, Buffer stride %u"), StartIndex, IndexCount, StateCache.GetGraphicsPipelinePrimitiveType(), IndexBuffer->GetSize(), IndexBuffer->GetStride());

	SetupDraw(IndexBufferRHI, NumPrimitives * NumInstances, NumVertices * NumInstances);

	GraphicsCommandList()->DrawIndexedInstanced(IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance);

	PostGpuEvent();
}

void FD3D12CommandContext::RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset, FRHIBuffer* CountBufferRHI, uint32 CountBufferOffset, uint32 MaxDrawArguments)
{
	FD3D12Buffer* IndexBuffer = RetrieveObject<FD3D12Buffer>(IndexBufferRHI);

	// called should make sure the input is valid, this avoid hidden bugs
	if (!ensure(IndexBufferRHI->GetSize() > 0) || !ensure(IndexBuffer->ResourceLocation.GetResource() != nullptr))
	{
		return;
	}

	ID3D12Resource* CountBufferResource = nullptr;
	uint64 CountBufferOffsetFromResourceBase = 0;
	if (CountBufferRHI)
	{
		FD3D12Buffer* CountBuffer = RetrieveObject<FD3D12Buffer>(CountBufferRHI);
		FD3D12ResourceLocation& CounterLocation = CountBuffer->ResourceLocation;
		CountBufferResource = CounterLocation.GetResource()->GetResource();
		
		CountBufferOffsetFromResourceBase = CounterLocation.GetOffsetFromBaseOfResource() + CountBufferOffset;
		UpdateResidency(CounterLocation.GetResource());
	}

	RHI_DRAW_CALL_INC();

	SetupDraw(IndexBufferRHI, 0, 0);

	FD3D12ResourceLocation& ArgumentBufferLocation = SetupIndirectArgument(ArgumentBufferRHI);

	GraphicsCommandList()->ExecuteIndirect(
		GetParentDevice()->GetParentAdapter()->GetDrawIndexedIndirectCommandSignature(),
		MaxDrawArguments,
		ArgumentBufferLocation.GetResource()->GetResource(),
		ArgumentBufferLocation.GetOffsetFromBaseOfResource() + ArgumentOffset,
		CountBufferResource,
		CountBufferOffsetFromResourceBase
	);

	PostGpuEvent();
}

void FD3D12CommandContext::RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	// DrawIndexedPrimitiveIndirect is a special case of a more general MDI in D3D12
	RHIMultiDrawIndexedPrimitiveIndirect(IndexBufferRHI, ArgumentBufferRHI, ArgumentOffset, nullptr /*CounterBuffer*/, 0 /*CounterBufferOffset*/, 1 /*MaxDrawArguments*/);
}

#if PLATFORM_SUPPORTS_MESH_SHADERS
void FD3D12CommandContext::RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	RHI_DRAW_CALL_INC();

	SetupDispatchDraw(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);

	GraphicsCommandList6()->DispatchMesh(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);

	PostGpuEvent();
}

void FD3D12CommandContext::RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DRAW_CALL_INC();

	SetupDispatchDraw(1, 1, 1);

	FD3D12ResourceLocation& ArgumentBufferLocation = SetupIndirectArgument(ArgumentBufferRHI);

	GraphicsCommandList()->ExecuteIndirect(
		GetParentDevice()->GetParentAdapter()->GetDispatchIndirectMeshCommandSignature(),
		1,
		ArgumentBufferLocation.GetResource()->GetResource(),
		ArgumentBufferLocation.GetOffsetFromBaseOfResource() + ArgumentOffset,
		NULL,
		0
	);

	PostGpuEvent();
}
#endif // PLATFORM_SUPPORTS_MESH_SHADERS

// Raster operations.
void FD3D12CommandContext::RHIClearMRTImpl(bool* bClearColorArray, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12ClearMRT);

	const D3D12_VIEWPORT& Viewport = StateCache.GetViewport();
	const D3D12_RECT& ScissorRect = StateCache.GetScissorRect();

	if (ScissorRect.left >= ScissorRect.right || ScissorRect.top >= ScissorRect.bottom)
	{
		return;
	}

	FD3D12RenderTargetView* RenderTargetViews[MaxSimultaneousRenderTargets];
	FD3D12DepthStencilView* DSView = nullptr;
	uint32 NumSimultaneousRTs = 0;
	StateCache.GetRenderTargets(RenderTargetViews, &NumSimultaneousRTs, &DSView);
	FD3D12BoundRenderTargets BoundRenderTargets(RenderTargetViews, NumSimultaneousRTs, DSView);
	FD3D12DepthStencilView* DepthStencilView = BoundRenderTargets.GetDepthStencilView();

	// Use rounding for when the number can't be perfectly represented by a float
	const LONG Width = static_cast<LONG>(FMath::RoundToInt(Viewport.Width));
	const LONG Height = static_cast<LONG>(FMath::RoundToInt(Viewport.Height));

	// When clearing we must pay attention to the currently set scissor rect
	bool bClearCoversEntireSurface = false;
	if (ScissorRect.left <= 0 && ScissorRect.top <= 0 &&
		ScissorRect.right >= Width && ScissorRect.bottom >= Height)
	{
		bClearCoversEntireSurface = true;
	}

	// Must specify enough clear colors for all active RTs
	check(!bClearColorArray || NumClearColors >= BoundRenderTargets.GetNumActiveTargets());

	const bool bSupportsFastClear = true;
	uint32 ClearRectCount = 0;
	D3D12_RECT* pClearRects = nullptr;
	D3D12_RECT ClearRects[4];

	// Only pass a rect down to the driver if we specifically want to clear a sub-rect
	if (!bSupportsFastClear || !bClearCoversEntireSurface)
	{
		{
			ClearRects[ClearRectCount] = ScissorRect;
			ClearRectCount++;
		}

		pClearRects = ClearRects;

		static const bool bSpewPerfWarnings = false;

		if (bSpewPerfWarnings)
		{
			UE_LOGF(LogD3D12RHI, Warning, "RHIClearMRTImpl: Using non-fast clear path! This has performance implications");
			UE_LOGF(LogD3D12RHI, Warning, "       Viewport: Width %d, Height: %d", static_cast<LONG>(FMath::RoundToInt(Viewport.Width)), static_cast<LONG>(FMath::RoundToInt(Viewport.Height)));
			UE_LOGF(LogD3D12RHI, Warning, "   Scissor Rect: Width %d, Height: %d", ScissorRect.right, ScissorRect.bottom);
		}
	}

	const bool ClearRTV = bClearColorArray && BoundRenderTargets.GetNumActiveTargets() > 0;
	const bool ClearDSV = (bClearDepth || bClearStencil) && DepthStencilView;

	uint32 ClearFlags = 0;
	if (ClearDSV)
	{
		if (bClearDepth && DepthStencilView->HasDepth())
		{
			ClearFlags |= D3D12_CLEAR_FLAG_DEPTH;
		}
		else if (bClearDepth)
		{
			UE_LOGF(LogD3D12RHI, Warning, "RHIClearMRTImpl: Asking to clear a DSV that does not store depth.");
		}

		if (bClearStencil && DepthStencilView->HasStencil())
		{
			ClearFlags |= D3D12_CLEAR_FLAG_STENCIL;
		}
		else if (bClearStencil)
		{
			UE_LOGF(LogD3D12RHI, Warning, "RHIClearMRTImpl: Asking to clear a DSV that does not store stencil.");
		}
	}

	if (ClearRTV || ClearDSV)
	{
		FlushResourceBarriers();

		if (ClearRTV)
		{
			for (int32 TargetIndex = 0; TargetIndex < BoundRenderTargets.GetNumActiveTargets(); TargetIndex++)
			{
				FD3D12RenderTargetView* RTView = BoundRenderTargets.GetRenderTargetView(TargetIndex);

				if (RTView != nullptr && bClearColorArray[TargetIndex])
				{
					GraphicsCommandList()->ClearRenderTargetView(RTView->GetOfflineCpuHandle(), (float*)&ClearColorArray[TargetIndex], ClearRectCount, pClearRects);
					UpdateResidency(RTView->GetResource());
				}
			}
		}

		if (ClearDSV)
		{
			GraphicsCommandList()->ClearDepthStencilView(DepthStencilView->GetOfflineCpuHandle(), (D3D12_CLEAR_FLAGS)ClearFlags, Depth, Stencil, ClearRectCount, pClearRects);
			UpdateResidency(DepthStencilView->GetResource());
		}

		ConditionalSplitCommandList();
	}

	DEBUG_EXECUTE_COMMAND_LIST(this);
}

// Blocks the CPU until the GPU catches up and goes idle.
void FD3D12DynamicRHI::RHIBlockUntilGPUIdle()
{
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		Adapter->BlockUntilIdle();
	}
}

void FD3D12CommandContext::RHISetDepthBounds(float MinDepth, float MaxDepth)
{
	StateCache.SetDepthBounds(MinDepth, MaxDepth);
}

void FD3D12CommandContext::SetDepthBounds(float MinDepth, float MaxDepth)
{
#if PLATFORM_WINDOWS
	if (GSupportsDepthBoundsTest && GraphicsCommandList1())
	{
		// This should only be called if Depth Bounds Test is supported.
		GraphicsCommandList1()->OMSetDepthBounds(MinDepth, MaxDepth);
	}
#endif
}

void FD3D12CommandContext::RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner)
{
#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
	// Note - this will override per-material VRS opt-out, but FRHICommandSetShadingRate isn't called from anywhere
	if (GRHISupportsPipelineVariableRateShading)
	{
		StateCache.SetShadingRate(ShadingRate, Combiner, VRSRB_Max);
	}
#endif
}

void FD3D12CommandContext::SetShadingRate(EVRSShadingRate ShadingRate, FD3D12Resource* RateImageTexture, const TStaticArray<EVRSRateCombiner, ED3D12VRSCombinerStages::Num>& Combiners)
{
#if	PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
	if (GraphicsCommandList5())
	{
		if (GRHISupportsPipelineVariableRateShading)
		{
			if (ShadingRate == EVRSShadingRate::VRSSR_1x1 && RateImageTexture == nullptr)
			{
				// Make sure VRS is fully disabled when rate is 1x1 and no shading rate image is passed in
				// Otherwise we may encounter validation issues on platforms where shaders must be compiled to support VRS
				for (int32 CombinerIndex = 0; CombinerIndex < Combiners.Num(); ++CombinerIndex)
				{
					VRSCombiners[CombinerIndex] = D3D12_SHADING_RATE_COMBINER_PASSTHROUGH;
				}
			}
			else
			{
				for (int32 CombinerIndex = 0; CombinerIndex < Combiners.Num(); ++CombinerIndex)
				{
					VRSCombiners[CombinerIndex] = ConvertShadingRateCombiner(Combiners[CombinerIndex]);
				}
			}
			VRSShadingRate = static_cast<D3D12_SHADING_RATE>(ShadingRate);
			GraphicsCommandList5()->RSSetShadingRate(VRSShadingRate, VRSCombiners);

			if (GRHISupportsAttachmentVariableRateShading) // In D3D12, support for attachment VRS implies support for pipeline VRS
			{
				GraphicsCommandList5()->RSSetShadingRateImage(RateImageTexture ? RateImageTexture->GetResource() : nullptr);
			}
		}
	}
#endif
}
