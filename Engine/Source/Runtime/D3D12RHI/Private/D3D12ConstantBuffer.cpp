// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12ConstantBuffer.h"
#include "D3D12RHIPrivate.h"

DEFINE_STAT(STAT_D3D12GlobalConstantBufferUpdateTime);

// New circular buffer system for faster constant uploads.  Avoids CopyResource and speeds things up considerably
FD3D12ConstantBuffer::FD3D12ConstantBuffer(FD3D12Device* InParent, FD3D12FastConstantAllocator& InAllocator) :
	FD3D12DeviceChild(InParent),
	CurrentUpdateSize(0),
	TotalUpdateSize(0),
	bIsDirty(false),
	Allocator(InAllocator)
{
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	View = new FD3D12ConstantBufferView(InParent, nullptr);
#endif
}

FD3D12ConstantBuffer::~FD3D12ConstantBuffer()
{
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	delete View;
#endif
}

bool FD3D12ConstantBuffer::Version(FD3D12ResourceLocation& BufferOut, bool bDiscardSharedConstants)
{
	// If nothing has changed there is no need to alloc a new buffer.
	if (CurrentUpdateSize == 0)
	{
		return false;
	}

	//SCOPE_CYCLE_COUNTER(STAT_D3D12GlobalConstantBufferUpdateTime);

	if (bDiscardSharedConstants)
	{
		// If we're discarding shared constants, just use constants that have been updated since the last Commit.
		TotalUpdateSize = CurrentUpdateSize;
	}
	else
	{
		// If we're re-using shared constants, use all constants.
		TotalUpdateSize = FMath::Max(CurrentUpdateSize, TotalUpdateSize);
	}

	FD3D12ConstantBufferView* ViewToUse = nullptr;
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	ViewToUse = View;
#endif

	// Get the next constant buffer
	void* Data = Allocator.Allocate(TotalUpdateSize, BufferOut, ViewToUse);

	check(TotalUpdateSize <= (uint32)ShadowData.Num());
	FMemory::Memcpy(Data, ShadowData.GetData(), TotalUpdateSize);

	bIsDirty = false;
	return true;
}
