// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
#include "D3D12View.h"
#endif

class FD3D12FastConstantAllocator;
class FD3D12ResourceLocation;

// Chunk size for global constant buffer shadow data. Should always be a multiple of 256
#define GLOBAL_CONSTANT_BUFFER_CHUNK_SIZE 512

/**
 * A D3D constant buffer
 */
class FD3D12ConstantBuffer : public FD3D12DeviceChild
{
public:
	FD3D12ConstantBuffer(FD3D12Device* InParent, FD3D12FastConstantAllocator& InAllocator);
	virtual ~FD3D12ConstantBuffer();

	/**
	* Updates a variable in the constant buffer.
	* @param Data - The data to copy into the constant buffer
	* @param Offset - The offset in the constant buffer to place the data at
	* @param InSize - The size of the data being copied
	*/
	FORCEINLINE_DEBUGGABLE void UpdateConstant(const uint8* Data, uint16 Offset, uint16 InSize)
	{
		// Make sure we have enough memory in our shadow data
		const uint32 RequiredSize = (uint32)Offset + (uint32)InSize;
		if ((uint32)ShadowData.Num() < RequiredSize)
		{
			// Make sure we grow in chunks to prevent too many reallocs
			const uint32 SizeToGrowTo = Align(RequiredSize, GLOBAL_CONSTANT_BUFFER_CHUNK_SIZE);
			ShadowData.SetNumZeroed(SizeToGrowTo);
		}

		FMemory::Memcpy(ShadowData.GetData() + Offset, Data, InSize);
		CurrentUpdateSize = FMath::Max(RequiredSize, CurrentUpdateSize);

		bIsDirty = true;
	}

	FORCEINLINE void Reset() { CurrentUpdateSize = 0; }

	bool Version(FD3D12ResourceLocation& BufferOut, bool bDiscardSharedConstants);

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	inline D3D12_CPU_DESCRIPTOR_HANDLE GetOfflineCpuHandle() const { return View->GetOfflineCpuHandle(); }
#endif

protected:
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	FD3D12ConstantBufferView* View = nullptr;
#endif

	TArray<uint8> ShadowData;
	
	/** Size of all constants that has been updated since the last call to Commit. */
	uint32	CurrentUpdateSize;

	/**
	 * Size of all constants that has been updated since the last Discard.
	 * Includes "shared" constants that don't necessarily gets updated between every Commit.
	 */
	uint32	TotalUpdateSize;
	
	// Indicates that a constant has been updated but this one hasn't been flushed.
	bool bIsDirty;

	FD3D12FastConstantAllocator& Allocator;
};
