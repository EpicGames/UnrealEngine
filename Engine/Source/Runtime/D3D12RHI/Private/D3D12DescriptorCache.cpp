// Copyright Epic Games, Inc. All Rights Reserved.

//-----------------------------------------------------------------------------
//	Include Files
//-----------------------------------------------------------------------------
#include "D3D12DescriptorCache.h"
#include "D3D12ExplicitDescriptorCache.h"
#include "D3D12RHIPrivate.h"
#include "D3D12RayTracing.h"
#include "D3D12StateCachePrivate.h"
#include "D3D12PipelineState.h"

bool FD3D12DescriptorCache::HeapRolledOver(ERHIDescriptorHeapType InHeapType)
{
	// A heap rolled over, so set the descriptor heaps again and return if the heaps actually changed.
	return SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags::None);
}

void FD3D12DescriptorCache::HeapLoopedAround(ERHIDescriptorHeapType InHeapType)
{
	if (InHeapType == ERHIDescriptorHeapType::Sampler)
	{
		SamplerMap.Reset();
	}
}

FD3D12DescriptorCache::FD3D12DescriptorCache(FD3D12CommandContext& Context, FRHIGPUMask Node)
	: FD3D12DeviceChild(Context.Device)
	, FD3D12SingleNodeGPUObject(Node)
	, Context(Context)
	, DefaultViews(Context.Device->GetDefaultViews())
	, LocalSamplerHeap(*this, Context)
	, SubAllocatedViewHeap(*this, Context)
	, SamplerMap(271) // Prime numbers for better hashing
{
}

FD3D12DescriptorCache::~FD3D12DescriptorCache()
{
	if (LocalViewHeap)
	{
		delete LocalViewHeap;
	}
}

void FD3D12DescriptorCache::Init(uint32 InNumLocalViewDescriptors, uint32 InNumSamplerDescriptors)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorManager& BindlessDescriptorManager = GetParentDevice()->GetBindlessDescriptorManager();
	const ERHIBindlessConfiguration BindlessConfiguration = BindlessDescriptorManager.GetConfiguration();

	bCouldUseBindless = IsBindlessEnabledForAnyGraphics(BindlessConfiguration);
	bFullyBindless = IsBindlessFullyEnabled(BindlessConfiguration);

#if !D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	bUsingViewHeap = !IsFullyBindless();
#endif
#endif

	// Always Init a local sampler heap as the high level cache will always miss initialy
	// so we need something to fall back on (The view heap never rolls over so we init that one
	// lazily as a backup to save memory)
	LocalSamplerHeap.Init(IsFullyBindless() ? 0 : InNumSamplerDescriptors, ERHIDescriptorHeapType::Sampler);

	NumLocalViewDescriptors = bUsingViewHeap ? InNumLocalViewDescriptors : 0;

	CurrentViewHeap = bUsingViewHeap  ? &SubAllocatedViewHeap : nullptr;
	CurrentSamplerHeap = nullptr;
}

bool FD3D12DescriptorCache::SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags SetFlags)
{
	const ERHIPipeline Pipeline = Context.GetPipeline();

	// See if the descriptor heaps changed.
	bool bHeapChanged = EnumHasAnyFlags(SetFlags, ED3D12SetDescriptorHeapsFlags::ForceChanged);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	bool bSetBindlessHeaps = CouldUseBindless() && EnumHasAnyFlags(SetFlags, ED3D12SetDescriptorHeapsFlags::Bindless);

#if DO_CHECK
	if (bSetBindlessHeaps)
	{
		checkf(BindlessResourcesHeap, TEXT("Bindless resource heap was not set in OpenCommandList!"));
		checkf(BindlessSamplersHeap, TEXT("Bindless sampler heap was not set in OpenCommandList!"));
	}
#endif
#endif

	ID3D12DescriptorHeap* PendingViewHeap =
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		bSetBindlessHeaps ? BindlessResourcesHeap->GetHeap() :
#endif
		CurrentViewHeap->GetHeap();
	if (LastSetViewHeap != PendingViewHeap)
	{
		// The view heap changed, so dirty the descriptor tables.
		bHeapChanged = true;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (!bSetBindlessHeaps)
#endif
		{
			Context.StateCache.DirtyViewDescriptorTables();
		}

		INC_DWORD_STAT_BY(STAT_ViewHeapChanged, LastSetViewHeap == nullptr ? 0 : 1);	// Don't count the initial set on a command list.
	}

	ID3D12DescriptorHeap* PendingSamplerHeap =
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		bSetBindlessHeaps ? BindlessSamplersHeap->GetHeap() :
#endif
		CurrentSamplerHeap->GetHeap();
	if (LastSetSamplerHeap != PendingSamplerHeap)
	{
		// The sampler heap changed, so dirty the descriptor tables.
		bHeapChanged = true;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (!bSetBindlessHeaps)
#endif
		{
			Context.StateCache.DirtySamplerDescriptorTables();

			// Reset the sampler map since it will have invalid entries for the new heap.
			SamplerMap.Reset();
		}

		INC_DWORD_STAT_BY(STAT_SamplerHeapChanged, LastSetSamplerHeap == nullptr ? 0 : 1);	// Don't count the initial set on a command list.
	}

	// Set the descriptor heaps.
	if (bHeapChanged)
	{
		ID3D12DescriptorHeap* ppHeaps[] =
		{
			PendingViewHeap,
			PendingSamplerHeap
		};
		Context.GraphicsCommandList()->SetDescriptorHeaps(UE_ARRAY_COUNT(ppHeaps), ppHeaps);

		LastSetViewHeap = PendingViewHeap;
		LastSetSamplerHeap = PendingSamplerHeap;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		bLastSetHeapsBindless = bSetBindlessHeaps;
#endif
	}

	check(LastSetSamplerHeap == PendingSamplerHeap);
	check(LastSetViewHeap == PendingViewHeap);
	return bHeapChanged;
}


void FD3D12DescriptorCache::OpenCommandList()
{
	// Clear the previous heap pointers (since it's a new command list) and then set the current descriptor heaps.
	LastSetViewHeap = nullptr;
	LastSetSamplerHeap = nullptr;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Always call the Bindless Manager OpenCommandList, it will determine when it needs to do anything.
	GetParentDevice()->GetBindlessDescriptorManager().OpenCommandList(Context);

	if (!IsFullyBindless())
#endif
	{
		// The global sampler heap doesn't care about the current command list
		LocalSamplerHeap.OpenCommandList();
	}

	if (!IsFullyBindless())
	{
		SwitchToGlobalSamplerHeap();
	}

	if (CurrentViewHeap)
	{
		CurrentViewHeap->OpenCommandList();
	}
}

void FD3D12DescriptorCache::CloseCommandList()
{
	if (CurrentViewHeap)
	{
		CurrentViewHeap->CloseCommandList();
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (!IsFullyBindless())
#endif
	{
		if (bLocalSamplerHeapOpen)
		{
			LocalSamplerHeap.CloseCommandList();
			bLocalSamplerHeapOpen = false;
		}

		GetParentDevice()->GetGlobalSamplerHeap().ConsolidateUniqueSamplerTables(UniqueTables);
		UniqueTables.Reset();
	}
}

void FD3D12DescriptorCache::SetVertexBuffers(FD3D12VertexBufferCache& Cache)
{
	const uint32 Count = Cache.MaxBoundVertexBufferIndex + 1;
	if (Count == 0)
	{
		return; // No-op
	}

	Context.GraphicsCommandList()->IASetVertexBuffers(0, Count, Cache.CurrentVertexBufferViews);

	for (uint32 i = 0; i < Count; ++i)
	{
		if (Cache.CurrentVertexBufferResources[i])
		{
			Context.UpdateResidency(Cache.Resources[i]);
		}
	}
}

D3D12_GPU_DESCRIPTOR_HANDLE FD3D12DescriptorCache::BuildUAVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12UnorderedAccessViewCache& Cache, const UAVSlotMask& SlotsNeededMask, uint32 SlotsNeeded, uint32& HeapSlot)
{
	UAVSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[ShaderStage];
	check(CurrentDirtySlotMask != 0);	// All dirty slots for the current shader stage.
	check(SlotsNeededMask != 0);		// All dirty slots for the current shader stage AND used by the current shader stage.
	check(SlotsNeeded != 0);

	// Reserve heap slots
	// Note: SlotsNeeded already accounts for the UAVStartSlot. For example, if a shader has 4 UAVs starting at slot 2 then SlotsNeeded will be 6 (because the root descriptor table currently starts at slot 0).
	uint32 FirstSlotIndex = HeapSlot;
	HeapSlot += SlotsNeeded;

	D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor(CurrentViewHeap->GetCPUSlotHandle(FirstSlotIndex));
	D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor(CurrentViewHeap->GetGPUSlotHandle(FirstSlotIndex));
	D3D12_CPU_DESCRIPTOR_HANDLE SrcDescriptors[MAX_UAVS];

	const uint32 UAVStartSlot = Cache.StartSlot[ShaderStage];
	auto& UAVs = Cache.Views[ShaderStage];

	// Fill heap slots
	for (uint32 SlotIndex = 0; SlotIndex < SlotsNeeded; SlotIndex++)
	{
		if ((SlotIndex < UAVStartSlot) || (UAVs[SlotIndex] == nullptr))
		{
			SrcDescriptors[SlotIndex] = DefaultViews.NullUAV;
		}
		else
		{
			SrcDescriptors[SlotIndex] = UAVs[SlotIndex]->GetOfflineCpuHandle();

			Context.UpdateResidency(Cache.Resources[ShaderStage][SlotIndex]);
		}
	}
	FD3D12UnorderedAccessViewCache::CleanSlots(CurrentDirtySlotMask, SlotsNeeded);

	check((CurrentDirtySlotMask & SlotsNeededMask) == 0);	// Check all slots that needed to be set, were set.

	// Gather the descriptors from the offline heaps to the online heap
	GetParentDevice()->CopyDescriptors(DestDescriptor, SrcDescriptors, SlotsNeeded, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);

	return BindDescriptor;
}

void FD3D12DescriptorCache::SetUAVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12UnorderedAccessViewCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor)
{
	check(D3D12DoesShaderFrequencySupportUAV(ShaderStage));
	const uint32 RootParameterIndex = RootSignature->UAVRDTBindSlot(ShaderStage);

	check(FD3D12RootSignature::IsValidBindSlot(RootParameterIndex));

	if (ShaderStage == SF_Compute)
	{
		Context.GraphicsCommandList()->SetComputeRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}
	else
	{
		Context.GraphicsCommandList()->SetGraphicsRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}

	// We changed the descriptor table, so all resources bound to slots outside of the table's range are now dirty.
	// If a shader needs to use resources bound to these slots later, we need to set the descriptor table again to ensure those
	// descriptors are valid.
	const UAVSlotMask OutsideCurrentTableRegisterMask = ~(((UAVSlotMask)1 << SlotsNeeded) - (UAVSlotMask)1);
	Cache.Dirty(ShaderStage, OutsideCurrentTableRegisterMask);

#ifdef VERBOSE_DESCRIPTOR_HEAP_DEBUG
	FMsg::Logf(__FILE__, __LINE__, TEXT("DescriptorCache"), ELogVerbosity::Log, TEXT("SetUnorderedAccessViewTable [STAGE %d] to slots %d - %d"), (int32)ShaderStage, FirstSlotIndex, FirstSlotIndex + SlotsNeeded - 1);
#endif
}

void FD3D12DescriptorCache::SetRenderTargets(FD3D12RenderTargetView** RenderTargetViewArray, uint32 Count, FD3D12DepthStencilView* DepthStencilTarget)
{
	// NOTE: For this function, setting zero render targets might not be a no-op, since this is also used
	//       sometimes for only setting a depth stencil.

	D3D12_CPU_DESCRIPTOR_HANDLE RTVDescriptors[D3D12_SIMULTANEOUS_RENDER_TARGET_COUNT];

	// RTV & DS stace should already be in the correct state. It is transitioned in RHISetRenderTargetsAndClear coming from BeginPass because
	// then we know the correct depth & stencil read & write flags.

	// Fill heap slots
	for (uint32 i = 0; i < Count; i++)
	{
		if (RenderTargetViewArray[i] != NULL)
		{
			RTVDescriptors[i] = RenderTargetViewArray[i]->GetOfflineCpuHandle();
			Context.UpdateResidency(RenderTargetViewArray[i]->GetResource());
		}
		else
		{
			RTVDescriptors[i] = DefaultViews.NullRTV;
		}
	}

	if (DepthStencilTarget != nullptr)
	{
		const D3D12_CPU_DESCRIPTOR_HANDLE DSVDescriptor = DepthStencilTarget->GetOfflineCpuHandle();
		Context.GraphicsCommandList()->OMSetRenderTargets(Count, RTVDescriptors, 0, &DSVDescriptor);
		Context.UpdateResidency(DepthStencilTarget->GetResource());
	}
	else
	{
		CA_SUPPRESS(6001);
		Context.GraphicsCommandList()->OMSetRenderTargets(Count, RTVDescriptors, 0, nullptr);
	}
}

D3D12_GPU_DESCRIPTOR_HANDLE FD3D12DescriptorCache::BuildSamplerTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12SamplerStateCache& Cache, const SamplerSlotMask& SlotsNeededMask, uint32 SlotsNeeded, uint32& HeapSlot)
{
	check(!UsingGlobalSamplerHeap());

	SamplerSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[ShaderStage];
	check(CurrentDirtySlotMask != 0);	// All dirty slots for the current shader stage.
	check(SlotsNeededMask != 0);		// All dirty slots for the current shader stage AND used by the current shader stage.
	check(SlotsNeeded != 0);

	auto& Samplers = Cache.States[ShaderStage];

	D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = { 0 };
	bool CacheHit = false;

	// Check to see if the sampler configuration is already in the sampler heap
	FD3D12SamplerArrayDesc Desc = {};
	if (SlotsNeeded <= UE_ARRAY_COUNT(Desc.SamplerID))
	{
		Desc.Count = SlotsNeeded;

		SamplerSlotMask CacheDirtySlotMask = CurrentDirtySlotMask;	// Temp mask
		for (uint32 SlotIndex = 0; SlotIndex < SlotsNeeded; SlotIndex++)
		{
			Desc.SamplerID[SlotIndex] = Samplers[SlotIndex] ? Samplers[SlotIndex]->ID : 0;
		}
		FD3D12SamplerStateCache::CleanSlots(CacheDirtySlotMask, SlotsNeeded);

		// The hash uses all of the bits
		for (uint32 SlotIndex = SlotsNeeded; SlotIndex < UE_ARRAY_COUNT(Desc.SamplerID); SlotIndex++)
		{
			Desc.SamplerID[SlotIndex] = 0;
		}

		D3D12_GPU_DESCRIPTOR_HANDLE* FoundDescriptor = SamplerMap.Find(Desc);
		if (FoundDescriptor)
		{
			check(IsHeapSet(LocalSamplerHeap.GetHeap()));
			BindDescriptor = *FoundDescriptor;
			CacheHit = true;
			CurrentDirtySlotMask = CacheDirtySlotMask;
		}
	}

	if (!CacheHit)
	{
		// Reserve heap slots
		const uint32 FirstSlotIndex = HeapSlot;
		HeapSlot += SlotsNeeded;
		D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor = CurrentSamplerHeap->GetCPUSlotHandle(FirstSlotIndex);
		BindDescriptor = CurrentSamplerHeap->GetGPUSlotHandle(FirstSlotIndex);

		checkSlow(SlotsNeeded <= MAX_SAMPLERS);

		// Fill heap slots
		D3D12_CPU_DESCRIPTOR_HANDLE SrcDescriptors[MAX_SAMPLERS];
		for (uint32 SlotIndex = 0; SlotIndex < SlotsNeeded; SlotIndex++)
		{
			if (Samplers[SlotIndex] != nullptr)
			{
				SrcDescriptors[SlotIndex] = Samplers[SlotIndex]->OfflineDescriptor;
			}
			else
			{
				SrcDescriptors[SlotIndex] = DefaultViews.DefaultSampler->OfflineDescriptor;
			}
		}
		FD3D12SamplerStateCache::CleanSlots(CurrentDirtySlotMask, SlotsNeeded);

		GetParentDevice()->CopyDescriptors(DestDescriptor, SrcDescriptors, SlotsNeeded, FD3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER);

		// Remember the locations of the samplers in the sampler map
		if (SlotsNeeded <= UE_ARRAY_COUNT(Desc.SamplerID))
		{
			UniqueTables.Add(FD3D12UniqueSamplerTable(Desc, SrcDescriptors));

			SamplerMap.Add(Desc, BindDescriptor);
		}
	}

	return BindDescriptor;
}

void FD3D12DescriptorCache::SetSamplerTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12SamplerStateCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor)
{
	const uint32 RootParameterIndex = RootSignature->SamplerRDTBindSlot(ShaderStage);

	if (ShaderStage == SF_Compute)
	{
		Context.GraphicsCommandList()->SetComputeRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}
	else
	{
		Context.GraphicsCommandList()->SetGraphicsRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}

	// We changed the descriptor table, so all resources bound to slots outside of the table's range are now dirty.
	// If a shader needs to use resources bound to these slots later, we need to set the descriptor table again to ensure those
	// descriptors are valid.
	const SamplerSlotMask OutsideCurrentTableRegisterMask = ~(((SamplerSlotMask)1 << SlotsNeeded) - (SamplerSlotMask)1);
	Cache.Dirty(ShaderStage, OutsideCurrentTableRegisterMask);

#ifdef VERBOSE_DESCRIPTOR_HEAP_DEBUG
	FMsg::Logf(__FILE__, __LINE__, TEXT("DescriptorCache"), ELogVerbosity::Log, TEXT("SetSamplerTable [STAGE %d] to slots %d - %d"), (int32)ShaderStage, FirstSlotIndex, FirstSlotIndex + SlotsNeeded - 1);
#endif
}

D3D12_GPU_DESCRIPTOR_HANDLE FD3D12DescriptorCache::BuildSRVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ShaderResourceViewCache& Cache, const SRVSlotMask& SlotsNeededMask, uint32 SlotsNeeded, uint32& HeapSlot)
{
	SRVSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[ShaderStage];
	check(CurrentDirtySlotMask != 0);	// All dirty slots for the current shader stage.
	check(SlotsNeededMask != 0);		// All dirty slots for the current shader stage AND used by the current shader stage.
	check(SlotsNeeded != 0);

	auto& SRVs = Cache.Views[ShaderStage];

	// Reserve heap slots
	uint32 FirstSlotIndex = HeapSlot;
	HeapSlot += SlotsNeeded;

	D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor = CurrentViewHeap->GetCPUSlotHandle(FirstSlotIndex);
	D3D12_CPU_DESCRIPTOR_HANDLE SrcDescriptors[MAX_SRVS];

	for (uint32 SlotIndex = 0; SlotIndex < SlotsNeeded; SlotIndex++)
	{
		if (FD3D12ShaderResourceView* SRV = SRVs[SlotIndex])
		{
			SrcDescriptors[SlotIndex] = SRV->GetOfflineCpuHandle();

			Context.UpdateResidency(Cache.Resources[ShaderStage][SlotIndex]);

#if RHI_RAYTRACING
			FD3D12RayTracingScene* RayTracingScene = SRV->GetRayTracingScene();
			if (RayTracingScene)
			{
				RayTracingScene->UpdateResidency(Context, true);
			}
#endif
		}
		else
		{
			SrcDescriptors[SlotIndex] = DefaultViews.NullSRV;
		}
		check(SrcDescriptors[SlotIndex].ptr != 0);
	}
	FD3D12ShaderResourceViewCache::CleanSlots(CurrentDirtySlotMask, SlotsNeeded);

	GetParentDevice()->CopyDescriptors(DestDescriptor, SrcDescriptors, SlotsNeeded, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);

	check((CurrentDirtySlotMask & SlotsNeededMask) == 0);	// Check all slots that needed to be set, were set.

	const D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = CurrentViewHeap->GetGPUSlotHandle(FirstSlotIndex);

	return BindDescriptor;
}

void FD3D12DescriptorCache::SetSRVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ShaderResourceViewCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor)
{
	const uint32 RootParameterIndex = RootSignature->SRVRDTBindSlot(ShaderStage);

	if (ShaderStage == SF_Compute)
	{
		Context.GraphicsCommandList()->SetComputeRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}
	else
	{
		Context.GraphicsCommandList()->SetGraphicsRootDescriptorTable(RootParameterIndex, BindDescriptor);
	}

	// We changed the descriptor table, so all resources bound to slots outside of the table's range are now dirty.
	// If a shader needs to use resources bound to these slots later, we need to set the descriptor table again to ensure those
	// descriptors are valid.
	const SRVSlotMask OutsideCurrentTableRegisterMask = ~(((SRVSlotMask)1 << SlotsNeeded) - (SRVSlotMask)1);
	Cache.Dirty(ShaderStage, OutsideCurrentTableRegisterMask);

#ifdef VERBOSE_DESCRIPTOR_HEAP_DEBUG
	FMsg::Logf(__FILE__, __LINE__, TEXT("DescriptorCache"), ELogVerbosity::Log, TEXT("SetShaderResourceViewTable [STAGE %d] to slots %d - %d"), (int32)ShaderStage, FirstSlotIndex, FirstSlotIndex + SlotsNeeded - 1);
#endif
}

void FD3D12DescriptorCache::PrepareBindlessViews(EShaderFrequency ShaderStage, TConstArrayView<FD3D12ShaderResourceView*> SRVs, TConstArrayView<FD3D12UnorderedAccessView*> UAVs)
{
	for (FD3D12ShaderResourceView* SRV : SRVs)
	{
		if (ensure(SRV))
		{
			Context.UpdateResidency(SRV->GetResource());

#if RHI_RAYTRACING
			FD3D12RayTracingScene* RayTracingScene = SRV->GetRayTracingScene();
			if (RayTracingScene)
			{
				RayTracingScene->UpdateResidency(Context, true);
			}
#endif
		}
	}

	for (FD3D12UnorderedAccessView* UAV : UAVs)
	{
		if (ensure(UAV))
		{
			Context.UpdateResidency(UAV->GetResource());
		}
	}
}

void FD3D12DescriptorCache::SetConstantBufferViews(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ConstantBufferCache& Cache, CBVSlotMask SlotsNeededMask, uint32 SlotsNeeded, uint32& HeapSlot)
{
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	CBVSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[ShaderStage];
	check(CurrentDirtySlotMask != 0);	// All dirty slots for the current shader stage.
	check(SlotsNeededMask != 0);		// All dirty slots for the current shader stage AND used by the current shader stage.

	ID3D12Device* Device = GetParentDevice()->GetDevice();

	// Process root CBV
	const CBVSlotMask RDCBVSlotsNeededMask = GRootCBVSlotMask & SlotsNeededMask;
	check(RDCBVSlotsNeededMask); // Check this wasn't a wasted call.

	// Now desc table with CBV
	auto& CBVHandles = Cache.CBHandles[ShaderStage];

	// Reserve heap slots
	uint32 FirstSlotIndex = HeapSlot;
	check(SlotsNeeded != 0);
	HeapSlot += SlotsNeeded;

	uint32 DestDescriptorSlot = FirstSlotIndex;

	for (uint32 SlotIndex = 0; SlotIndex < SlotsNeeded; SlotIndex++)
	{
		const D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor = CurrentViewHeap->GetCPUSlotHandle(DestDescriptorSlot);

		if (CBVHandles[SlotIndex].ptr != 0)
		{
			Device->CopyDescriptorsSimple(1, DestDescriptor, CBVHandles[SlotIndex], D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);

			// Update residency.
			Context.UpdateResidency(Cache.Resources[ShaderStage][SlotIndex]);
		}
		else
		{
			Device->CopyDescriptorsSimple(1, DestDescriptor, DefaultViews.NullCBV, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);
		}

		DestDescriptorSlot++;

		// Clear the dirty bit.
		FD3D12ConstantBufferCache::CleanSlot(CurrentDirtySlotMask, SlotIndex);
	}

	check((CurrentDirtySlotMask & SlotsNeededMask) == 0);	// Check all slots that needed to be set, were set.

	const D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = CurrentViewHeap->GetGPUSlotHandle(FirstSlotIndex);

	if (ShaderStage == SF_Compute)
	{
		const uint32 RDTIndex = RootSignature->CBVRDTBindSlot(ShaderStage);
		ensure(RDTIndex != 255);

		if (RDTIndex < 64)
		{
			Context.GraphicsCommandList()->SetComputeRootDescriptorTable(RDTIndex, BindDescriptor);
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "SetComputeRootDescriptorTable call exceed max 64 slots: %d ", RDTIndex);
		}
	}
	else
	{
		const uint32 RDTIndex = RootSignature->CBVRDTBindSlot(ShaderStage);
		ensure(RDTIndex != 255);

		if (RDTIndex < 64)
		{
			Context.GraphicsCommandList()->SetGraphicsRootDescriptorTable(RDTIndex, BindDescriptor);
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "SetGraphicsRootDescriptorTable call exceed max 64 slots: %d ", RDTIndex);
		}
	}

	// We changed the descriptor table, so all resources bound to slots outside of the table's range are now dirty.
	// If a shader needs to use resources bound to these slots later, we need to set the descriptor table again to ensure those
	// descriptors are valid.
	const CBVSlotMask OutsideCurrentTableRegisterMask = ~(((CBVSlotMask)1 << SlotsNeeded) - (CBVSlotMask)1);
	Cache.Dirty(ShaderStage, OutsideCurrentTableRegisterMask);

#ifdef VERBOSE_DESCRIPTOR_HEAP_DEBUG
	FMsg::Logf(__FILE__, __LINE__, TEXT("DescriptorCache"), ELogVerbosity::Log, TEXT("SetShaderResourceViewTable [STAGE %d] to slots %d - %d"), (int32)ShaderStage, FirstSlotIndex, FirstSlotIndex + SlotsNeeded - 1);
#endif

#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
}

static FString GetPipelineShaderHashList(FD3D12GraphicsPipelineState* GraphicsPSO)
{
	FString ShaderHashList;

	if (GraphicsPSO == nullptr)
	{
		return ShaderHashList;
	}

	uint64 CombinedHash = 0;

	const auto AddShaderHash = [&CombinedHash, &ShaderHashList](const FRHIShader* Shader)
		{
			FShaderHash ShaderHash;
			if (Shader)
			{
				ShaderHash = Shader->GetHash();
				ShaderHashList.Appendf(TEXT("%s: %s, "), GetShaderFrequencyString(Shader->GetFrequency(), false), *ShaderHash.ToString());
			}
			CombinedHash += ShaderHash.Hash;
		};

	AddShaderHash(GraphicsPSO->GetVertexShader());
	AddShaderHash(GraphicsPSO->GetMeshShader());
	AddShaderHash(GraphicsPSO->GetAmplificationShader());
	AddShaderHash(GraphicsPSO->GetPixelShader());
	AddShaderHash(GraphicsPSO->GetGeometryShader());

	FShaderHash PipelineHash;
	PipelineHash.Hash = CombinedHash;

	ShaderHashList.Appendf(TEXT("Pipeline: %s"), *PipelineHash.ToString());

	return ShaderHashList;
}

void FD3D12DescriptorCache::SetRootConstantBuffers(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ConstantBufferCache& Cache, CBVSlotMask SlotsNeededMask, FD3D12StateCache* StateCache)
{
	CBVSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[ShaderStage];
	check(CurrentDirtySlotMask != 0);	// All dirty slots for the current shader stage.
	check(SlotsNeededMask != 0);		// All dirty slots for the current shader stage AND used by the current shader stage.

	// Process root CBV
	const CBVSlotMask RDCBVSlotsNeededMask = GRootCBVSlotMask & SlotsNeededMask;
	check(RDCBVSlotsNeededMask); // Check this wasn't a wasted call.

	// Set root descriptors.
	// At least one needed root descriptor is dirty.
	const uint32 BaseIndex = RootSignature->CBVRDBaseBindSlot(ShaderStage);
	ensure(BaseIndex != 255);
	const uint32 RDCBVsNeeded = FMath::FloorLog2(RDCBVSlotsNeededMask) + 1;	// Get the index of the most significant bit that's set.
	check(RDCBVsNeeded <= MAX_ROOT_CBVS);
	for (uint32 SlotIndex = 0; SlotIndex < RDCBVsNeeded; SlotIndex++)
	{
		// Only set the root descriptor if it's dirty and we need to set it (it can be used by the shader).
		if (!FD3D12ConstantBufferCache::IsSlotDirty(RDCBVSlotsNeededMask, SlotIndex))
		{
			continue;
		}

		const D3D12_GPU_VIRTUAL_ADDRESS CurrentGPUVirtualAddress = Cache.CurrentGPUVirtualAddress[ShaderStage][SlotIndex];

		// If the slot is not set in the cache, we need to check if the shader really uses the constant buffer. If the expected layout hash is 0, it means that the
		// constant buffer is unused, so it's fine if it's not bound. If the hash is non-zero, the shader expects a buffer, so this is a fatal error.
		if (CurrentGPUVirtualAddress == 0)
		{
			FRHIShader* Shader;

			if (ShaderStage == SF_Compute)
			{
				FD3D12ComputePipelineState* ComputePSO = StateCache->GetComputePipelineState();
				Shader = ComputePSO ? ComputePSO->GetComputeShader() : nullptr;
			}
			else
			{
				FD3D12GraphicsPipelineState* GraphicsPSO = StateCache->GetGraphicsPipelineState();
				Shader = GraphicsPSO ? GraphicsPSO->GetShader(ShaderStage) : nullptr;
			}

			// The shader should never be null if we got here, but check anyway because we don't want to crash while crashing.
			if (Shader == nullptr)
			{
				UE_LOGF(LogD3D12RHI, Fatal, "Missing uniform buffer at slot %u, stage %ls and no shader is bound.", SlotIndex, GetShaderFrequencyString(ShaderStage));
				// The above call terminates the process, but static analysis tools don't know that so let's make this easy for them.
				return;
			}

			const TArray<uint32>& LayoutHashes = Shader->GetShaderResourceTable().ResourceTableLayoutHashes;
			uint32 ExpectedLayoutHash = LayoutHashes.IsValidIndex(SlotIndex) ? LayoutHashes[SlotIndex] : 0;
			if (ExpectedLayoutHash != 0)
			{
				FString ShaderHashList;

				if (ShaderStage == SF_Compute)
				{
					ShaderHashList.Appendf(TEXT("Compute: %s"), *Shader->GetHash().ToString());
				}
				else
				{
					ShaderHashList = GetPipelineShaderHashList(StateCache->GetGraphicsPipelineState());
				}

				UE_LOGF(LogD3D12RHI, Fatal, "Missing uniform buffer at slot %u, stage %ls. Please check the high level drawing code. Hashes: %ls.", SlotIndex, GetShaderFrequencyString(ShaderStage), *ShaderHashList);
				return;
			}
		}

		if ((BaseIndex + SlotIndex) < 64)
		{
			if (ShaderStage == SF_Compute)
			{
				Context.GraphicsCommandList()->SetComputeRootConstantBufferView(BaseIndex + SlotIndex, CurrentGPUVirtualAddress);
			}
			else
			{
				Context.GraphicsCommandList()->SetGraphicsRootConstantBufferView(BaseIndex + SlotIndex, CurrentGPUVirtualAddress);
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "%ls call exceed max 64 slots: %d ", (ShaderStage == SF_Compute) ? TEXT("SetComputeRootConstantBufferView") : TEXT("SetGraphicsRootConstantBufferView"), BaseIndex + SlotIndex);
		}

		// Update residency is the slot is used.
		if (CurrentGPUVirtualAddress != 0)
		{
			Context.UpdateResidency(Cache.Resources[ShaderStage][SlotIndex]);
		}

		// Clear the dirty bit.
		FD3D12ConstantBufferCache::CleanSlot(CurrentDirtySlotMask, SlotIndex);
	}
	check((CurrentDirtySlotMask & RDCBVSlotsNeededMask) == 0);	// Check all slots that needed to be set, were set.

	static_assert(GDescriptorTableCBVSlotMask == 0, "FD3D12DescriptorCache::SetConstantBuffers needs to be updated to handle descriptor tables.");	// Check that all CBVs slots are controlled by root descriptors.
}

bool FD3D12DescriptorCache::SwitchToContextLocalViewHeap()
{
	check(!IsFullyBindless());

	if (LocalViewHeap == nullptr)
	{
		UE_LOGF(LogD3D12RHI, Log, "This should only happen in the Editor where it doesn't matter as much. If it happens in game you should increase the device global heap size!");
		
		// Allocate the heap lazily
		LocalViewHeap = new FD3D12LocalOnlineHeap(*this, Context);
		if (LocalViewHeap)
		{
			check(NumLocalViewDescriptors);
			LocalViewHeap->Init(NumLocalViewDescriptors, ERHIDescriptorHeapType::Standard);
		}
		else
		{
			check(false);
			return false;
		}
	}

	CurrentViewHeap->CloseCommandList();
	CurrentViewHeap = LocalViewHeap;
	CurrentViewHeap->OpenCommandList();

	const bool bDescriptorHeapsChanged = SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags::None);
	check(IsHeapSet(LocalViewHeap->GetHeap()));

	return bDescriptorHeapsChanged;
}

bool FD3D12DescriptorCache::SwitchToContextLocalSamplerHeap()
{
	check(!IsFullyBindless());

	LocalSamplerHeap.OpenCommandList();
	bLocalSamplerHeapOpen = true;

	CurrentSamplerHeap = &LocalSamplerHeap;

	bool bDescriptorHeapsChanged = SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags::None);
	check(IsHeapSet(LocalSamplerHeap.GetHeap()));

	return bDescriptorHeapsChanged;
}

void FD3D12DescriptorCache::SwitchToGlobalSamplerHeap()
{
	check(!IsFullyBindless());
	check(!bLocalSamplerHeapOpen);

	FD3D12GlobalOnlineSamplerHeap& GlobalSamplerHeap = GetParentDevice()->GetGlobalSamplerHeap();
	LocalSamplerSet = GlobalSamplerHeap.GetUniqueDescriptorTables();
	CurrentSamplerHeap = &GlobalSamplerHeap;
}

void FD3D12DescriptorCache::SetExplicitDescriptorCache(FD3D12ExplicitDescriptorCache& ExplicitDescriptorCache)
{
	ID3D12DescriptorHeap* ViewHeapToSet = nullptr;
	ID3D12DescriptorHeap* SamplerHeapToSet = nullptr;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// We have three scenarios:
	//   Bindless on globally: BindlessHeaps and LastSetXXX will match
	//   Bindless RT Only: BindlessHeaps will override LastSetXXX
	//   No Bindless:  BindlessHeaps will be null, ExplicitDescriptorCache heaps will override LastSetXXX

	const FD3D12DescriptorHeapPair BindlessHeaps = GetParentDevice()->GetBindlessDescriptorManager().GetExplicitHeapsForContext(Context, ExplicitDescriptorCache.BindlessConfiguration);

	ViewHeapToSet = BindlessHeaps.ResourceHeap ? BindlessHeaps.ResourceHeap->GetHeap() : nullptr;
	SamplerHeapToSet = BindlessHeaps.SamplerHeap ? BindlessHeaps.SamplerHeap->GetHeap() : nullptr;
#endif

	const bool bViewHeapIsBindless = (ViewHeapToSet != nullptr);

	if (!ViewHeapToSet)
	{
		check(ExplicitDescriptorCache.ViewHeap.GetParentDevice() == GetParentDevice());
		ViewHeapToSet = ExplicitDescriptorCache.ViewHeap.D3D12Heap;
	}

	if (!SamplerHeapToSet)
	{
		check(ExplicitDescriptorCache.SamplerHeap.GetParentDevice() == GetParentDevice());
		SamplerHeapToSet = ExplicitDescriptorCache.SamplerHeap.D3D12Heap;
	}

	if (ViewHeapToSet != LastSetViewHeap || SamplerHeapToSet != LastSetSamplerHeap)
	{
		LastSetViewHeap = ViewHeapToSet;
		LastSetSamplerHeap = SamplerHeapToSet;

		ID3D12DescriptorHeap* ppHeaps[] = { ViewHeapToSet, SamplerHeapToSet };
		Context.GraphicsCommandList()->SetDescriptorHeaps(UE_ARRAY_COUNT(ppHeaps), ppHeaps);

		bUsingExplicitCacheHeaps = true;
		bExplicitViewHeapIsBindless = bViewHeapIsBindless;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		bLastSetHeapBeforeExplicitIsBindless = bLastSetHeapsBindless;
		bLastSetHeapsBindless = bViewHeapIsBindless;
#endif
	}
}

void FD3D12DescriptorCache::UnsetExplicitDescriptorCache()
{
	if (bUsingExplicitCacheHeaps)
	{
		ED3D12SetDescriptorHeapsFlags Flags = ED3D12SetDescriptorHeapsFlags::None;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bLastSetHeapBeforeExplicitIsBindless)
		{
			EnumAddFlags(Flags, ED3D12SetDescriptorHeapsFlags::Bindless);
		}
#endif
		SetDescriptorHeaps(Flags);

		bUsingExplicitCacheHeaps = false;
		bExplicitViewHeapIsBindless = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12OnlineHeap
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
Initialization constructor
**/
FD3D12OnlineHeap::FD3D12OnlineHeap(FD3D12Device* Device, bool CanLoopAround)
	: FD3D12DeviceChild(Device)
	, bCanLoopAround(CanLoopAround)
{
}

FD3D12OnlineHeap::~FD3D12OnlineHeap() = default;

void FD3D12OnlineHeap::CleanupResources()
{
	Heap.SafeRelease();
}

/**
Check if requested number of slots still fit the heap
**/
bool FD3D12OnlineHeap::CanReserveSlots(uint32 NumSlots)
{
	const uint32 HeapSize = GetTotalSize();

	// Sanity checks
	if (NumSlots == 0)
	{
		return true;
	}

	if (NumSlots > HeapSize)
	{
		return false;
	}

	uint32 FirstRequestedSlot = NextSlotIndex;
	uint32 SlotAfterReservation = NextSlotIndex + NumSlots;

	// TEMP: Disable wrap around by not allowing it to reserve slots if the heap is full.
	if (SlotAfterReservation > HeapSize)
	{
		return false;
	}

	return true;

	// TEMP: Uncomment this code once the heap wrap around is fixed.
	//if (SlotAfterReservation <= HeapSize)
	//{
	//	return true;
	//}

	//// Try looping around to prevent rollovers
	//SlotAfterReservation = NumSlots;

	//if (SlotAfterReservation <= FirstUsedSlot)
	//{
	//	return true;
	//}

	//return false;
}


/**
Reserve requested amount of descriptor slots - should fit, user has to check with CanReserveSlots first
**/
uint32 FD3D12OnlineHeap::ReserveSlots(uint32 NumSlotsRequested)
{
	const ERHIDescriptorHeapType HeapType = Heap->GetType();

#ifdef VERBOSE_DESCRIPTOR_HEAP_DEBUG
	FMsg::Logf(__FILE__, __LINE__, TEXT("DescriptorCache"), ELogVerbosity::Log, TEXT("Requesting reservation [TYPE %s] with %d slots, required fence is %d"),
		ToString(HeapType), NumSlotsRequested, RequiredFenceForCurrentCL);
#endif

	const uint32 HeapSize = GetTotalSize();

	// Sanity checks
	check(NumSlotsRequested <= HeapSize);

	// CanReserveSlots should have been called first
	check(CanReserveSlots(NumSlotsRequested));

	// Decide which slots will be reserved and what needs to be cleaned up
	uint32 FirstRequestedSlot = NextSlotIndex;
	uint32 SlotAfterReservation = NextSlotIndex + NumSlotsRequested;

	// Loop around if the end of the heap has been reached
	if (bCanLoopAround && SlotAfterReservation > HeapSize)
	{
		FirstRequestedSlot = 0;
		SlotAfterReservation = NumSlotsRequested;

		FirstUsedSlot = SlotAfterReservation;

		// Notify the derived class that the heap has been looped around
		HeapLoopedAround();
	}

	// Note where to start looking next time
	NextSlotIndex = SlotAfterReservation;

	if (HeapType == ERHIDescriptorHeapType::Standard)
	{
		INC_DWORD_STAT_BY(STAT_NumReservedViewOnlineDescriptors, NumSlotsRequested);
	}
	else
	{
		INC_DWORD_STAT_BY(STAT_NumReservedSamplerOnlineDescriptors, NumSlotsRequested);
	}

	return FirstRequestedSlot;
}


/**
Increment the internal slot counter - only used by threadlocal sampler heap
**/
void FD3D12OnlineHeap::SetNextSlot(uint32 NextSlot)
{
	// For samplers, ReserveSlots will be called with a conservative estimate
	// This is used to correct for the actual number of heap slots used
	check(NextSlot <= NextSlotIndex);

	check(Heap->GetType() != ERHIDescriptorHeapType::Standard);
	DEC_DWORD_STAT_BY(STAT_NumReservedSamplerOnlineDescriptors, NextSlotIndex - NextSlot);

	NextSlotIndex = NextSlot;
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12GlobalSamplerOnlineHeap
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FD3D12GlobalOnlineSamplerHeap::FD3D12GlobalOnlineSamplerHeap(FD3D12Device* Device)
	: FD3D12OnlineHeap(Device, false)
	, UniqueDescriptorTables(MakeShared<FD3D12SamplerSet>())
{}

/** Allocate and initialize the global sampler heap */
void FD3D12GlobalOnlineSamplerHeap::Init(uint32 TotalSize)
{
	Heap = GetParentDevice()->GetDescriptorHeapManager().AllocateHeap(
		TEXT("Device Global - Online Sampler Heap"),
		ERHIDescriptorHeapType::Sampler,
		TotalSize,
		ED3D12DescriptorHeapFlags::GpuVisible
	);

	INC_DWORD_STAT(STAT_NumSamplerOnlineDescriptorHeaps);
	INC_MEMORY_STAT_BY(STAT_SamplerOnlineDescriptorHeapMemory, Heap->GetMemorySize());
	INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, Heap->GetMemorySize());
}

bool FD3D12GlobalOnlineSamplerHeap::RollOver()
{
	// No rollover supported
	check(false);
	UE_LOGF(LogD3D12RHI, Fatal, "Global Descriptor heaps can't roll over!");
	return false;
}

TSharedPtr<FD3D12SamplerSet> FD3D12GlobalOnlineSamplerHeap::GetUniqueDescriptorTables()
{
	FReadScopeLock Lock(Mutex);
	return UniqueDescriptorTables;
}

void FD3D12GlobalOnlineSamplerHeap::ConsolidateUniqueSamplerTables(TArrayView<FD3D12UniqueSamplerTable> UniqueTables)
{
	if (UniqueTables.Num() == 0)
		return;

	FWriteScopeLock Lock(Mutex);

	bool bModified = false;
	for (auto& Table : UniqueTables)
	{
		if (UniqueDescriptorTables->Contains(Table) == false)
		{
			if (CanReserveSlots(Table.Key.Count))
			{
				if (!bModified)
				{
					// Replace with a new copy, to avoid modifying the copy used by other threads.
					UniqueDescriptorTables = MakeShared<FD3D12SamplerSet>(*UniqueDescriptorTables.Get());
					bModified = true;
				}

				uint32 HeapSlot = ReserveSlots(Table.Key.Count);

				D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor = GetCPUSlotHandle(HeapSlot);

				GetParentDevice()->CopyDescriptors(DestDescriptor, Table.CPUTable, Table.Key.Count, FD3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER);

				Table.GPUHandle = GetGPUSlotHandle(HeapSlot);
				UniqueDescriptorTables->Add(Table);
			}
		}
	}

	if (bModified)
	{
		SET_DWORD_STAT(STAT_NumReuseableSamplerOnlineDescriptorTables, UniqueDescriptorTables->Num());
		SET_DWORD_STAT(STAT_NumReuseableSamplerOnlineDescriptors, GetNextSlotIndex());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12SubAllocatedOnlineHeap

FD3D12SubAllocatedOnlineHeap::FD3D12SubAllocatedOnlineHeap(FD3D12DescriptorCache& DescriptorCache, FD3D12CommandContext& Context)
	: FD3D12OnlineHeap(Context.Device, false)
	, DescriptorCache(DescriptorCache)
	, Context(Context)
{
}

/** Handle roll over on the sub allocated online heap - needs a new block */
bool FD3D12SubAllocatedOnlineHeap::RollOver()
{
	// Try and allocate a new block from the global heap
	AllocateBlock();

	// Sub-allocated descriptor heaps don't change, so no need to set descriptor heaps if we still have a block allocated
	return CurrentBlock == nullptr;
}

/** Set the current command list which needs to be notified about changes */
void FD3D12SubAllocatedOnlineHeap::OpenCommandList()
{
	// Allocate a new block if we don't have one yet
	if (CurrentBlock == nullptr)
	{
		AllocateBlock();
	}
}

/** Tries to allocate a new block from the global heap - if it fails then it will switch to thread local view heap */
bool FD3D12SubAllocatedOnlineHeap::AllocateBlock()
{
	FD3D12OnlineDescriptorManager& OnlineManager = GetParentDevice()->GetOnlineDescriptorManager();

	// If we still have a block, then free it first
	if (CurrentBlock)
	{
		// Update actual used size
		check(FirstUsedSlot == 0);
		CurrentBlock->SizeUsed = NextSlotIndex;

		OnlineManager.FreeHeapBlock(CurrentBlock);
		CurrentBlock = nullptr;
	}

	// Try and allocate from the global heap
	CurrentBlock = OnlineManager.AllocateHeapBlock();

	// Reset counters
	NextSlotIndex = 0;
	FirstUsedSlot = 0;
	Heap.SafeRelease();

	// Extract global heap data
	if (CurrentBlock)
	{
		Heap = new FD3D12DescriptorHeap(OnlineManager.GetDescriptorHeap(Context.GetPipeline()), CurrentBlock->BaseSlot, CurrentBlock->Size);
	}
	else
	{
		// Notify parent that we have run out of sub allocations
		// This should *never* happen but we will handle it and revert to local heaps to be safe
		UE_LOGF(LogD3D12RHI, Warning, "Descriptor cache ran out of sub allocated descriptor blocks! Moving to Context local View heap strategy");
		DescriptorCache.SwitchToContextLocalViewHeap();
	}

	// Allocation succeeded?
	return (CurrentBlock != nullptr);
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12LocalOnlineHeap

FD3D12LocalOnlineHeap::FD3D12LocalOnlineHeap(FD3D12DescriptorCache& DescriptorCache, FD3D12CommandContext& Context)
	: FD3D12OnlineHeap(Context.Device, true)
	, DescriptorCache(DescriptorCache)
	, Context(Context)
{
}

/**
Initialize a thread local online heap
**/
void FD3D12LocalOnlineHeap::Init(uint32 InNumDescriptors, ERHIDescriptorHeapType InHeapType)
{
	if (InNumDescriptors > 0)
	{
		const TCHAR* DebugName = InHeapType == ERHIDescriptorHeapType::Standard ? L"Thread Local - Online View Heap" : L"Thread Local - Online Sampler Heap";
		Heap = GetParentDevice()->GetDescriptorHeapManager().AllocateHeap(
			DebugName,
			InHeapType,
			InNumDescriptors,
			ED3D12DescriptorHeapFlags::GpuVisible
		);

		Entry.Heap = Heap;

		if (InHeapType == ERHIDescriptorHeapType::Standard)
		{
			INC_DWORD_STAT(STAT_NumViewOnlineDescriptorHeaps);
			INC_MEMORY_STAT_BY(STAT_ViewOnlineDescriptorHeapMemory, Heap->GetMemorySize());
		}
		else
		{
			INC_DWORD_STAT(STAT_NumSamplerOnlineDescriptorHeaps);
			INC_MEMORY_STAT_BY(STAT_SamplerOnlineDescriptorHeapMemory, Heap->GetMemorySize());
		}
		INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, Heap->GetMemorySize());
	}
	else
	{
		Heap = nullptr;
		Entry.Heap = nullptr;
	}
}


/**
Handle roll over
**/
bool FD3D12LocalOnlineHeap::RollOver()
{
	// Enqueue the current entry
	Entry.SyncPoint = Context.GetContextSyncPoint();
	ReclaimPool.Enqueue(Entry);

	if (ReclaimPool.Peek(Entry) && Entry.SyncPoint->IsComplete())
	{
		ReclaimPool.Dequeue(Entry);

		Heap = Entry.Heap;
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Log, "OnlineHeap RollOver Detected. Increase the heap size to prevent creation of additional heaps");

		//LLM_SCOPE(ELLMTag::DescriptorCache);

		const ERHIDescriptorHeapType HeapType = Heap->GetType();
		const uint32 NumDescriptors = Heap->GetNumDescriptors();

		const TCHAR* DebugName = HeapType == ERHIDescriptorHeapType::Standard ? L"Thread Local - Online View Heap" : L"Thread Local - Online Sampler Heap";
		Heap = GetParentDevice()->GetDescriptorHeapManager().AllocateHeap(
			DebugName,
			HeapType,
			NumDescriptors,
			ED3D12DescriptorHeapFlags::GpuVisible
		);

		if (HeapType == ERHIDescriptorHeapType::Standard)
		{
			INC_DWORD_STAT(STAT_NumViewOnlineDescriptorHeaps);
			INC_MEMORY_STAT_BY(STAT_ViewOnlineDescriptorHeapMemory, Heap->GetMemorySize());
		}
		else
		{
			INC_DWORD_STAT(STAT_NumSamplerOnlineDescriptorHeaps);
			INC_MEMORY_STAT_BY(STAT_SamplerOnlineDescriptorHeapMemory, Heap->GetMemorySize());
		}
		INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, Heap->GetMemorySize());

		Entry.Heap = Heap;
	}

	NextSlotIndex = 0;
	FirstUsedSlot = 0;

	return DescriptorCache.HeapRolledOver(Heap->GetType());
}


/**
Handle loop around on the heap
**/
void FD3D12LocalOnlineHeap::HeapLoopedAround()
{
	DescriptorCache.HeapLoopedAround(Heap->GetType());
}

void FD3D12LocalOnlineHeap::RecycleSlots()
{
	// Free up slots for finished command lists
	FSyncPointEntry SyncPoint;
	while (SyncPoints.Peek(SyncPoint) && SyncPoint.SyncPoint->IsComplete())
	{
		SyncPoints.Dequeue(SyncPoint);
		FirstUsedSlot = SyncPoint.LastSlotInUse + 1;
	}
}

void FD3D12LocalOnlineHeap::OpenCommandList()
{
	RecycleSlots();
}

void FD3D12LocalOnlineHeap::CloseCommandList()
{
	if (NextSlotIndex > 0)
	{
		// Track the previous command list
		FSyncPointEntry SyncPoint;
		SyncPoint.SyncPoint = Context.GetContextSyncPoint();
		SyncPoint.LastSlotInUse = NextSlotIndex - 1;
		SyncPoints.Enqueue(SyncPoint);

		Entry.SyncPoint = Context.GetContextSyncPoint();

		RecycleSlots();
	}
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Util
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 GetTypeHash(const FD3D12SamplerArrayDesc& Key)
{
	return FD3D12PipelineStateCache::HashData((void*)Key.SamplerID, Key.Count * sizeof(Key.SamplerID[0]));
}

uint32 GetTypeHash(const FD3D12UniqueSamplerTable& Table)
{
	return FD3D12PipelineStateCache::HashData((void*)Table.Key.SamplerID, Table.Key.Count * sizeof(Table.Key.SamplerID[0]));
}
