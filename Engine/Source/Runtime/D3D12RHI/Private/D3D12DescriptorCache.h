// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Set.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "D3D12RHICommon.h"
#include "D3D12Descriptors.h"

class FD3D12CommandContext;
class FD3D12DynamicRHI;
class FD3D12DepthStencilView;
class FD3D12ExplicitDescriptorCache;
class FD3D12RenderTargetView;
class FD3D12ShaderResourceView;
class FD3D12UnorderedAccessView;
class FD3D12DescriptorCache;
class FD3D12RootSignature;

struct FD3D12DefaultViews;
struct FD3D12VertexBufferCache;
struct FD3D12IndexBufferCache;
struct FD3D12ConstantBufferCache;
struct FD3D12ShaderResourceViewCache;
struct FD3D12UnorderedAccessViewCache;
struct FD3D12SamplerStateCache;
class FD3D12StateCache;

class FD3D12SyncPoint;
using FD3D12SyncPointRef = TRefCountPtr<FD3D12SyncPoint>;

// Like a TMap<KeyType, ValueType>
// Faster lookup performance, but possibly has false negatives
template<typename KeyType, typename ValueType>
class FD3D12ConservativeMap
{
public:
	FD3D12ConservativeMap(uint32 Size)
	{
		Table.AddUninitialized(Size);

		Reset();
	}

	void Add(const KeyType& Key, const ValueType& Value)
	{
		uint32 Index = GetIndex(Key);

		Entry& Pair = Table[Index];

		Pair.Valid = true;
		Pair.Key = Key;
		Pair.Value = Value;
	}

	ValueType* Find(const KeyType& Key)
	{
		uint32 Index = GetIndex(Key);

		Entry& Pair = Table[Index];

		if (Pair.Valid &&
			(Pair.Key == Key))
		{
			return &Pair.Value;
		}
		else
		{
			return nullptr;
		}
	}

	void Reset()
	{
		for (int32 i = 0; i < Table.Num(); i++)
		{
			Table[i].Valid = false;
		}
	}

private:
	uint32 GetIndex(const KeyType& Key)
	{
		uint32 Hash = GetTypeHash(Key);

		return Hash % static_cast<uint32>(Table.Num());
	}

	struct Entry
	{
		bool Valid;
		KeyType Key;
		ValueType Value;
	};

	TArray<Entry> Table;
};

struct FD3D12SamplerArrayDesc
{
	uint32 Count;
	uint16 SamplerID[MAX_SAMPLERS];

	inline bool operator==(const FD3D12SamplerArrayDesc& rhs) const
	{
		check(Count <= UE_ARRAY_COUNT(SamplerID));
		check(rhs.Count <= UE_ARRAY_COUNT(rhs.SamplerID));

		if (Count != rhs.Count)
		{
			return false;
		}
		else
		{
			// It is safe to compare pointers, because samplers are kept alive for the lifetime of the RHI
			return 0 == FMemory::Memcmp(SamplerID, rhs.SamplerID, sizeof(SamplerID[0]) * Count);
		}
	}
};

uint32 GetTypeHash(const D3D12_SAMPLER_DESC& Desc);
uint32 GetTypeHash(const FD3D12SamplerArrayDesc& Key);

typedef FD3D12ConservativeMap<FD3D12SamplerArrayDesc, D3D12_GPU_DESCRIPTOR_HANDLE> FD3D12SamplerMap;

struct FD3D12UniqueSamplerTable
{
	FD3D12UniqueSamplerTable() = default;
	FD3D12UniqueSamplerTable(FD3D12SamplerArrayDesc KeyIn, D3D12_CPU_DESCRIPTOR_HANDLE* Table)
	{
		FMemory::Memcpy(&Key, &KeyIn, sizeof(Key));//Memcpy to avoid alignement issues
		FMemory::Memcpy(CPUTable, Table, Key.Count * sizeof(D3D12_CPU_DESCRIPTOR_HANDLE));
	}

	FD3D12SamplerArrayDesc Key{};
	D3D12_CPU_DESCRIPTOR_HANDLE CPUTable[MAX_SAMPLERS]{};

	// This will point to the table start in the global heap
	D3D12_GPU_DESCRIPTOR_HANDLE GPUHandle{};
};
uint32 GetTypeHash(const FD3D12UniqueSamplerTable& Table);


struct FD3D12UniqueSamplerTableKeyFuncs : BaseKeyFuncs<FD3D12UniqueSamplerTable, FD3D12UniqueSamplerTable, /*bInAllowDuplicateKeys = */ false>
{
	typedef typename TCallTraits<FD3D12UniqueSamplerTable>::ParamType KeyInitType;
	typedef typename TCallTraits<FD3D12UniqueSamplerTable>::ParamType ElementInitType;

	/**
	* @return The key used to index the given element.
	*/
	static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	/**
	* @return True if the keys match.
	*/
	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.Key == B.Key;
	}

	/** Calculates a hash index for a key. */
	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key.Key);
	}
};

typedef TSet<FD3D12UniqueSamplerTable, FD3D12UniqueSamplerTableKeyFuncs> FD3D12SamplerSet;

/** Manages a D3D heap which is GPU visible - base class which can be used by the FD3D12DescriptorCache */
class FD3D12OnlineHeap : public FD3D12DeviceChild
{
public:
	FD3D12OnlineHeap(FD3D12Device* Device, bool CanLoopAround);
	virtual ~FD3D12OnlineHeap();

	void CleanupResources();

	ID3D12DescriptorHeap* GetHeap() { return Heap->GetHeap(); }

	FORCEINLINE D3D12_CPU_DESCRIPTOR_HANDLE GetCPUSlotHandle(uint32 Slot) const { return Heap->GetCPUSlotHandle(Slot); }
	FORCEINLINE D3D12_GPU_DESCRIPTOR_HANDLE GetGPUSlotHandle(uint32 Slot) const { return Heap->GetGPUSlotHandle(Slot); }

	// Call this to reserve descriptor heap slots for use by the command list you are currently recording. This will wait if
	// necessary until slots are free (if they are currently in use by another command list.) If the reservation can be
	// fulfilled, the index of the first reserved slot is returned (all reserved slots are consecutive.) If not, it will 
	// throw an exception.
	bool CanReserveSlots(uint32 NumSlots);
	uint32 ReserveSlots(uint32 NumSlotsRequested);

	void SetNextSlot(uint32 NextSlot);
	uint32 GetNextSlotIndex() const { return NextSlotIndex; }

	// Function which can/should be implemented by the derived classes
	virtual bool RollOver() = 0;
	virtual void HeapLoopedAround() { }
	virtual void OpenCommandList () { }
	virtual void CloseCommandList() { }
	virtual uint32 GetTotalSize() { return Heap->GetNumDescriptors(); }

protected:
	// Keeping this ptr around is basically just for lifetime management
	TRefCountPtr<FD3D12DescriptorHeap> Heap;

	// This index indicate where the next set of descriptors should be placed *if* there's room
	uint32 NextSlotIndex = 0;

	// Indicates the last free slot marked by the command list being finished
	uint32 FirstUsedSlot = 0;

	// Does the heap support loop around allocations
	const bool bCanLoopAround;
};

/** Global sampler heap managed by the device which stored a unique set of sampler sets */
class FD3D12GlobalOnlineSamplerHeap : public FD3D12OnlineHeap
{
public:
	FD3D12GlobalOnlineSamplerHeap(FD3D12Device* Device);

	void Init(uint32 TotalSize);

	// Override FD3D12OnlineHeap functions
	virtual bool RollOver() final override;

	void ConsolidateUniqueSamplerTables(TArrayView<FD3D12UniqueSamplerTable> UniqueTables);

	TSharedPtr<FD3D12SamplerSet> GetUniqueDescriptorTables();

private:
	TSharedPtr<FD3D12SamplerSet> UniqueDescriptorTables;
	FRWLock Mutex;
};

/** Online heap which can be used by a FD3D12DescriptorCache to manage a block allocated from a GlobalHeap */
class FD3D12SubAllocatedOnlineHeap : public FD3D12OnlineHeap
{
public:
	FD3D12SubAllocatedOnlineHeap(FD3D12DescriptorCache& DescriptorCache, FD3D12CommandContext& Context);

	// Override FD3D12OnlineHeap functions
	virtual bool RollOver() final override;
	virtual void OpenCommandList() final override;
	virtual uint32 GetTotalSize() final override
	{
		return CurrentBlock ? CurrentBlock->Size : 0;
	}

private:
	// Allocate a new block from the global heap - return true if allocation succeeds
	bool AllocateBlock();

	FD3D12OnlineDescriptorBlock* CurrentBlock = nullptr;

	FD3D12DescriptorCache& DescriptorCache;
	FD3D12CommandContext& Context;
};


/** Online heap which is not shared between multiple FD3D12DescriptorCache.
 *  Used as overflow heap when the global heaps are full or don't contain the required data
 */
class FD3D12LocalOnlineHeap : public FD3D12OnlineHeap
{
public:
	FD3D12LocalOnlineHeap(FD3D12DescriptorCache& DescriptorCache, FD3D12CommandContext& Context);

	// Allocate the actual overflow heap
	void Init(uint32 InNumDescriptors, ERHIDescriptorHeapType InHeapType);

	// Override FD3D12OnlineHeap functions
	virtual bool RollOver() final override;
	virtual void HeapLoopedAround() final override;
	virtual void OpenCommandList () final override;
	virtual void CloseCommandList() final override;

private:
	struct FSyncPointEntry
	{
		FD3D12SyncPointRef SyncPoint;
		uint32 LastSlotInUse = 0;
	};
	TQueue<FSyncPointEntry> SyncPoints;

	struct FPoolEntry
	{
		TRefCountPtr<FD3D12DescriptorHeap> Heap;
		FD3D12SyncPointRef SyncPoint;
	};
	FPoolEntry Entry {};
	TQueue<FPoolEntry> ReclaimPool;

	FD3D12DescriptorCache& DescriptorCache;
	FD3D12CommandContext& Context;

	void RecycleSlots();
};

enum class ED3D12SetDescriptorHeapsFlags
{
	None         = 0,
	ForceChanged = 1 << 0,
	Bindless     = 1 << 1,
};
ENUM_CLASS_FLAGS(ED3D12SetDescriptorHeapsFlags);

class FD3D12DescriptorCache : public FD3D12DeviceChild, public FD3D12SingleNodeGPUObject
{
public:
	FD3D12DescriptorCache() = delete;
	FD3D12DescriptorCache(FD3D12CommandContext& Context, FRHIGPUMask Node);

	~FD3D12DescriptorCache();

	inline FD3D12OnlineHeap* GetCurrentViewHeap() const { return CurrentViewHeap; }
	inline FD3D12OnlineHeap* GetCurrentSamplerHeap() const { return CurrentSamplerHeap; }

	ID3D12DescriptorHeap* GetLastSetViewHeap() const { return LastSetViewHeap; }

	// Checks if the specified descriptor heap has been set on the current command list.
	bool IsHeapSet(ID3D12DescriptorHeap* const pHeap) const
	{
		return (pHeap == LastSetViewHeap) || (pHeap == LastSetSamplerHeap);
	}

	// Notify the descriptor cache every time you start recording a command list.
	// This sets descriptor heaps on the command list and indicates the current fence value which allows
	// us to avoid querying DX12 for that value thousands of times per frame, which can be costly.
	void OpenCommandList();
	void CloseCommandList();

	// ------------------------------------------------------
	// end Descriptor Slot Reservation stuff

	void SetVertexBuffers(FD3D12VertexBufferCache& Cache);
	void SetRenderTargets(FD3D12RenderTargetView** RenderTargetViewArray, uint32 Count, FD3D12DepthStencilView* DepthStencilTarget);

	D3D12_GPU_DESCRIPTOR_HANDLE BuildUAVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12UnorderedAccessViewCache& Cache, const UAVSlotMask& SlotsNeededMask, uint32 Count, uint32 &HeapSlot);
	D3D12_GPU_DESCRIPTOR_HANDLE BuildSamplerTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12SamplerStateCache& Cache, const SamplerSlotMask& SlotsNeededMask, uint32 Count, uint32& HeapSlot);
	D3D12_GPU_DESCRIPTOR_HANDLE BuildSRVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ShaderResourceViewCache& Cache, const SRVSlotMask& SlotsNeededMask, uint32 Count, uint32& HeapSlot);

	void SetUAVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12UnorderedAccessViewCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor);
	void SetSamplerTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12SamplerStateCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor);
	void SetSRVTable(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ShaderResourceViewCache& Cache, uint32 SlotsNeeded, const D3D12_GPU_DESCRIPTOR_HANDLE& BindDescriptor);

	void SetConstantBufferViews(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ConstantBufferCache& Cache, CBVSlotMask SlotsNeededMask, uint32 Count, uint32& HeapSlot);
	void SetRootConstantBuffers(EShaderFrequency ShaderStage, const FD3D12RootSignature* RootSignature, FD3D12ConstantBufferCache& Cache, CBVSlotMask SlotsNeededMask, FD3D12StateCache* StateCache);

	void PrepareBindlessViews(EShaderFrequency ShaderStage, TConstArrayView<FD3D12ShaderResourceView*> SRVs, TConstArrayView<FD3D12UnorderedAccessView*> UAVs);

	bool HeapRolledOver(ERHIDescriptorHeapType InHeapType);
	void HeapLoopedAround(ERHIDescriptorHeapType InHeapType);
	void Init(uint32 InNumLocalViewDescriptors, uint32 InNumSamplerDescriptors);

	bool SwitchToContextLocalViewHeap();
	bool SwitchToContextLocalSamplerHeap();
	void SwitchToGlobalSamplerHeap();

	void SetExplicitDescriptorCache(FD3D12ExplicitDescriptorCache& ExplicitDescriptorCache);
	void UnsetExplicitDescriptorCache();

	bool UsingGlobalSamplerHeap() const
	{
		return CurrentSamplerHeap != &LocalSamplerHeap;
	}

	FD3D12SamplerSet& GetLocalSamplerSet()
	{
		return *LocalSamplerSet.Get();
	}

	// Sets the current descriptor tables on the command list and marks any descriptor tables as dirty if necessary.
	// Returns true if one of the heaps actually changed, false otherwise.
	bool SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags Flags);


#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	bool CouldUseBindless() const { return bCouldUseBindless; }
	bool IsFullyBindless()  const { return bFullyBindless; }
#else
	constexpr bool CouldUseBindless() const { return false; }
	constexpr bool IsFullyBindless()  const { return false; }
#endif

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetBindlessResourcesHeap(FD3D12DescriptorHeap* InHeap)
	{
		BindlessResourcesHeap = InHeap;
	}

	void SetBindlessSamplersHeap(FD3D12DescriptorHeap* InHeap)
	{
		BindlessSamplersHeap = InHeap;
	}

	FD3D12DescriptorHeap* GetBindlessResourcesHeap() const
	{
		return BindlessResourcesHeap;
	}

	bool IsUsingBindlessHeap() const
	{
		return bLastSetHeapsBindless || bFullyBindless;
	}
#endif

protected:
	FD3D12CommandContext& Context;
	const FD3D12DefaultViews& DefaultViews;

private:
	// The previous view and sampler heaps set on the current command list.
	ID3D12DescriptorHeap* LastSetViewHeap = nullptr;
	ID3D12DescriptorHeap* LastSetSamplerHeap = nullptr;

	FD3D12OnlineHeap* CurrentViewHeap = nullptr;
	FD3D12OnlineHeap* CurrentSamplerHeap = nullptr;

	FD3D12LocalOnlineHeap* LocalViewHeap = nullptr;
	FD3D12LocalOnlineHeap LocalSamplerHeap;
	FD3D12SubAllocatedOnlineHeap SubAllocatedViewHeap;

	FD3D12SamplerMap SamplerMap;

	TArray<FD3D12UniqueSamplerTable> UniqueTables;

	TSharedPtr<FD3D12SamplerSet> LocalSamplerSet;
	bool bUsingExplicitCacheHeaps = false;
	bool bExplicitViewHeapIsBindless = false;
	bool bLocalSamplerHeapOpen = false;
	bool bUsingViewHeap = true;

	uint32 NumLocalViewDescriptors = 0;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	bool bCouldUseBindless = false;
	bool bFullyBindless = false;

	bool bLastSetHeapsBindless = false;
	bool bLastSetHeapBeforeExplicitIsBindless = false;

	FD3D12DescriptorHeapPtr BindlessResourcesHeap = nullptr;
	FD3D12DescriptorHeapPtr BindlessSamplersHeap = nullptr;
#endif
};