// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12DescriptorRange.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

#include "D3D12Adapter.h"
#include "D3D12RHIPrivate.h"

static FD3D12View* GetViewFromDescriptorResource(const FRHIDescriptorResource& Resource, uint32 GpuIndex)
{
	switch (Resource.Type)
	{
	case FRHIDescriptorResource::EType::Texture:
	{
		FD3D12Texture* D3D12Texture = FD3D12CommandContext::RetrieveTexture(static_cast<FRHITexture*>(Resource.Resource), GpuIndex);
		return D3D12Texture->GetShaderResourceView();
	}
	case FRHIDescriptorResource::EType::ResourceView:
	{
		FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Resource.Resource);
		FD3D12ShaderResourceView_RHI* D3D12ShaderResourceView = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(SRV, GpuIndex);
		return D3D12ShaderResourceView;
	}
	case FRHIDescriptorResource::EType::UnorderedAccessView:
	{
		FRHIUnorderedAccessView* UAV = static_cast<FRHIUnorderedAccessView*>(Resource.Resource);
		FD3D12UnorderedAccessView_RHI* D3D12UnorderedAccessView = FD3D12CommandContext::RetrieveObject<FD3D12UnorderedAccessView_RHI>(UAV, GpuIndex);
		return D3D12UnorderedAccessView;
	}
	}

	return nullptr;
}

FD3D12DescriptorRange::FD3D12DescriptorRange(FD3D12Device* InParent)
	: FRHIDescriptorRange(FRHIDescriptorRangeDesc(), {})
	, FD3D12DeviceChild(InParent)
	, Allocation(0, 0)
{
}

FD3D12DescriptorRange::FD3D12DescriptorRange(FD3D12Device* InParent, const FRHIDescriptorRangeDesc& InDesc, TConstArrayView<FRHIDescriptorResource> InResources, FRHIDescriptorAllocation InAllocation)
	: FRHIDescriptorRange(InDesc, InResources)
	, FD3D12DeviceChild(InParent)
	, Allocation(InAllocation)
{
}

FD3D12DescriptorRange::~FD3D12DescriptorRange()
{
	if (Allocation.Count > 0)
	{
		// Deferred so in-flight command lists can finish referencing these descriptors.
		FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(Allocation, GetDesc().TypeMask, GetParentDevice());
	}
}

FRHIDescriptorRangeShaderData FD3D12DescriptorRange::GetShaderData() const
{
	FRHIDescriptorRangeShaderData ShaderData{};
	ShaderData.StartIndex = Allocation.StartIndex;
	ShaderData.Count = Allocation.Count;
	return ShaderData;
}

void FD3D12DescriptorRange::InitializeResources(TConstArrayView<FRHIDescriptorResource> InResources)
{
	const ERHIDescriptorTypeMask DescriptorTypeMask = GetDesc().TypeMask;

	FD3D12Device* Device = GetParentDevice();
	const uint32 GpuIndex = Device->GetGPUIndex();

	FD3D12BindlessDescriptorManager& Manager = Device->GetBindlessDescriptorManager();

	const int32 NumResources = FMath::Min(InResources.Num(), (int32)GetDesc().Count);

	for (int32 Index = 0; Index < NumResources; Index++)
	{
		const FRHIDescriptorResource& Resource = InResources[Index];

		if (const FD3D12View* View = GetViewFromDescriptorResource(Resource, GpuIndex))
		{
			checkf(EnumHasAllFlags(DescriptorTypeMask, View->GetBindlessHandle().GetTypeMask()),
				TEXT("View descriptor type %s is not compatible with descriptor range type mask %s."),
				GetDescriptorTypeString(View->GetBindlessHandle().GetType()),
				*GetDescriptorMaskString(DescriptorTypeMask));

			const FRHIDescriptorHandle UpdateHandle(View->GetBindlessHandle().GetType(), Allocation.StartIndex + Index);
			Manager.InitializeDescriptor(UpdateHandle, View);
		}
	}

	// Rebuild cached views so residency is correct at next bind.
	PopulateResidencyViews();
}

void FD3D12DescriptorRange::UpdateResources(const FRHIContextArray& Contexts, uint32 FirstResourceIndex, TConstArrayView<FRHIDescriptorResource> NewResources)
{
	const ERHIDescriptorTypeMask DescriptorTypeMask = GetDesc().TypeMask;

	FD3D12Device* Device = GetParentDevice();
	const uint32 GpuIndex = Device->GetGPUIndex();

	FD3D12BindlessDescriptorManager& Manager = Device->GetBindlessDescriptorManager();

	for (int32 Index = 0; Index < NewResources.Num(); Index++)
	{
		const int32 InternalIndex = FirstResourceIndex + Index;
		if (InternalIndex < (int32)GetDesc().Count)
		{
			const FRHIDescriptorResource& Resource = NewResources[Index];
			UpdateResource(FirstResourceIndex + Index, Resource);

			if (const FD3D12View* View = GetViewFromDescriptorResource(Resource, GpuIndex))
			{
				checkf(EnumHasAllFlags(DescriptorTypeMask, View->GetBindlessHandle().GetTypeMask()),
					TEXT("View descriptor type %s is not compatible with descriptor range type mask %s."),
					GetDescriptorTypeString(View->GetBindlessHandle().GetType()),
					*GetDescriptorMaskString(DescriptorTypeMask));

				const FRHIDescriptorHandle UpdateHandle(View->GetBindlessHandle().GetType(), Allocation.StartIndex + InternalIndex);
				Manager.UpdateDescriptor(Contexts, UpdateHandle, View);
			}
		}
	}

	// Rebuild cached views so residency is correct at next bind.
	PopulateResidencyViews();
}

void FD3D12DescriptorRange::PopulateResidencyViews()
{
	AllSrvs.Reset();
	AllUavs.Reset();

	const uint32 GpuIndex = GetParentDevice()->GetGPUIndex();

	for (int32 Index = 0; Index < Resources.Num(); Index++)
	{
		FRHIResource* Resource = Resources[Index].GetReference();
		if (!Resource)
		{
			continue;
		}

		const FRHIDescriptorResource DescriptorResource(ResourceTypes[Index], Resource);

		if (FD3D12View* View = GetViewFromDescriptorResource(DescriptorResource, GpuIndex))
		{
			if (ResourceTypes[Index] == FRHIDescriptorResource::EType::UnorderedAccessView)
			{
				AllUavs.AddUnique(static_cast<FD3D12UnorderedAccessView*>(View));
			}
			else
			{
				AllSrvs.AddUnique(static_cast<FD3D12ShaderResourceView*>(View));
			}
		}
	}
}

FDescriptorRangeRHIRef FD3D12DynamicRHI::RHICreateDescriptorRange(FRHICommandListBase& RHICmdList, const FRHIDescriptorRangeDesc& Desc, TConstArrayView<FRHIDescriptorResource> InitialResources)
{
	FD3D12DescriptorRange* DescriptorRange = GetAdapter().CreateLinkedObject<FD3D12DescriptorRange>(FRHIGPUMask::All(),
		[&Desc, &InitialResources](FD3D12Device* Device, FRHIDescriptorRange* FirstLinkedObject)
		{
			if (TOptional<FRHIDescriptorAllocation> Allocation = Device->GetBindlessDescriptorAllocator().AllocateDescriptors(Desc.TypeMask, Desc.Count))
			{
				return new FD3D12DescriptorRange(Device, Desc, InitialResources, *Allocation);
			}

			return new FD3D12DescriptorRange(Device);
		});

	// Because view descriptor initialization can be deferred to the RHI thread, we need to also defer our descriptor copies.
	TConstArrayView<FRHIDescriptorResource> InitialResourcesCmd = RHICmdList.IsTopOfPipe() ? RHICmdList.AllocArray(InitialResources) : InitialResources;

	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("RHICreateDescriptorRange"),
		[DescriptorRange, InitialResources = MoveTemp(InitialResourcesCmd)](const FRHIContextArray& Contexts)
		{
			for (FD3D12DescriptorRange::FLinkedObjectIterator RangeIt(DescriptorRange); RangeIt; ++RangeIt)
			{
				RangeIt->InitializeResources(InitialResources);
			}
		});

	return DescriptorRange;
}

void FD3D12DynamicRHI::RHIUpdateDescriptorRange(FRHICommandListBase& RHICmdList, FRHIDescriptorRange* InDescriptorRange, uint32 FirstResourceIndex, TConstArrayView<FRHIDescriptorResource> InNewResources)
{
	FD3D12DescriptorRange* DescriptorRange = ResourceCast(InDescriptorRange);

	if (FirstResourceIndex < DescriptorRange->GetDesc().Count)
	{
		// Clamp the incoming resources to not exceed the range's capacity.
		TConstArrayView<FRHIDescriptorResource> NewResources = InNewResources.Left(DescriptorRange->GetDesc().Count - FirstResourceIndex);
		TConstArrayView<FRHIDescriptorResource> NewResourcesCmd = RHICmdList.IsTopOfPipe() ? RHICmdList.AllocArray(NewResources) : NewResources;

		RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("RHIUpdateDescriptorRange"),
			[DescriptorRange, FirstResourceIndex, NewResources = MoveTemp(NewResourcesCmd)](const FRHIContextArray& Contexts)
			{
				for (FD3D12DescriptorRange::FLinkedObjectIterator RangeIt(DescriptorRange); RangeIt; ++RangeIt)
				{
					RangeIt->UpdateResources(Contexts, FirstResourceIndex, NewResources);
				}
			});
	}
}

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
