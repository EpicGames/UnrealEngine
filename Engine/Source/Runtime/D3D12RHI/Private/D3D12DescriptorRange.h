// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDescriptorRange.h"
#include "RHIDescriptorAllocator.h"
#include "D3D12RHICommon.h"

class FD3D12ShaderResourceView;
class FD3D12UnorderedAccessView;

class FD3D12DescriptorRange : public FRHIDescriptorRange, public FD3D12DeviceChild, public FD3D12LinkedAdapterObject<FD3D12DescriptorRange>
{
public:
	// Constructs an invalid/empty descriptor range. Used as a fallback when allocation fails.
	FD3D12DescriptorRange(FD3D12Device* InParent);
	FD3D12DescriptorRange(FD3D12Device* InParent, const FRHIDescriptorRangeDesc& InDesc, TConstArrayView<FRHIDescriptorResource> InResources, FRHIDescriptorAllocation Allocation);
	~FD3D12DescriptorRange();

	FRHIDescriptorRangeShaderData GetShaderData() const override;

	void InitializeResources(TConstArrayView<FRHIDescriptorResource> InResources);
	void UpdateResources(const FRHIContextArray& Contexts, uint32 FirstResourceIndex, TConstArrayView<FRHIDescriptorResource> NewResources);

	// Rebuilds the cached AllSrvs/AllUavs arrays from the current resource list.
	void PopulateResidencyViews();

	FRHIDescriptorAllocation Allocation;

	// Cached D3D12 views for all resources in the range. Rebuilt on create/update via PopulateResidencyViews().
	// Queued for residency (via QueueBindlessSRVs/UAVs) when the descriptor range is bound to a shader.
	TArray<FD3D12ShaderResourceView*> AllSrvs;
	TArray<FD3D12UnorderedAccessView*> AllUavs;
};

template<>
struct TD3D12ResourceTraits<FRHIDescriptorRange>
{
	using TConcreteType = FD3D12DescriptorRange;
};
