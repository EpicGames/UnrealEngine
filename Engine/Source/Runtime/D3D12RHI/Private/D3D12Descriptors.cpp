// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12Descriptors.h"
#include "D3D12RHIPrivate.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UE::D3D12Descriptors::CopyDescriptor(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorIndex DstIndex, D3D12_CPU_DESCRIPTOR_HANDLE SrcCpuHandle)
{
	const D3D12_CPU_DESCRIPTOR_HANDLE DestCpuHandle = TargetHeap->GetCPUSlotHandle(DstIndex);
	const D3D12_DESCRIPTOR_HEAP_TYPE D3DHeapType = Translate(TargetHeap->GetType());

	Device->GetDevice()->CopyDescriptorsSimple(1, DestCpuHandle, SrcCpuHandle, D3DHeapType);
}

void UE::D3D12Descriptors::CopyDescriptor(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FRHIDescriptorHandle DstHandle, D3D12_CPU_DESCRIPTOR_HANDLE SrcCpuHandle)
{
	if (DstHandle.IsValid())
	{
		checkSlow(Translate(TargetHeap->GetType()) == Translate(DstHandle.GetType()));
		CopyDescriptor(Device, TargetHeap, DstHandle.GetIndex(), SrcCpuHandle);
	}
}

void UE::D3D12Descriptors::CopyDescriptors(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorHeap* SourceHeap, uint32 FirstDescriptor, uint32 NumDescriptors)
{
	// Use regular CPU Profile without dynamic value - almost same overhead for string formatting as actual Copy
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("CopyDescriptors HeapToHeap");
	//SCOPED_NAMED_EVENT_F(TEXT("CopyDescriptors HeapToHeap (%d)"), FColor::Turquoise, NumDescriptors);

	const D3D12_CPU_DESCRIPTOR_HANDLE TargetStart = TargetHeap->GetCPUSlotHandle(FirstDescriptor);
	const D3D12_CPU_DESCRIPTOR_HANDLE SourceStart = SourceHeap->GetCPUSlotHandle(FirstDescriptor);
	const D3D12_DESCRIPTOR_HEAP_TYPE D3DHeapType = Translate(TargetHeap->GetType());

	Device->GetDevice()->CopyDescriptorsSimple(
		NumDescriptors,
		TargetStart,
		SourceStart,
		D3DHeapType
	);
}

void UE::D3D12Descriptors::CopyDescriptors(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorHeap* SourceHeap, TConstArrayView<FD3D12DescriptorIndex> Indices)
{
	// Use regular CPU Profile without dynamic value - almost same overhead for string formatting as actual Copy
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("CopyDescriptors HeapToHeap Scattered");
	//SCOPED_NAMED_EVENT_F(TEXT("CopyDescriptors HeapToHeap Scattered (%d)"), FColor::Turquoise, Indices.Num());

	const int32 NumDescriptors = Indices.Num();
	const D3D12_DESCRIPTOR_HEAP_TYPE D3DHeapType = Translate(TargetHeap->GetType());

	TArray<D3D12_CPU_DESCRIPTOR_HANDLE> DstDescriptors;
	TArray<D3D12_CPU_DESCRIPTOR_HANDLE> SrcDescriptors;
	TArray<UINT> CopyCounts;

	DstDescriptors.SetNumUninitialized(NumDescriptors);
	SrcDescriptors.SetNumUninitialized(NumDescriptors);
	CopyCounts.SetNumUninitialized(NumDescriptors);

	for (int32 Index = 0; Index < NumDescriptors; Index++)
	{
		DstDescriptors[Index] = TargetHeap->GetCPUSlotHandle(Indices[Index]);
		SrcDescriptors[Index] = SourceHeap->GetCPUSlotHandle(Indices[Index]);
		CopyCounts[Index] = 1;
	}

	Device->GetDevice()->CopyDescriptors(
		NumDescriptors, DstDescriptors.GetData(), CopyCounts.GetData(),
		NumDescriptors, SrcDescriptors.GetData(), CopyCounts.GetData(),
		D3DHeapType
	);
}

FD3D12OfflineDescriptor UE::D3D12Descriptors::CreateOfflineCopy(FD3D12Device* Device, D3D12_CPU_DESCRIPTOR_HANDLE InDescriptor, ERHIDescriptorHeapType InType)
{
	FD3D12OfflineDescriptor NewDescriptor = Device->GetOfflineDescriptorManager(InType).AllocateHeapSlot();

	Device->GetDevice()->CopyDescriptorsSimple(1, NewDescriptor, InDescriptor, Translate(InType));

	NewDescriptor.IncrementVersion();

	return NewDescriptor;
}

FD3D12OfflineDescriptor UE::D3D12Descriptors::CreateOfflineCopy(FD3D12Device* Device, FD3D12DescriptorHeap* InHeap, FRHIDescriptorHandle InHandle)
{
	D3D12_CPU_DESCRIPTOR_HANDLE SourceDescriptor = InHeap->GetCPUSlotHandle(InHandle.GetIndex());
	return UE::D3D12Descriptors::CreateOfflineCopy(Device, SourceDescriptor, InHeap->GetType());
}

void UE::D3D12Descriptors::FreeOfflineCopy(FD3D12Device* Device, FD3D12OfflineDescriptor& InDescriptor, ERHIDescriptorHeapType InType)
{
	Device->GetOfflineDescriptorManager(InType).FreeHeapSlot(InDescriptor);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12DescriptorHeap

FD3D12DescriptorHeap::FD3D12DescriptorHeap(FD3D12Device* InDevice, TRefCountPtr<ID3D12DescriptorHeap>&& InHeap, uint32 InNumDescriptors, ERHIDescriptorHeapType InType, ED3D12DescriptorHeapFlags InFlags, bool bInIsGlobal)
	: FD3D12DeviceChild(InDevice)
	, Heap(Forward<TRefCountPtr<ID3D12DescriptorHeap>>(InHeap))
	, CpuBase(Heap->GetCPUDescriptorHandleForHeapStart())
	, GpuBase(EnumHasAnyFlags(InFlags, ED3D12DescriptorHeapFlags::GpuVisible) ? Heap->GetGPUDescriptorHandleForHeapStart() : D3D12_GPU_DESCRIPTOR_HANDLE{})
	, Offset(0)
	, NumDescriptors(InNumDescriptors)
	, DescriptorSize(InDevice->GetDevice()->GetDescriptorHandleIncrementSize(Translate(InType)))
	, Type(InType)
	, Flags(InFlags)
	, bIsGlobal(bInIsGlobal)
	, bIsSuballocation(false)
{
#if STATS
	const size_t MemorySize = DescriptorSize * NumDescriptors;
	if (Type == ERHIDescriptorHeapType::Standard)
	{
		INC_MEMORY_STAT_BY(STAT_BindlessResourceHeapMemory, MemorySize);
	}
	else if (Type == ERHIDescriptorHeapType::Sampler)
	{
		INC_MEMORY_STAT_BY(STAT_BindlessSamplerHeapMemory, MemorySize);
	}
#endif
}

FD3D12DescriptorHeap::FD3D12DescriptorHeap(FD3D12DescriptorHeap* SubAllocateSourceHeap, uint32 InOffset, uint32 InNumDescriptors)
	: FD3D12DeviceChild(SubAllocateSourceHeap->GetParentDevice())
	, Heap(SubAllocateSourceHeap->Heap)
	, CpuBase(SubAllocateSourceHeap->CpuBase, InOffset, SubAllocateSourceHeap->DescriptorSize)
	, GpuBase(SubAllocateSourceHeap->GpuBase, InOffset, SubAllocateSourceHeap->DescriptorSize)
	, Offset(InOffset)
	, NumDescriptors(InNumDescriptors)
	, DescriptorSize(SubAllocateSourceHeap->DescriptorSize)
	, Type(SubAllocateSourceHeap->GetType())
	, Flags(SubAllocateSourceHeap->GetFlags())
	, bIsGlobal(SubAllocateSourceHeap->IsGlobal())
	, bIsSuballocation(true)
{
}

FD3D12DescriptorHeap::FD3D12DescriptorHeap(FD3D12DescriptorManager* InManager, uint32 InOffset, uint32 InNumDescriptors)
	: FD3D12DescriptorHeap(InManager->GetHeap(), InOffset, InNumDescriptors)
{
	Manager = InManager;
}

FD3D12DescriptorHeap::~FD3D12DescriptorHeap()
{
	if (Manager)
	{
		Manager->FreeSubheap(this);
	}

	if (CanBePooled())
	{
		GetParentDevice()->GetDescriptorHeapManager().AddHeapToPool(MoveTemp(Heap), GetType(), GetNumDescriptors(), GetFlags());
	}

#if STATS
	if (!bIsSuballocation)
	{
		const size_t MemorySize = DescriptorSize * NumDescriptors;
		if (Type == ERHIDescriptorHeapType::Standard)
		{
			DEC_MEMORY_STAT_BY(STAT_BindlessResourceHeapMemory, MemorySize);
		}
		else if (Type == ERHIDescriptorHeapType::Sampler)
		{
			DEC_MEMORY_STAT_BY(STAT_BindlessSamplerHeapMemory, MemorySize);
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12DescriptorManager

FD3D12DescriptorManager::FD3D12DescriptorManager(FD3D12Device* Device, FD3D12DescriptorHeap* InHeap, TConstArrayView<TStatId> InStats)
	: FD3D12DeviceChild(Device)
	, FRHIHeapDescriptorAllocator(InHeap->GetDescriptorTypeMask(), InHeap->GetNumDescriptors(), InStats)
	, Heap(InHeap)
{
}

FD3D12DescriptorManager::~FD3D12DescriptorManager() = default;

FD3D12DescriptorHeap* FD3D12DescriptorManager::AllocateSubheap(uint32 InNumDescriptors)
{
	if (TOptional<FRHIDescriptorAllocation> Allocation = Allocate(InNumDescriptors))
	{
		return new FD3D12DescriptorHeap(this, Allocation->StartIndex, Allocation->Count);
	}

	return nullptr;
}

void FD3D12DescriptorManager::FreeSubheap(FD3D12DescriptorHeap* InHeap)
{
	Free(FRHIDescriptorAllocation(InHeap->GetOffset(), InHeap->GetNumDescriptors()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12OnlineDescriptorManager

FD3D12OnlineDescriptorManager::FD3D12OnlineDescriptorManager(FD3D12Device* Device)
	: FD3D12DeviceChild(Device)
{
}

FD3D12OnlineDescriptorManager::~FD3D12OnlineDescriptorManager()
{
	for (const FD3D12DescriptorHeapPtr& Heap : Heaps)
	{
		checkf(!Heap, TEXT("FD3D12OnlineDescriptorManager destroyed without CleanupResources being called."));
	}
}

// Allocate and initialize the online heap
void FD3D12OnlineDescriptorManager::Init(uint32 InTotalSize, uint32 InBlockSize, bool bBindlessResources)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (bBindlessResources)
	{
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		Heaps = GetParentDevice()->GetBindlessDescriptorManager().AllocateResourceHeapsForAllPipelines(InTotalSize);
		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			if (Heaps[Pipeline])
			{
				INC_DWORD_STAT(STAT_NumViewOnlineDescriptorHeaps);
				INC_MEMORY_STAT_BY(STAT_ViewOnlineDescriptorHeapMemory, Heaps[Pipeline]->GetMemorySize());
			}
		}
#else
		// We won't need CBVs, so don't allocate the heap for them
		return;
#endif
	}
	else
#endif
	{
		FD3D12DescriptorHeapPtr Heap = GetParentDevice()->GetDescriptorHeapManager().AllocateHeap(
			TEXT("Device Global - Online View Heap"),
			ERHIDescriptorHeapType::Standard,
			InTotalSize,
			ED3D12DescriptorHeapFlags::GpuVisible);

		INC_DWORD_STAT(STAT_NumViewOnlineDescriptorHeaps);
		INC_MEMORY_STAT_BY(STAT_ViewOnlineDescriptorHeapMemory, Heap->GetMemorySize());

		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			Heaps[Pipeline] = Heap;
		}
	}

	// Update the stats
	INC_DWORD_STAT_BY(STAT_GlobalViewHeapFreeDescriptors, InTotalSize);

	// Compute amount of free blocks
	uint32 BlockCount = InTotalSize / InBlockSize;

	// Allocate the free blocks
	uint32 CurrentBaseSlot = 0;
	for (uint32 BlockIndex = 0; BlockIndex < BlockCount; ++BlockIndex)
	{
		// Last entry take the rest
		uint32 ActualBlockSize = (BlockIndex == (BlockCount - 1)) ? InTotalSize - CurrentBaseSlot : InBlockSize;
		FreeBlocks.Enqueue(new FD3D12OnlineDescriptorBlock(CurrentBaseSlot, ActualBlockSize));
		CurrentBaseSlot += ActualBlockSize;
	}
}

void FD3D12OnlineDescriptorManager::CleanupResources()
{
	for (FD3D12DescriptorHeapPtr& Heap : Heaps)
	{
		Heap.SafeRelease();
	}
}

// Allocate a new heap block
FD3D12OnlineDescriptorBlock* FD3D12OnlineDescriptorManager::AllocateHeapBlock()
{
	SCOPED_NAMED_EVENT(FD3D12OnlineViewHeap_AllocateHeapBlock, FColor::Silver);

	FScopeLock Lock(&CriticalSection);

	// Free block
	FD3D12OnlineDescriptorBlock* Result = nullptr;
	FreeBlocks.Dequeue(Result);

	if (Result)
	{
		// Update stats
		INC_DWORD_STAT(STAT_GlobalViewHeapBlockAllocations);
		DEC_DWORD_STAT_BY(STAT_GlobalViewHeapFreeDescriptors, Result->Size);
		INC_DWORD_STAT_BY(STAT_GlobalViewHeapReservedDescriptors, Result->Size);
	}

	return Result;
}

// Free given block - can still be used by the GPU
void FD3D12OnlineDescriptorManager::FreeHeapBlock(FD3D12OnlineDescriptorBlock* InHeapBlock)
{
	// Update stats
	DEC_DWORD_STAT_BY(STAT_GlobalViewHeapReservedDescriptors, InHeapBlock->Size);
	INC_DWORD_STAT_BY(STAT_GlobalViewHeapUsedDescriptors, InHeapBlock->SizeUsed);
	INC_DWORD_STAT_BY(STAT_GlobalViewHeapWastedDescriptors, InHeapBlock->Size - InHeapBlock->SizeUsed);

	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(InHeapBlock, this);
}

// Called by the EOP task to recycle blocks
void FD3D12OnlineDescriptorManager::Recycle(FD3D12OnlineDescriptorBlock* Block)
{
	FScopeLock Lock(&CriticalSection);

	// Update stats
	DEC_DWORD_STAT_BY(STAT_GlobalViewHeapUsedDescriptors, Block->SizeUsed);
	DEC_DWORD_STAT_BY(STAT_GlobalViewHeapWastedDescriptors, Block->Size - Block->SizeUsed);
	INC_DWORD_STAT_BY(STAT_GlobalViewHeapFreeDescriptors, Block->Size);

	Block->SizeUsed = 0;
	FreeBlocks.Enqueue(Block);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12OfflineDescriptorManager

static uint32 GetOfflineDescriptorHeapDefaultSize(ERHIDescriptorHeapType InHeapType)
{
	switch (InHeapType)
	{
	default: checkNoEntry();
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	case ERHIDescriptorHeapType::Standard:     return 4096;
#else
	case ERHIDescriptorHeapType::Standard:     return 2048;
#endif
	case ERHIDescriptorHeapType::RenderTarget: return 256;
	case ERHIDescriptorHeapType::DepthStencil: return 256;
	case ERHIDescriptorHeapType::Sampler:      return 128;
	}
}

FD3D12OfflineDescriptorManager::FD3D12OfflineDescriptorManager(FD3D12Device* InDevice, ERHIDescriptorHeapType InHeapType)
	: FD3D12DeviceChild(InDevice)
	, HeapType(InHeapType)
{
	DescriptorSize = GetParentDevice()->GetDevice()->GetDescriptorHandleIncrementSize(Translate(InHeapType));
	NumDescriptorsPerHeap = GetOfflineDescriptorHeapDefaultSize(InHeapType);
}

FD3D12OfflineDescriptorManager::~FD3D12OfflineDescriptorManager() = default;

void FD3D12OfflineDescriptorManager::AllocateHeap()
{
	checkf(NumDescriptorsPerHeap != 0, TEXT("Init() needs to be called before allocating heaps."));
	TRefCountPtr<FD3D12DescriptorHeap> Heap = GetParentDevice()->GetDescriptorHeapManager().AllocateHeap(
		TEXT("FD3D12OfflineDescriptorManager"),
		HeapType,
		NumDescriptorsPerHeap,
		ED3D12DescriptorHeapFlags::None
	);

	D3D12_CPU_DESCRIPTOR_HANDLE HeapBase = Heap->GetCPUSlotHandle(0);
	check(HeapBase.ptr != 0);

	// Allocate and initialize a single new entry in the map
	const uint32 NewHeapIndex = Heaps.Num();

	Heaps.Emplace(Heap, HeapBase, NumDescriptorsPerHeap * DescriptorSize);
	FreeHeaps.AddTail(NewHeapIndex);
}

FD3D12OfflineDescriptor FD3D12OfflineDescriptorManager::AllocateHeapSlot()
{
	FScopeLock Lock(&CriticalSection);
	FD3D12OfflineDescriptor Result;

	if (FreeHeaps.Num() == 0)
	{
		AllocateHeap();
	}

	check(FreeHeaps.Num() != 0);

	auto Head = FreeHeaps.GetHead();
	Result.HeapIndex = Head->GetValue();

	FD3D12OfflineHeapEntry& HeapEntry = Heaps[Result.HeapIndex];
	check(0 != HeapEntry.FreeList.Num());
	FD3D12OfflineHeapFreeRange& Range = HeapEntry.FreeList.GetHead()->GetValue();

	Result.ptr = Range.Start;
	Range.Start += DescriptorSize;

	if (Range.Start == Range.End)
	{
		HeapEntry.FreeList.RemoveNode(HeapEntry.FreeList.GetHead());
		if (HeapEntry.FreeList.Num() == 0)
		{
			FreeHeaps.RemoveNode(Head);
		}
	}

	return Result;
}

void FD3D12OfflineDescriptorManager::FreeHeapSlot(FD3D12OfflineDescriptor& Descriptor)
{
	FScopeLock Lock(&CriticalSection);
	FD3D12OfflineHeapEntry& HeapEntry = Heaps[Descriptor.HeapIndex];

	const FD3D12OfflineHeapFreeRange NewRange{ Descriptor.ptr, Descriptor.ptr + DescriptorSize };

	bool bFound = false;
	for (auto Node = HeapEntry.FreeList.GetHead();
		Node != nullptr && !bFound;
		Node = Node->GetNextNode())
	{
		FD3D12OfflineHeapFreeRange& Range = Node->GetValue();
		check(Range.Start < Range.End);
		if (Range.Start == Descriptor.ptr + DescriptorSize)
		{
			Range.Start = Descriptor.ptr;
			bFound = true;
		}
		else if (Range.End == Descriptor.ptr)
		{
			Range.End += DescriptorSize;
			bFound = true;
		}
		else
		{
			check(Range.End < Descriptor.ptr || Range.Start > Descriptor.ptr);
			if (Range.Start > Descriptor.ptr)
			{
				HeapEntry.FreeList.InsertNode(NewRange, Node);
				bFound = true;
			}
		}
	}

	if (!bFound)
	{
		if (HeapEntry.FreeList.Num() == 0)
		{
			FreeHeaps.AddTail(Descriptor.HeapIndex);
		}
		HeapEntry.FreeList.AddTail(NewRange);
	}

	Descriptor = {};
}

void FD3D12OfflineDescriptorManager::CleanupResources()
{
	for (FD3D12OfflineHeapEntry& HeapEntry : Heaps)
	{
		HeapEntry.Heap.SafeRelease();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FD3D12DescriptorHeapManager

FD3D12DescriptorHeapManager::FD3D12DescriptorHeapManager(FD3D12Device* InDevice)
	: FD3D12DeviceChild(InDevice)
{
}

FD3D12DescriptorHeapManager::~FD3D12DescriptorHeapManager()
{
	Destroy();
}

static FD3D12DescriptorHeap* CreateDescriptorHeap(FD3D12Device* Device, const TCHAR* DebugName, ERHIDescriptorHeapType HeapType, uint32 NumDescriptors, ED3D12DescriptorHeapFlags Flags, bool bIsGlobal)
{
	SCOPED_NAMED_EVENT_F(TEXT("D3D12Device::CreateDescriptorHeap(%s, %d)"), FColor::Turquoise, DebugName, NumDescriptors);

	D3D12_DESCRIPTOR_HEAP_DESC Desc{};
	Desc.Type = Translate(HeapType);
	Desc.NumDescriptors = NumDescriptors;
	Desc.Flags = Translate(Flags);
	Desc.NodeMask = Device->GetGPUMask().GetNative();

	TRefCountPtr<ID3D12DescriptorHeap> Heap;
	VERIFYD3D12RESULT(Device->GetDevice()->CreateDescriptorHeap(&Desc, IID_PPV_ARGS(Heap.GetInitReference())));

	SetD3D12ObjectName(Heap, DebugName);

	return new FD3D12DescriptorHeap(Device, MoveTemp(Heap), NumDescriptors, HeapType, Flags, bIsGlobal);
}

void FD3D12DescriptorHeapManager::Init(uint32 InNumGlobalResourceDescriptors, uint32 InNumGlobalSamplerDescriptors)
{
	if (InNumGlobalResourceDescriptors > 0)
	{
		FD3D12DescriptorHeap* Heap = CreateDescriptorHeap(
			GetParentDevice(),
			TEXT("GlobalResourceHeap"),
			ERHIDescriptorHeapType::Standard,
			InNumGlobalResourceDescriptors,
			ED3D12DescriptorHeapFlags::GpuVisible,
			true /* bIsGlobal */
		);

		const TStatId Stats[] = {GET_STATID(STAT_ResourceDescriptorsAllocated)};
		GlobalHeaps.Emplace(GetParentDevice(), Heap, Stats);
	}

	if (InNumGlobalSamplerDescriptors > 0)
	{
		FD3D12DescriptorHeap* Heap = CreateDescriptorHeap(
			GetParentDevice(),
			TEXT("GlobalSamplerHeap"),
			ERHIDescriptorHeapType::Sampler,
			InNumGlobalSamplerDescriptors,
			ED3D12DescriptorHeapFlags::GpuVisible,
			true /* bIsGlobal */
		);

		const TStatId Stats[] = { GET_STATID(STAT_SamplerDescriptorsAllocated) };
		GlobalHeaps.Emplace(GetParentDevice(), Heap, Stats);
	}
}

void FD3D12DescriptorHeapManager::Destroy()
{
}

TRefCountPtr<ID3D12DescriptorHeap> FD3D12DescriptorHeapManager::AcquirePooledHeap(ERHIDescriptorHeapType InType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InFlags)
{
	FScopeLock Lock(&PooledHeapsCS);
	const int32 Index = PooledHeaps.IndexOfByPredicate([InType, InNumDescriptors, InFlags](const FPooledHeap& PooledHeap) -> bool
		{
			return PooledHeap.Type == InType
				&& PooledHeap.NumDescriptors == InNumDescriptors
				&& PooledHeap.Flags == InFlags;
		});

	if (Index != INDEX_NONE)
	{
		TRefCountPtr<ID3D12DescriptorHeap> Result;
		PooledHeaps[Index].Heap.Swap(Result);
		PooledHeaps.RemoveAt(Index);
		return Result;
	}

	return nullptr;
}

FD3D12DescriptorHeap* FD3D12DescriptorHeapManager::AllocateIndependentHeap(const TCHAR* InDebugName, ERHIDescriptorHeapType InHeapType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InHeapFlags)
{
	if (EnumHasAnyFlags(InHeapFlags, ED3D12DescriptorHeapFlags::Poolable))
	{
		TRefCountPtr<ID3D12DescriptorHeap> PooledHeap = AcquirePooledHeap(InHeapType, InNumDescriptors, InHeapFlags);
		if (PooledHeap.GetReference())
		{
			return new FD3D12DescriptorHeap(GetParentDevice(), MoveTemp(PooledHeap), InNumDescriptors, InHeapType, InHeapFlags, false);
		}
	}

	return CreateDescriptorHeap(GetParentDevice(), InDebugName, InHeapType, InNumDescriptors, InHeapFlags, false);
}

FD3D12DescriptorHeap* FD3D12DescriptorHeapManager::AllocateHeap(const TCHAR* InDebugName, ERHIDescriptorHeapType InHeapType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InHeapFlags)
{
	for (FD3D12DescriptorManager& GlobalHeap : GlobalHeaps)
	{
		if (GlobalHeap.HandlesHeapAllocationsWithFlags(InHeapType, InHeapFlags))
		{
			if (FD3D12DescriptorHeap* NewHeap = GlobalHeap.AllocateSubheap(InNumDescriptors))
			{
				return NewHeap;
			}

			// TODO: handle running out of space
			//checkNoEntry();
		}
	}

	return AllocateIndependentHeap(InDebugName, InHeapType, InNumDescriptors, InHeapFlags);
}

void FD3D12DescriptorHeapManager::DeferredFreeHeap(FD3D12DescriptorHeap* InHeap)
{
	InHeap->AddRef();
	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(InHeap, FD3D12DeferredDeleteObject::EType::DescriptorHeap);
}

void FD3D12DescriptorHeapManager::AddHeapToPool(TRefCountPtr<ID3D12DescriptorHeap>&& InHeap, ERHIDescriptorHeapType InType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InFlags)
{
	FScopeLock Lock(&PooledHeapsCS);
	PooledHeaps.Emplace(Forward<TRefCountPtr<ID3D12DescriptorHeap>>(InHeap), InType, InNumDescriptors, InFlags);
}
