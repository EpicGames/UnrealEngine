// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDescriptorAllocator.h"
#include "Containers/List.h"
#include "Containers/Queue.h"
#include "D3D12RHI.h"
#include "D3D12RHICommon.h"
#include "MultiGPU.h"
#include "RHIPipeline.h"
#include "Templates/RefCounting.h"

class FD3D12DescriptorManager;
class FD3D12View;

struct FD3D12DescriptorHeap;
struct FD3D12OfflineDescriptor;

inline D3D12_DESCRIPTOR_HEAP_TYPE Translate(ERHIDescriptorHeapType InHeapType)
{
	switch (InHeapType)
	{
	default: checkNoEntry();
	case ERHIDescriptorHeapType::Standard:     return D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV;
	case ERHIDescriptorHeapType::RenderTarget: return D3D12_DESCRIPTOR_HEAP_TYPE_RTV;
	case ERHIDescriptorHeapType::DepthStencil: return D3D12_DESCRIPTOR_HEAP_TYPE_DSV;
	case ERHIDescriptorHeapType::Sampler:      return FD3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER;
	}
}

inline D3D12_DESCRIPTOR_HEAP_TYPE Translate(ERHIDescriptorType InDescriptorType)
{
	const ERHIDescriptorTypeMask TypeMask = RHIDescriptorTypeMaskFromType(InDescriptorType);
	if (EnumHasAnyFlags(TypeMask, ERHIDescriptorTypeMask::CBV | ERHIDescriptorTypeMask::SRV | ERHIDescriptorTypeMask::UAV))
	{
		return D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV;
	}

	if (EnumHasAnyFlags(TypeMask, ERHIDescriptorTypeMask::Sampler))
	{
		return FD3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER;
	}

	checkNoEntry();
	return D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV;
}

inline ERHIDescriptorTypeMask D3D12DescriptorTypeMaskFromHeapType(ERHIDescriptorHeapType InHeapType)
{
	if (InHeapType == ERHIDescriptorHeapType::Standard)
	{
		return ERHIDescriptorTypeMask::CBV | ERHIDescriptorTypeMask::SRV | ERHIDescriptorTypeMask::UAV;
	}

	if (InHeapType == ERHIDescriptorHeapType::Sampler)
	{
		return ERHIDescriptorTypeMask::Sampler;
	}

	return ERHIDescriptorTypeMask::None;
}

enum class ED3D12DescriptorHeapFlags : uint8
{
	None       = 0,
	GpuVisible = 1 << 0,
	Poolable   = 1 << 1,
};
ENUM_CLASS_FLAGS(ED3D12DescriptorHeapFlags)

inline D3D12_DESCRIPTOR_HEAP_FLAGS Translate(ED3D12DescriptorHeapFlags InHeapFlags)
{
	if (EnumHasAnyFlags(InHeapFlags, ED3D12DescriptorHeapFlags::GpuVisible))
	{
		return D3D12_DESCRIPTOR_HEAP_FLAG_SHADER_VISIBLE;
	}

	return D3D12_DESCRIPTOR_HEAP_FLAG_NONE;
}

using FD3D12DescriptorIndex = uint32;

namespace UE::D3D12Descriptors
{
	FD3D12DescriptorHeap* CreateDescriptorHeap(FD3D12Device* Device, const TCHAR* DebugName, ERHIDescriptorHeapType HeapType, uint32 NumDescriptors, ED3D12DescriptorHeapFlags Flags, bool bGlobal = false);

	void CopyDescriptor(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorIndex DstIndex, D3D12_CPU_DESCRIPTOR_HANDLE SrcCpuHandle);
	void CopyDescriptor(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FRHIDescriptorHandle DstHandle, D3D12_CPU_DESCRIPTOR_HANDLE SrcCpuHandle);

	void CopyDescriptors(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorHeap* SourceHeap, uint32 FirstDescriptor, uint32 NumDescriptors);
	void CopyDescriptors(FD3D12Device* Device, FD3D12DescriptorHeap* TargetHeap, FD3D12DescriptorHeap* SourceHeap, TConstArrayView<FD3D12DescriptorIndex> Indices);

	FD3D12OfflineDescriptor CreateOfflineCopy(FD3D12Device* Device, D3D12_CPU_DESCRIPTOR_HANDLE InDescriptor, ERHIDescriptorHeapType InType);
	FD3D12OfflineDescriptor CreateOfflineCopy(FD3D12Device* Device, FD3D12DescriptorHeap* InHeap, FRHIDescriptorHandle InHandle);
	void FreeOfflineCopy(FD3D12Device* Device, FD3D12OfflineDescriptor& InDescriptor, ERHIDescriptorHeapType InType);
}

struct FD3D12DescriptorHeap : public FD3D12DeviceChild, public FRefCountedObject
{
public:
	FD3D12DescriptorHeap() = delete;

	// Heap created with its own D3D heap object.
	FD3D12DescriptorHeap(FD3D12Device* InDevice, TRefCountPtr<ID3D12DescriptorHeap>&& InHeap, uint32 InNumDescriptors, ERHIDescriptorHeapType InType, ED3D12DescriptorHeapFlags InFlags, bool bInIsGlobal);

	// Heap created as a suballocation of another heap.
	FD3D12DescriptorHeap(FD3D12DescriptorHeap* SubAllocateSourceHeap, uint32 InOffset, uint32 InNumDescriptors);

	// Heap created as an allocation from a descriptor manager.
	FD3D12DescriptorHeap(FD3D12DescriptorManager* Manager, uint32 InOffset, uint32 InNumDescriptors);

	~FD3D12DescriptorHeap();

	ID3D12DescriptorHeap*       GetHeap()  const { return Heap; }
	ERHIDescriptorHeapType      GetType()  const { return Type; }
	ED3D12DescriptorHeapFlags   GetFlags() const { return Flags; }

	ERHIDescriptorTypeMask GetDescriptorTypeMask() const
	{
		return D3D12DescriptorTypeMaskFromHeapType(GetType());
	}

	uint32 GetOffset()         const { return Offset; }
	uint32 GetNumDescriptors() const { return NumDescriptors; }
	uint32 GetDescriptorSize() const { return DescriptorSize; }
	bool   IsGlobal()          const { return bIsGlobal; }
	bool   IsSuballocation()   const { return bIsSuballocation; }

	uint32 GetMemorySize() const { return DescriptorSize * NumDescriptors; }

	D3D12_CPU_DESCRIPTOR_HANDLE GetCPUSlotHandle(uint32 Slot) const { return CD3DX12_CPU_DESCRIPTOR_HANDLE(CpuBase, Slot, DescriptorSize); }
	D3D12_GPU_DESCRIPTOR_HANDLE GetGPUSlotHandle(uint32 Slot) const { return CD3DX12_GPU_DESCRIPTOR_HANDLE(GpuBase, Slot, DescriptorSize); }

	bool CanBePooled() const { return EnumHasAnyFlags(GetFlags(), ED3D12DescriptorHeapFlags::Poolable); }

private:
	TRefCountPtr<ID3D12DescriptorHeap> Heap;

	// Manager this heap is a suballocation of, if any.
	FD3D12DescriptorManager* Manager = nullptr;

	const CD3DX12_CPU_DESCRIPTOR_HANDLE CpuBase;
	const CD3DX12_GPU_DESCRIPTOR_HANDLE GpuBase;

	// Offset in descriptors into the heap, only used when heap is suballocated.
	const uint32 Offset = 0;

	// Total number of descriptors in this heap.
	const uint32 NumDescriptors;

	// Device provided size of each descriptor in this heap.
	const uint32 DescriptorSize;

	const ERHIDescriptorHeapType Type;
	const ED3D12DescriptorHeapFlags Flags;

	// Enabled if this heap is the "global" heap.
	const bool bIsGlobal;

	// Enabled if this heap was allocated inside another heap.
	const bool bIsSuballocation;
};
using FD3D12DescriptorHeapPtr = TRefCountPtr<FD3D12DescriptorHeap>;

/** Manager for resource descriptor allocations. */
class FD3D12DescriptorManager : public FD3D12DeviceChild, public FRHIHeapDescriptorAllocator
{
public:
	FD3D12DescriptorManager() = delete;
	FD3D12DescriptorManager(FD3D12Device* Device, FD3D12DescriptorHeap* InHeap, TConstArrayView<TStatId> InStats);
	~FD3D12DescriptorManager();

	inline       FD3D12DescriptorHeap* GetHeap()       { return Heap.GetReference(); }
	inline const FD3D12DescriptorHeap* GetHeap() const { return Heap.GetReference(); }

	FD3D12DescriptorHeap* AllocateSubheap(uint32 InNumDescriptors);
	void FreeSubheap(FD3D12DescriptorHeap* InHeap);

	bool HandlesHeapAllocationsWithFlags(ERHIDescriptorHeapType InHeapType, ED3D12DescriptorHeapFlags InHeapFlags) const
	{
		return HandlesAllocations(D3D12DescriptorTypeMaskFromHeapType(InHeapType)) && Heap->GetFlags() == InHeapFlags;
	}

	bool HandlesAllocationWithFlags(ERHIDescriptorType InDescriptorType, ED3D12DescriptorHeapFlags InHeapFlags) const
	{
		return HandlesAllocation(InDescriptorType) && Heap->GetFlags() == InHeapFlags;
	}

	inline bool IsHeapAChild(const FD3D12DescriptorHeap* InHeap)
	{
		return InHeap->GetHeap() == Heap->GetHeap();
	}

private:
	FD3D12DescriptorHeapPtr Heap;
};

/** Heap sub block of an online heap */
struct FD3D12OnlineDescriptorBlock
{
	FD3D12OnlineDescriptorBlock() = delete;
	FD3D12OnlineDescriptorBlock(uint32 InBaseSlot, uint32 InSize)
		: BaseSlot(InBaseSlot)
		, Size(InSize)
	{
	}

	uint32 BaseSlot;
	uint32 Size;
	uint32 SizeUsed = 0;
};

/** Primary online heap from which sub blocks can be allocated and freed. Used when allocating blocks of descriptors for tables. */
class FD3D12OnlineDescriptorManager : public FD3D12DeviceChild
{
public:
	FD3D12OnlineDescriptorManager(FD3D12Device* Device);
	~FD3D12OnlineDescriptorManager();

	// Setup the actual heap
	void Init(uint32 InTotalSize, uint32 InBlockSize, bool bBindlessResources);
	void CleanupResources();

	// Allocate an available sub heap block from the global heap
	FD3D12OnlineDescriptorBlock* AllocateHeapBlock();
	void FreeHeapBlock(FD3D12OnlineDescriptorBlock* InHeapBlock);

	ID3D12DescriptorHeap* GetHeap(ERHIPipeline Pipeline) { return Heaps[Pipeline]->GetHeap(); }
	FD3D12DescriptorHeap* GetDescriptorHeap(ERHIPipeline Pipeline) { return Heaps[Pipeline].GetReference(); }

	D3D12_CPU_DESCRIPTOR_HANDLE GetCPUSlotHandle(ERHIPipeline Pipeline, FD3D12OnlineDescriptorBlock* InBlock) const { return Heaps[Pipeline]->GetCPUSlotHandle(InBlock->BaseSlot); }
	D3D12_GPU_DESCRIPTOR_HANDLE GetGPUSlotHandle(ERHIPipeline Pipeline, FD3D12OnlineDescriptorBlock* InBlock) const { return Heaps[Pipeline]->GetGPUSlotHandle(InBlock->BaseSlot); }
	
	// Called by the EOP task to recycle blocks
	void Recycle(FD3D12OnlineDescriptorBlock* Block);

private:
	TRHIPipelineArray<FD3D12DescriptorHeapPtr> Heaps;

	TQueue<FD3D12OnlineDescriptorBlock*> FreeBlocks;

	FCriticalSection CriticalSection;
};

struct FD3D12OfflineHeapFreeRange
{
	size_t Start;
	size_t End;
};

struct FD3D12OfflineHeapEntry
{
	FD3D12OfflineHeapEntry(FD3D12DescriptorHeap* InHeap, const D3D12_CPU_DESCRIPTOR_HANDLE& InHeapBase, size_t InSize)
		: Heap(InHeap)
	{
		FreeList.AddTail({ InHeapBase.ptr, InHeapBase.ptr + InSize });
	}

	TRefCountPtr<FD3D12DescriptorHeap> Heap;
	TDoubleLinkedList<FD3D12OfflineHeapFreeRange> FreeList;
};

struct FD3D12OfflineDescriptor : public D3D12_CPU_DESCRIPTOR_HANDLE
{
public:
	operator bool () const { return ptr != 0; }
	FD3D12OfflineDescriptor() { ptr = 0; }

	// Descriptor version can be used to invalidate any caches that are based on D3D12_CPU_DESCRIPTOR_HANDLE,
	// for example to detect when a view has been updated following a resource rename.
	void IncrementVersion() { ++Version; }
	uint32 GetVersion() const { return Version; }

private:
	friend class FD3D12OfflineDescriptorManager;
	uint32 HeapIndex = 0;
	uint32 Version = 0;
};

/** Manages and allows allocations of CPU descriptors only. Creates small heaps on demand to satisfy allocations. */
class FD3D12OfflineDescriptorManager : public FD3D12DeviceChild
{
public:
	FD3D12OfflineDescriptorManager() = delete;
	FD3D12OfflineDescriptorManager(FD3D12Device* InDevice, ERHIDescriptorHeapType InHeapType);
	~FD3D12OfflineDescriptorManager();

	inline ERHIDescriptorHeapType GetHeapType() const { return HeapType; }

	FD3D12OfflineDescriptor AllocateHeapSlot();
	void FreeHeapSlot(FD3D12OfflineDescriptor& Descriptor);

	void CleanupResources();

private:
	void AllocateHeap();

	TArray<FD3D12OfflineHeapEntry> Heaps;
	TDoubleLinkedList<uint32> FreeHeaps;

	ERHIDescriptorHeapType HeapType;
	uint32 NumDescriptorsPerHeap{};
	uint32 DescriptorSize{};

	FCriticalSection CriticalSection;
};

/** Primary descriptor heap and descriptor manager. All heap allocations come from here.
	All GPU visible resource heap allocations will be sub-allocated from a single heap in this manager. */
class FD3D12DescriptorHeapManager : public FD3D12DeviceChild
{
public:
	FD3D12DescriptorHeapManager(FD3D12Device* InDevice);
	~FD3D12DescriptorHeapManager();

	void Init(uint32 InNumGlobalResourceDescriptors, uint32 InNumGlobalSamplerDescriptors);
	void Destroy();

	FD3D12DescriptorHeap* AllocateIndependentHeap(const TCHAR* InDebugName, ERHIDescriptorHeapType InHeapType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InHeapFlags);
	FD3D12DescriptorHeap* AllocateHeap(const TCHAR* InDebugName, ERHIDescriptorHeapType InHeapType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InHeapFlags);
	void DeferredFreeHeap(FD3D12DescriptorHeap* InHeap);

	void AddHeapToPool(TRefCountPtr<ID3D12DescriptorHeap>&& InHeap, ERHIDescriptorHeapType InType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InFlags);

private:
	TRefCountPtr<ID3D12DescriptorHeap> AcquirePooledHeap(ERHIDescriptorHeapType InType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InFlags);

	TArray<FD3D12DescriptorManager, TFixedAllocator<2>> GlobalHeaps; // Use TFixedAllocator to prevent move of CRITICAL_SECTION

	struct FPooledHeap
	{
		FPooledHeap(TRefCountPtr<ID3D12DescriptorHeap>&& InHeap, ERHIDescriptorHeapType InType, uint32 InNumDescriptors, ED3D12DescriptorHeapFlags InFlags)
			: Heap(Forward<TRefCountPtr<ID3D12DescriptorHeap>>(InHeap))
			, NumDescriptors(InNumDescriptors)
			, Type(InType)
			, Flags(InFlags)
		{
		}

		TRefCountPtr<ID3D12DescriptorHeap> Heap;
		uint32                             NumDescriptors;
		ERHIDescriptorHeapType             Type;
		ED3D12DescriptorHeapFlags          Flags;
	};
	TArray<FPooledHeap> PooledHeaps;
	FCriticalSection PooledHeapsCS;
};
