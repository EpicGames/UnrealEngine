// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12Device.h"
#include "D3D12RHIPrivate.h"
#include "D3D12IntelExtensions.h"
#include "D3D12RayTracing.h"
#include "D3D12ExplicitDescriptorCache.h"
#include "D3D12Tracing.h"
#include "HAL/Platform.h"

static TAutoConsoleVariable<int32> CVarD3D12GPUTimeout(
	TEXT("r.D3D12.GPUTimeout"),
	1,
	TEXT("0: Disable GPU Timeout; use with care as it could freeze your PC!\n")
	TEXT("1: Enable GPU Timeout; operation taking long on the GPU will fail(default)\n"),
	ECVF_ReadOnly
);

TAutoConsoleVariable<int32> CVarD3D12ExtraDiagnosticBufferMemory(
	TEXT("r.D3D12.DiagnosticBufferExtraMemory"),
	0,
	TEXT("Extra allocated memory for diagnostic buffer"),
	ECVF_ReadOnly
);

static uint32 GetQueryHeapPoolIndex(D3D12_QUERY_HEAP_TYPE HeapType)
{
	switch (HeapType)
	{
	default: checkNoEntry(); [[fallthrough]];
	case D3D12_QUERY_HEAP_TYPE_OCCLUSION:            return 0;
	case D3D12_QUERY_HEAP_TYPE_TIMESTAMP:            return 1;
	case D3D12_QUERY_HEAP_TYPE_COPY_QUEUE_TIMESTAMP: return 2;
	case D3D12_QUERY_HEAP_TYPE_PIPELINE_STATISTICS:  return 3;
	}
}

FD3D12Timing::FD3D12Timing(FD3D12Queue& Queue)
	: Queue(Queue)
	, EventStream(Queue.GetProfilerQueue())
{}

FD3D12Queue::FD3D12Queue(FD3D12Device* Device, ED3D12QueueType QueueType, int32 QueueIndex)
	: Device(Device)
	, QueueType(QueueType)
	, QueueIndex(QueueIndex)
	, Fence(this)
	, BarrierTimestamps(Device, QueueType, D3D12_QUERY_TYPE_TIMESTAMP)
	, bSupportsTileMapping(FD3D12DynamicRHI::GetD3DRHI()->QueueSupportsTileMapping(QueueType))
	, ExecuteCommandListsFence(this)
{
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	D3D12_COMMAND_QUEUE_DESC CommandQueueDesc = {};
	CommandQueueDesc.Type = GetD3DCommandListType((ED3D12QueueType)QueueType);
	CommandQueueDesc.Priority = 0;
	CommandQueueDesc.NodeMask = Device->GetGPUMask().GetNative();
	CommandQueueDesc.Flags = (CVarD3D12GPUTimeout.GetValueOnAnyThread() == 0)
		? D3D12_COMMAND_QUEUE_FLAG_DISABLE_GPU_TIMEOUT
		: D3D12_COMMAND_QUEUE_FLAG_NONE;

	FD3D12DynamicRHI::GetD3DRHI()->CreateCommandQueue(Device, CommandQueueDesc, D3DCommandQueue);
	D3DCommandQueue->SetName(*FString::Printf(TEXT("%s Queue (GPU %d)"), GetD3DCommandQueueTypeName(QueueType), Device->GetGPUIndex()));

	VERIFYD3D12RESULT(Device->GetDevice()->CreateFence(
		0,
		D3D12_FENCE_FLAG_NONE,
		IID_PPV_ARGS(Fence.D3DFence.GetInitReference())
	));
	Fence.D3DFence->SetName(*FString::Printf(TEXT("%s Queue Fence (GPU %d)"), GetD3DCommandQueueTypeName(QueueType), Device->GetGPUIndex()));


	VERIFYD3D12RESULT(Device->GetDevice()->CreateFence(
		0,
		D3D12_FENCE_FLAG_NONE,
		IID_PPV_ARGS(ExecuteCommandListsFence.D3DFence.GetInitReference())
	));
	ExecuteCommandListsFence.D3DFence->SetName(*FString::Printf(TEXT("%s ExecuteCommandListsFence (GPU %d)"), GetD3DCommandQueueTypeName(QueueType), Device->GetGPUIndex()));
}

FD3D12Queue::~FD3D12Queue()
{
	check(PendingSubmission.IsEmpty());
	check(PendingInterrupt.IsEmpty());
}

UE::RHI::GPUProfiler::FQueue FD3D12Queue::GetProfilerQueue() const
{
	UE::RHI::GPUProfiler::FQueue Queue;
	Queue.GPU = Device->GetGPUIndex();
	Queue.Index = 0;

	switch (QueueType)
	{
	default: checkNoEntry(); [[fallthrough]];
	case ED3D12QueueType::Direct: Queue.Type = UE::RHI::GPUProfiler::FQueue::EType::Graphics; break;
	case ED3D12QueueType::Async : Queue.Type = UE::RHI::GPUProfiler::FQueue::EType::Compute ; break;
	case ED3D12QueueType::Copy  : Queue.Type = UE::RHI::GPUProfiler::FQueue::EType::Copy    ; break;
	}

	return Queue;
}

FD3D12Device::FD3D12Device(FRHIGPUMask InGPUMask, FD3D12Adapter* InAdapter)
	: FD3D12SingleNodeGPUObject(InGPUMask)
	, FD3D12AdapterChild       (InAdapter)
	, TileMappingFence         (nullptr)
	, ResidencyManager         (*this)
	, DescriptorHeapManager    (this)
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, BindlessDescriptorAllocator(InAdapter->GetBindlessDescriptorAllocator())
	, BindlessDescriptorManager(this, InAdapter->GetBindlessDescriptorAllocator())
#endif
	, GlobalSamplerHeap        (this)
	, OnlineDescriptorManager  (this)
	, SamplerCache			(D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE)
	, DefaultBufferAllocator   (this, FRHIGPUMask::All()) //Note: Cross node buffers are possible 
	, DefaultFastAllocator     (this, FRHIGPUMask::All(), D3D12_HEAP_TYPE_UPLOAD, 1024 * 1024 * 4)
	, TextureAllocator         (this, FRHIGPUMask::All())
{
	check(IsInGameThread());

	for (uint32 HeapType = 0; HeapType < (uint32)ERHIDescriptorHeapType::Count; ++HeapType)
	{
		OfflineDescriptorManagers.Emplace(this, (ERHIDescriptorHeapType)HeapType);
	}

	for (uint32 QueueIndex = 0; QueueIndex < (uint32)ED3D12QueueType::Count; ++QueueIndex)
	{
		Queues.Emplace(this, (ED3D12QueueType)QueueIndex, QueueIndex);
	}

	// Some hardware is not capable of running tile mapping operations on all queue types.
	// Direct queue is used as a fallback if tile updates are requested on unsupported queue.
	TileMappingQueue = Queues[size_t(ED3D12QueueType::Direct)].D3DCommandQueue;
	VERIFYD3D12RESULT(GetDevice()->CreateFence(
		0,
		D3D12_FENCE_FLAG_NONE,
		IID_PPV_ARGS(TileMappingFence.D3DFence.GetInitReference())
	));
#if NAME_OBJECTS
	TileMappingFence.D3DFence->SetName(TEXT("TileMappingFence"));
#endif
}

FD3D12Device::~FD3D12Device()
{
#if D3D12_RHI_RAYTRACING
	delete RayTracingCompactionRequestHandler;
	RayTracingCompactionRequestHandler = nullptr;
#endif

	DestroyExplicitDescriptorCache(); // #dxr_todo UE-72158: unify RT descriptor cache with main FD3D12DescriptorCache

	// Cleanup the allocator near the end, as some resources may be returned to the allocator or references are shared by multiple GPUs
	DefaultBufferAllocator.FreeDefaultBufferPools();

	DefaultFastAllocator.Destroy();

	TextureAllocator.CleanUpAllocations();
	TextureAllocator.Destroy();

	SamplerCache.Empty();
}

FD3D12Device::FResidencyManager::FResidencyManager(FD3D12Device& Parent)
{
#if ENABLE_RESIDENCY_MANAGEMENT
	IDXGIAdapter3* DxgiAdapter3 = nullptr;
	VERIFYD3D12RESULT(Parent.GetParentAdapter()->GetAdapter()->QueryInterface(IID_PPV_ARGS(&DxgiAdapter3)));
	const uint32 ResidencyMangerGPUIndex = GVirtualMGPU ? 0 : Parent.GetGPUIndex(); // GPU node index is used by residency manager to query budget
	D3DX12Residency::InitializeResidencyManager(*this, Parent.GetDevice(), ResidencyMangerGPUIndex, DxgiAdapter3, RESIDENCY_PIPELINE_DEPTH);
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

FD3D12Device::FResidencyManager::~FResidencyManager()
{
#if ENABLE_RESIDENCY_MANAGEMENT
	D3DX12Residency::DestroyResidencyManager(*this);
#endif
}

ID3D12Device* FD3D12Device::GetDevice()
{
	return GetParentAdapter()->GetD3DDevice();
}

#if D3D12_SUPPORTS_DXGI_DEBUG
typedef HRESULT(WINAPI *FDXGIGetDebugInterface1)(UINT, REFIID, void **);
#endif

static D3D12_FEATURE_DATA_FORMAT_SUPPORT GetFormatSupport(ID3D12Device* InDevice, DXGI_FORMAT InFormat)
{
	D3D12_FEATURE_DATA_FORMAT_SUPPORT FormatSupport{};
	FormatSupport.Format = InFormat;

	InDevice->CheckFeatureSupport(D3D12_FEATURE_FORMAT_SUPPORT, &FormatSupport, sizeof(FormatSupport));

	return FormatSupport;
}

void FD3D12Device::SetupAfterDeviceCreation()
{
	ID3D12Device* Direct3DDevice = GetParentAdapter()->GetD3DDevice();

	for (uint32 FormatIndex = PF_Unknown; FormatIndex < PF_MAX; FormatIndex++)
	{
		FPixelFormatInfo& PixelFormatInfo = GPixelFormats[FormatIndex];
		const DXGI_FORMAT PlatformFormat = static_cast<DXGI_FORMAT>(PixelFormatInfo.PlatformFormat);

		EPixelFormatCapabilities Capabilities = EPixelFormatCapabilities::None;

		if (PlatformFormat != DXGI_FORMAT_UNKNOWN)
		{
			const D3D12_FEATURE_DATA_FORMAT_SUPPORT FormatSupport    = GetFormatSupport(Direct3DDevice, PlatformFormat);
			const D3D12_FEATURE_DATA_FORMAT_SUPPORT SRVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, false));
			const D3D12_FEATURE_DATA_FORMAT_SUPPORT UAVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindUnorderedAccessFormat(PlatformFormat));
			const D3D12_FEATURE_DATA_FORMAT_SUPPORT RTVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, false));
			const D3D12_FEATURE_DATA_FORMAT_SUPPORT DSVFormatSupport = GetFormatSupport(Direct3DDevice, UE::DXGIUtilities::FindDepthStencilFormat(PlatformFormat));

			auto ConvertCap1 = [&Capabilities](const D3D12_FEATURE_DATA_FORMAT_SUPPORT& InSupport, EPixelFormatCapabilities UnrealCap, D3D12_FORMAT_SUPPORT1 InFlags)
			{
				if (EnumHasAnyFlags(InSupport.Support1, InFlags))
				{
					EnumAddFlags(Capabilities, UnrealCap);
				}
			};
			auto ConvertCap2 = [&Capabilities](const D3D12_FEATURE_DATA_FORMAT_SUPPORT& InSupport, EPixelFormatCapabilities UnrealCap, D3D12_FORMAT_SUPPORT2 InFlags)
			{
				if (EnumHasAnyFlags(InSupport.Support2, InFlags))
				{
					EnumAddFlags(Capabilities, UnrealCap);
				}
			};

			ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture1D,               D3D12_FORMAT_SUPPORT1_TEXTURE1D);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture2D,               D3D12_FORMAT_SUPPORT1_TEXTURE2D);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::Texture3D,               D3D12_FORMAT_SUPPORT1_TEXTURE3D);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::TextureCube,             D3D12_FORMAT_SUPPORT1_TEXTURECUBE);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::Buffer,                  D3D12_FORMAT_SUPPORT1_BUFFER);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::VertexBuffer,            D3D12_FORMAT_SUPPORT1_IA_VERTEX_BUFFER);
			ConvertCap1(FormatSupport, EPixelFormatCapabilities::IndexBuffer,             D3D12_FORMAT_SUPPORT1_IA_INDEX_BUFFER);

			if (EnumHasAnyFlags(Capabilities, EPixelFormatCapabilities::AnyTexture))
			{
				ConvertCap1(FormatSupport, EPixelFormatCapabilities::RenderTarget,        D3D12_FORMAT_SUPPORT1_RENDER_TARGET);
				ConvertCap1(FormatSupport, EPixelFormatCapabilities::DepthStencil,        D3D12_FORMAT_SUPPORT1_DEPTH_STENCIL);
				ConvertCap1(FormatSupport, EPixelFormatCapabilities::TextureMipmaps,      D3D12_FORMAT_SUPPORT1_MIP);
				ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureLoad,      D3D12_FORMAT_SUPPORT1_SHADER_LOAD);
				ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureSample | EPixelFormatCapabilities::TextureFilterable,    D3D12_FORMAT_SUPPORT1_SHADER_SAMPLE);
				ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureGather,    D3D12_FORMAT_SUPPORT1_SHADER_GATHER);
				ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TextureAtomics,   D3D12_FORMAT_SUPPORT2_UAV_ATOMIC_EXCHANGE);
				ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::TextureBlendable, D3D12_FORMAT_SUPPORT1_BLENDABLE);
				ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TextureStore,     D3D12_FORMAT_SUPPORT2_UAV_TYPED_STORE);
			}

			if (EnumHasAnyFlags(Capabilities, EPixelFormatCapabilities::Buffer))
			{
				ConvertCap1(SRVFormatSupport, EPixelFormatCapabilities::BufferLoad,       D3D12_FORMAT_SUPPORT1_BUFFER);
				ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::BufferStore,      D3D12_FORMAT_SUPPORT2_UAV_TYPED_STORE);
				ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::BufferAtomics,    D3D12_FORMAT_SUPPORT2_UAV_ATOMIC_EXCHANGE);
			}

			ConvertCap1(UAVFormatSupport, EPixelFormatCapabilities::UAV,                  D3D12_FORMAT_SUPPORT1_TYPED_UNORDERED_ACCESS_VIEW);
			ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TypedUAVLoad,         D3D12_FORMAT_SUPPORT2_UAV_TYPED_LOAD);
			ConvertCap2(UAVFormatSupport, EPixelFormatCapabilities::TypedUAVStore,        D3D12_FORMAT_SUPPORT2_UAV_TYPED_STORE);
		}

		PixelFormatInfo.Capabilities = Capabilities;
	}

	GRHISupportsArrayIndexFromAnyShader = true;
	GRHISupportsStencilRefFromPixelShader = false; // TODO: Sort out DXC shader database SM6.0 usage. DX12 supports this feature, but need to improve DXC support.

#if PLATFORM_WINDOWS
	// Check if we're running under GPU capture
	bool bUnderGPUCapture = false;

	// RenderDoc
	if (D3D12RHI_IsRenderDocPresent(Direct3DDevice))
	{
		// Running under RenderDoc, so enable capturing mode
		bUnderGPUCapture = true;
	}

	// Intel GPA
	{
		TRefCountPtr<IUnknown> IntelGPA;
		static const IID IntelGPAID = { 0xCCFFEF16, 0x7B69, 0x468F, {0xBC, 0xE3, 0xCD, 0x95, 0x33, 0x69, 0xA3, 0x9A} };

		if (SUCCEEDED(Direct3DDevice->QueryInterface(IntelGPAID, (void**)(IntelGPA.GetInitReference()))))
		{
			// Running under Intel GPA, so enable capturing mode
			bUnderGPUCapture = true;
		}
	}

	// AMD RGP profiler
	if (GEmitRgpFrameMarkers && FD3D12DynamicRHI::GetD3DRHI()->GetAmdAgsContext())
	{
		// Running on AMD with RGP profiling enabled, so enable capturing mode
		bUnderGPUCapture = true;
	}

#if USE_PIX
	// PIX (note that DXGIGetDebugInterface1 requires Windows 8.1 and up)
	if (FPlatformMisc::VerifyWindowsVersion(6, 3))
	{
		FDXGIGetDebugInterface1 DXGIGetDebugInterface1FnPtr = nullptr;

		// CreateDXGIFactory2 is only available on Win8.1+, find it if it exists
		HMODULE DxgiDLL = LoadLibraryA("dxgi.dll");
		if (DxgiDLL)
		{
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			DXGIGetDebugInterface1FnPtr = (FDXGIGetDebugInterface1)(GetProcAddress(DxgiDLL, "DXGIGetDebugInterface1"));
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			FreeLibrary(DxgiDLL);
		}
		
		if (DXGIGetDebugInterface1FnPtr)
		{
			IID GraphicsAnalysisID;
			if (SUCCEEDED(IIDFromString(L"{9F251514-9D4D-4902-9D60-18988AB7D4B5}", &GraphicsAnalysisID)))
			{
				TRefCountPtr<IUnknown> GraphicsAnalysis;
				if (SUCCEEDED(DXGIGetDebugInterface1FnPtr(0, GraphicsAnalysisID, (void**)GraphicsAnalysis.GetInitReference())))
				{
					// Running under PIX, so enable capturing mode
					bUnderGPUCapture = true;
				}
			}
		}
	}
#endif // USE_PIX

	if (bUnderGPUCapture)
	{
		FDynamicRHI::EnableIdealGPUCaptureOptions(true);
	}
#endif // PLATFORM_WINDOWS


	const int32 MaximumResourceHeapSize = GetParentAdapter()->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Standard);
	const int32 MaximumSamplerHeapSize = GetParentAdapter()->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Sampler);

	// This value can be tuned on a per app basis. I.e. most apps will never run into descriptor heap pressure so
	// can make this global heap smaller
	check(GGlobalResourceDescriptorHeapSize <= MaximumResourceHeapSize || MaximumResourceHeapSize < 0);
	check(GGlobalSamplerDescriptorHeapSize <= MaximumSamplerHeapSize);

	check(GGlobalSamplerHeapSize <= MaximumSamplerHeapSize);

	check(GOnlineDescriptorHeapSize <= GGlobalResourceDescriptorHeapSize);

	bool bFullyBindless = false;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	BindlessDescriptorManager.Init();

	bFullyBindless = IsBindlessFullyEnabled(BindlessDescriptorManager.GetConfiguration());
#endif

	DescriptorHeapManager.Init(bFullyBindless ? 0 : GGlobalResourceDescriptorHeapSize, bFullyBindless ? 0 : GGlobalSamplerDescriptorHeapSize);

	if (!bFullyBindless)
	{
		GlobalSamplerHeap.Init(GGlobalSamplerHeapSize);
	}

	{
		const uint32 HeapSize = bFullyBindless ? GBindlessOnlineDescriptorHeapSize : GOnlineDescriptorHeapSize;
		const uint32 BlockSize = bFullyBindless ? GBindlessOnlineDescriptorHeapBlockSize : GOnlineDescriptorHeapBlockSize;

		OnlineDescriptorManager.Init(HeapSize, BlockSize, bFullyBindless);
	}

	// Make sure we create the default views before the first command context
	CreateDefaultViews();

	// Needs to be called before creating command contexts
	UpdateConstantBufferPageProperties();

	UpdateMSAASettings();

#if D3D12_RHI_RAYTRACING
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6)
	{
		check(RayTracingCompactionRequestHandler == nullptr);
		RayTracingCompactionRequestHandler = new FD3D12RayTracingCompactionRequestHandler(this);
		for (FD3D12Queue& Queue : Queues)
		{
			const size_t DispatchRaysSize = sizeof(D3D12_DISPATCH_RAYS_DESC);

			const D3D12_RESOURCE_DESC DispatchRaysBufferDesc = CD3DX12_RESOURCE_DESC::Buffer(DispatchRaysSize, D3D12RHI_RESOURCE_FLAG_ALLOW_INDIRECT_BUFFER | D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("DispatchRaysDescBuffer"), DispatchRaysSize, 0, EBufferUsageFlags::DrawIndirect)
				.SetGPUMask(GetGPUMask())
				.SetInitialState(ERHIAccess::IndirectArgs);

			Queue.RayTracingDispatchRaysDescBuffer = GetParentAdapter()->CreateRHIBuffer(
				DispatchRaysBufferDesc,
				256,
				CreateDesc,
				ED3D12ResourceStateMode::MultiState,
				ED3D12Access::IndirectArgs);
		}
	}
#endif // D3D12_RHI_RAYTRACING

	check(!ImmediateCommandContext);
	ImmediateCommandContext = FD3D12DynamicRHI::GetD3DRHI()->CreateCommandContext(this, ED3D12QueueType::Direct, true);

	// Setup diagnostic buffer that contains GPU messages as well as breadcrumb data to to track GPU progress on this command queue (when GPU crash debugging is enabled).
	// The buffer is always allocated and bound to shaders that require it, but breadcrumbs are controlled by UE::RHI::UseGPUCrashBreadcrumbs() and WITH_RHI_BREADCRUMBS.
	for (FD3D12Queue& Queue : Queues)
	{
		Queue.DiagnosticBuffer = MakeUnique<FD3D12DiagnosticBuffer>(Queue);
	}
}

void FD3D12Queue::CleanupResources()
{
	ObjectPool.Contexts.CleanupResources();
	ObjectPool.Allocators.CleanupResources();
	ObjectPool.Lists.CleanupResources();
}

void FD3D12Device::CleanupResources()
{
	for (FD3D12Queue& Queue : Queues)
	{
		Queue.CleanupResources();
	}

	for (FD3D12OfflineDescriptorManager& Manager : OfflineDescriptorManagers)
	{
		Manager.CleanupResources();
	}
	OnlineDescriptorManager.CleanupResources();
	GlobalSamplerHeap.CleanupResources();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	BindlessDescriptorManager.CleanupResources();
#endif

#if D3D12_RHI_RAYTRACING
	CleanupRayTracing();
#endif
}

void FD3D12Device::CreateDefaultViews()
{
	{
		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
		SRVDesc.Format = DXGI_FORMAT_B8G8R8A8_UNORM;
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2D;
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Texture2D.MipLevels = 1;
		SRVDesc.Texture2D.MostDetailedMip = 0;
		SRVDesc.Texture2D.ResourceMinLODClamp = 0.0f;

		DefaultViews.NullSRV = GetOfflineDescriptorManager(ERHIDescriptorHeapType::Standard).AllocateHeapSlot();
		GetDevice()->CreateShaderResourceView(nullptr, &SRVDesc, DefaultViews.NullSRV);
	}

	{
		D3D12_RENDER_TARGET_VIEW_DESC RTVDesc{};
		RTVDesc.Format = DXGI_FORMAT_B8G8R8A8_UNORM;
		RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2D;
		RTVDesc.Texture2D.MipSlice = 0;

		DefaultViews.NullRTV = GetOfflineDescriptorManager(ERHIDescriptorHeapType::RenderTarget).AllocateHeapSlot();
		GetDevice()->CreateRenderTargetView(nullptr, &RTVDesc, DefaultViews.NullRTV);
	}

	{
		D3D12_UNORDERED_ACCESS_VIEW_DESC UAVDesc{};
		UAVDesc.Format = DXGI_FORMAT_R8G8B8A8_UNORM;
		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_TEXTURE2D;
		UAVDesc.Texture2D.MipSlice = 0;

		DefaultViews.NullUAV = GetOfflineDescriptorManager(ERHIDescriptorHeapType::Standard).AllocateHeapSlot();
		GetDevice()->CreateUnorderedAccessView(nullptr, nullptr, &UAVDesc, DefaultViews.NullUAV);
	}

	{
		D3D12_DEPTH_STENCIL_VIEW_DESC DSVDesc{};
		DSVDesc.Format = DXGI_FORMAT_D32_FLOAT;
		DSVDesc.ViewDimension = D3D12_DSV_DIMENSION_TEXTURE2D;
		DSVDesc.Texture2D.MipSlice = 0;

		DefaultViews.NullDSV = GetOfflineDescriptorManager(ERHIDescriptorHeapType::DepthStencil).AllocateHeapSlot();
		GetDevice()->CreateDepthStencilView(nullptr, &DSVDesc, DefaultViews.NullDSV);
	}

	{
		D3D12_CONSTANT_BUFFER_VIEW_DESC CBVDesc{};

		DefaultViews.NullCBV = GetOfflineDescriptorManager(ERHIDescriptorHeapType::Standard).AllocateHeapSlot();
		GetDevice()->CreateConstantBufferView(&CBVDesc, DefaultViews.NullCBV);
	}

	{
		const FSamplerStateInitializerRHI SamplerDesc(SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		DefaultViews.DefaultSampler = CreateSampler(SamplerDesc, GetGPUIndex() > 0 ? GetParentAdapter()->GetDevice(0)->DefaultViews.DefaultSampler : nullptr);

		// The default sampler must have ID=0
		// FD3D12DescriptorCache::SetSamplers relies on this
		check(DefaultViews.DefaultSampler->ID == 0);
	}
}

void FD3D12Device::UpdateConstantBufferPageProperties()
{
	//In genera, constant buffers should use write-combine memory (i.e. upload heaps) for optimal performance
	bool bForceWriteBackConstantBuffers = false;

	if (bForceWriteBackConstantBuffers)
	{
		ConstantBufferPageProperties = GetDevice()->GetCustomHeapProperties(0, D3D12_HEAP_TYPE_UPLOAD);
		ConstantBufferPageProperties.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_WRITE_BACK;
	}
	else
	{
		ConstantBufferPageProperties = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_UPLOAD);
	}
}

void FD3D12Device::UpdateMSAASettings()
{
	check(DX_MAX_MSAA_COUNT == 8);

	// quality levels are only needed for CSAA which we cannot use with custom resolves

	// 0xffffffff means not available
	AvailableMSAAQualities[0] = 0xffffffff;
	AvailableMSAAQualities[1] = 0xffffffff;
	AvailableMSAAQualities[2] = 0;
	AvailableMSAAQualities[3] = 0xffffffff;
	AvailableMSAAQualities[4] = 0;
	AvailableMSAAQualities[5] = 0xffffffff;
	AvailableMSAAQualities[6] = 0xffffffff;
	AvailableMSAAQualities[7] = 0xffffffff;
	AvailableMSAAQualities[8] = 0;
}

void FD3D12Device::BlockUntilIdle()
{
	// Submit a new sync point to each queue
	TArray<FD3D12Payload*> Payloads;
	Payloads.Reserve(int32(ED3D12QueueType::Count));

	TArray<FD3D12SyncPointRef, TInlineAllocator<(uint32)ED3D12QueueType::Count>> SyncPoints;

	for (uint32 QueueTypeIndex = 0; QueueTypeIndex < (uint32)ED3D12QueueType::Count; ++QueueTypeIndex)
	{
		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, UE_D3D12_STATIC_FNAME("BlockUntilGPUIdle"));

		FD3D12Payload* Payload = new FD3D12Payload(GetQueue((ED3D12QueueType)QueueTypeIndex));
		Payload->SyncPointsToSignal.Add(SyncPoint);
		Payload->bAlwaysSignal = true;

		Payloads.Add(Payload);
		SyncPoints.Add(SyncPoint);

		IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SignalSyncPoint(Payload, SyncPoint));
	}

	FD3D12DynamicRHI::GetD3DRHI()->SubmitPayloads(MoveTemp(Payloads));

	// Block this thread until the sync points have signaled.
	for (FD3D12SyncPointRef& SyncPoint : SyncPoints)
	{
		SyncPoint->Wait();
	}
}

D3D12_RESOURCE_ALLOCATION_INFO FD3D12Device::GetResourceAllocationInfoUncached(const FD3D12ResourceDesc& InDesc)
{
	D3D12_RESOURCE_ALLOCATION_INFO Result;

#if INTEL_EXTENSIONS
	if (InDesc.bRequires64BitAtomicSupport && IsRHIDeviceIntel() && GDX12INTCAtomicUInt64Emulation)
	{
		FD3D12ResourceDesc LocalDesc = InDesc;

		INTC_D3D12_RESOURCE_DESC_0001 IntelLocalDesc{};
		IntelLocalDesc.pD3D12Desc = &LocalDesc;
		IntelLocalDesc.EmulatedTyped64bitAtomics = true;

		Result = INTC_D3D12_GetResourceAllocationInfo(FD3D12DynamicRHI::GetD3DRHI()->GetIntelExtensionContext(), 0, 1, &IntelLocalDesc);
	}
	else
#endif
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	if (InDesc.SupportsUncompressedUAV())
	{
		// Convert the desc to the version required by GetResourceAllocationInfo3
		const CD3DX12_RESOURCE_DESC1 LocalDesc(InDesc);

		const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();

		const UINT32 NumCastableFormats = CastableFormats.Num();
		D3D12_RESOURCE_ALLOCATION_INFO1* NoExtraAllocationInfo = nullptr;

		const DXGI_FORMAT* const Formats[] = { CastableFormats.GetData() };

		Result = GetParentAdapter()->GetD3DDevice12()->GetResourceAllocationInfo3(0, 1, &LocalDesc, &NumCastableFormats, Formats, NoExtraAllocationInfo);
	}
	else
#endif
	{
		Result = GetDevice()->GetResourceAllocationInfo(0, 1, &InDesc);
		if (Result.SizeInBytes == UINT64_MAX)
		{
			// The description provided caused an error per the docs. This will almost certainly crash outside this fn.
			UE_LOGF(LogD3D12RHI, Error, "D3D12 GetResourceAllocationInfo failed - likely a resource was requested that has invalid allocation info (e.g. is an invalid texture size)");
			UE_LOGF(LogD3D12RHI, Error, "     W %llu H %d depth %d mips %d pf %d", InDesc.Width, InDesc.Height, InDesc.DepthOrArraySize, InDesc.MipLevels, InDesc.PixelFormat);
		}
	}
	return Result;
}

D3D12_RESOURCE_ALLOCATION_INFO FD3D12Device::GetResourceAllocationInfo(const FD3D12ResourceDesc& InDesc)
{
	D3D12_RESOURCE_ALLOCATION_INFO Result;

	const uint64 Hash = CityHash64((const char*)&InDesc, sizeof(FD3D12ResourceDesc));

	// By default there'll be more threads trying to read this than to write it.
	ResourceAllocationInfoMapMutex.ReadLock();
	D3D12_RESOURCE_ALLOCATION_INFO* CachedInfo = ResourceAllocationInfoMap.Find(Hash);
	if (CachedInfo)
	{
		// We need to hold on to the lock until we copy the cached data, because CachedInfo points to the map storage, which
		// can be reallocated on insertion.
		Result = *CachedInfo;
		ResourceAllocationInfoMapMutex.ReadUnlock();
		return Result;
	}

	ResourceAllocationInfoMapMutex.ReadUnlock();

	Result = GetResourceAllocationInfoUncached(InDesc);

	ResourceAllocationInfoMapMutex.WriteLock();
	// Try search again with write lock because could have been added already
	CachedInfo = ResourceAllocationInfoMap.Find(Hash);
	if (CachedInfo == nullptr)
	{
		ResourceAllocationInfoMap.Add(Hash, Result);
	}
	ResourceAllocationInfoMapMutex.WriteUnlock();

	return Result;
}

FD3D12ContextCommon* FD3D12Device::ObtainContext(ED3D12QueueType QueueType)
{
	FD3D12ContextCommon* Context = Queues[(uint32)QueueType].ObjectPool.Contexts.Pop();
	if (!Context)
	{
		switch (QueueType)
		{
		default: checkNoEntry(); // fallthrough
		case ED3D12QueueType::Direct: Context = FD3D12DynamicRHI::GetD3DRHI()->CreateCommandContext(this, QueueType, false); break;
		case ED3D12QueueType::Async : Context = FD3D12DynamicRHI::GetD3DRHI()->CreateCommandContext(this, QueueType, false); break;
		case ED3D12QueueType::Copy  : Context = new FD3D12ContextCopy(this); break;
		}
	}

	check(Context);
	return Context;
}

void FD3D12Device::ReleaseContext(FD3D12ContextCommon* Context)
{
	check(Context && !Context->IsOpen());

	Queues[(uint32)Context->QueueType].ObjectPool.Contexts.Push(Context);
}

FD3D12CommandAllocator* FD3D12Device::ObtainCommandAllocator(ED3D12QueueType QueueType)
{
	FD3D12CommandAllocator* Allocator = Queues[(uint32)QueueType].ObjectPool.Allocators.Pop();
	if (!Allocator)
	{
		Allocator = new FD3D12CommandAllocator(this, QueueType);
	}

	check(Allocator);
	return Allocator;
}

void FD3D12Device::ReleaseCommandAllocator(FD3D12CommandAllocator* Allocator)
{
	check(Allocator);
	Allocator->Reset();
	Queues[(uint32)Allocator->QueueType].ObjectPool.Allocators.Push(Allocator);
}

FD3D12CommandList* FD3D12Device::ObtainCommandList(FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator)
{
	check(CommandAllocator->Device == this);

	FD3D12CommandList* List = Queues[(uint32)CommandAllocator->QueueType].ObjectPool.Lists.Pop();
	if (!List)
	{
		List = new FD3D12CommandList(CommandAllocator, TimestampAllocator, PipelineStatsAllocator);
	}
	else
	{
		List->Reset(CommandAllocator, TimestampAllocator, PipelineStatsAllocator);
	}
	
	check(List);
	return List;
}

void FD3D12Device::ReleaseCommandList(FD3D12CommandList* CommandList)
{
	check(CommandList);
	Queues[(uint32)CommandList->QueueType].ObjectPool.Lists.Push(CommandList);
}

TRefCountPtr<FD3D12QueryHeap> FD3D12Device::ObtainQueryHeap(ED3D12QueueType QueueType, D3D12_QUERY_TYPE QueryType)
{
	D3D12_QUERY_HEAP_TYPE HeapType;
	switch (QueryType)
	{
		default:
			checkNoEntry();
			return nullptr;

		case D3D12_QUERY_TYPE_OCCLUSION:
			HeapType = D3D12_QUERY_HEAP_TYPE_OCCLUSION;
			break;

		case D3D12_QUERY_TYPE_TIMESTAMP:
			if (QueueType == ED3D12QueueType::Copy)
			{
				// Support for copy queue timestamps is driver dependent.
				if (!GetParentAdapter()->AreCopyQueueTimestampQueriesSupported())
					return nullptr; // Not supported

				HeapType = D3D12_QUERY_HEAP_TYPE_COPY_QUEUE_TIMESTAMP;
			}
			else
			{
				HeapType = D3D12_QUERY_HEAP_TYPE_TIMESTAMP;
			}
			break;

		case D3D12_QUERY_TYPE_PIPELINE_STATISTICS:
#if D3D12RHI_ENABLE_PIPELINE_STATISTICS
			if (QueueType != ED3D12QueueType::Direct)
			{
				// Only graphics/direct queues support pipeline statistics
				return nullptr;
			}
			HeapType = D3D12_QUERY_HEAP_TYPE_PIPELINE_STATISTICS;
			break;
#else
			return nullptr;
#endif
	}

	FD3D12QueryHeap* QueryHeap = QueryHeapPool[GetQueryHeapPoolIndex(HeapType)].Pop();
	if (!QueryHeap)
	{
		QueryHeap = new FD3D12QueryHeap(this, QueryType, HeapType);
	}

	check(QueryHeap->QueryType == QueryType);
	check(QueryHeap->HeapType == HeapType);

	return QueryHeap;
}

void FD3D12Device::ReleaseQueryHeap(FD3D12QueryHeap* QueryHeap)
{
	check(QueryHeap);
	QueryHeapPool[GetQueryHeapPoolIndex(QueryHeap->HeapType)].Push(QueryHeap);
}

uint64 FD3D12Device::GetTimestampFrequency(ED3D12QueueType QueueType)
{
	check(QueueType != ED3D12QueueType::Copy || GetParentAdapter()->AreCopyQueueTimestampQueriesSupported());

	uint64 Frequency;
	VERIFYD3D12RESULT(Queues[(uint32)QueueType].D3DCommandQueue->GetTimestampFrequency(&Frequency));
	return Frequency;
}

void FD3D12Device::InitExplicitDescriptorHeap()
{
	check(ExplicitDescriptorHeapCache == nullptr);
	ExplicitDescriptorHeapCache = new FD3D12ExplicitDescriptorHeapCache(this);

	// Note: ExplicitDescriptorHeapCache is destroyed in ~FD3D12Device, after all deferred deletion is processed
}

void FD3D12Device::DestroyExplicitDescriptorCache()
{
	delete ExplicitDescriptorHeapCache;
	ExplicitDescriptorHeapCache = nullptr;
}

