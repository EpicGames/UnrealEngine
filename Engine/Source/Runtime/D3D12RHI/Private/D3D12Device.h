// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Device.h: D3D12 Device Interfaces
=============================================================================*/

#pragma once

#include "RHIBreadcrumbs.h"
#include "RHIDiagnosticBuffer.h"

#include "D3D12BindlessDescriptors.h"
#include "D3D12CommandContext.h"
#include "D3D12Descriptors.h"
#include "D3D12Query.h"
#include "D3D12Queue.h"
#include "D3D12Resources.h"
#include "D3D12Submission.h"

#include "Containers/LruCache.h"
#include "Containers/SpscQueue.h"
#include "Containers/MpscQueue.h"

class FD3D12Device;
class FD3D12DynamicRHI;
class FD3D12Buffer;
class FD3D12Queue;
class FD3D12ExplicitDescriptorHeapCache;
class FD3D12RayTracingPipelineCache;
class FD3D12RayTracingCompactionRequestHandler;
struct FD3D12RayTracingPipelineInfo;

//
// Diagnostic buffer, backed by a virtual heap. Stays accessible after a GPU crash to allow readback of diagnostic messages.
// Also used to track the progress of the GPU via breadcrumb markers.
//
class FD3D12DiagnosticBuffer : public FRHIDiagnosticBuffer
{
private:

	TRefCountPtr<FD3D12Heap> Heap;
	TRefCountPtr<FD3D12Resource> Resource;

	D3D12_GPU_VIRTUAL_ADDRESS GpuAddress = 0;
	D3D12_GPU_VIRTUAL_ADDRESS ToGPUAddress(void* Ptr) const
	{
		return GpuAddress + (uintptr_t(Ptr) - uintptr_t(Data));
	}

public:
	FD3D12DiagnosticBuffer(FD3D12Queue& Queue);
	~FD3D12DiagnosticBuffer();

	D3D12_GPU_VIRTUAL_ADDRESS GetGPUQueueData     () const { return ToGPUAddress(Data); }

#if WITH_RHI_BREADCRUMBS
	D3D12_GPU_VIRTUAL_ADDRESS GetGPUQueueMarkerIn () const { return ToGPUAddress(&Data->MarkerIn ); }
	D3D12_GPU_VIRTUAL_ADDRESS GetGPUQueueMarkerOut() const { return ToGPUAddress(&Data->MarkerOut); }

	uint32 ReadMarkerIn () const { return Data->MarkerIn;  }
	uint32 ReadMarkerOut() const { return Data->MarkerOut; }
#endif

	bool IsValid() const { return Resource.IsValid(); }
};

// Encapsulates the state required for tracking GPU queue performance across a frame.
class FD3D12Timing
{
public:
	FD3D12Queue& Queue;
	D3D12_QUERY_DATA_PIPELINE_STATISTICS PipelineStats {};

	// Timer calibration data
	uint64 GPUFrequency = 0, GPUTimestamp = 0;
	uint64 CPUFrequency = 0, CPUTimestamp = 0;

	UE::RHI::GPUProfiler::FEventStream EventStream;

	FD3D12Timing(FD3D12Queue& Queue);
};

// Encapsulates a single D3D command queue, and maintains the 
// state required by the submission thread for managing the queue.
class FD3D12Queue final
{
public:
	void CleanupResources();

	FD3D12Device* const Device;
	ED3D12QueueType const QueueType;
	int32 const QueueIndex;

	// The underlying D3D queue object
	TRefCountPtr<ID3D12CommandQueue> D3DCommandQueue;

	// A single D3D fence to manage completion of work on this queue
	FD3D12Fence Fence;

	TMpscQueue<FD3D12Payload*> PendingSubmission;
	TSpscQueue<FD3D12Payload*> PendingInterrupt;

	FD3D12Payload*          PayloadToSubmit  = nullptr;
	FD3D12CommandAllocator* BarrierAllocator = nullptr;
	FD3D12QueryAllocator    BarrierTimestamps;

	FD3D12BatchedPayloadObjects BatchedObjects;

	// A pool of reusable command list/allocator/context objects
	struct
	{
		TD3D12ObjectPool<FD3D12ContextCommon   > Contexts;
		TD3D12ObjectPool<FD3D12CommandAllocator> Allocators;
		TD3D12ObjectPool<FD3D12CommandList     > Lists;
	} ObjectPool;

	// The active timing struct on this queue. Updated / accessed by the interrupt thread.
	FD3D12Timing* Timing = nullptr;

	TUniquePtr<FD3D12DiagnosticBuffer> DiagnosticBuffer;

#if D3D12_RHI_RAYTRACING
	FD3D12Buffer* RayTracingDispatchRaysDescBuffer = nullptr;
#endif

	// On some hardware, some auxiliary queue types may not support tile mapping and a separate queue must be used
	bool bSupportsTileMapping = true;

	static constexpr uint32 MaxBatchedPayloads = 128;
	using FPayloadArray = TArray<FD3D12Payload*, TInlineAllocator<MaxBatchedPayloads>>;

	// Batches the current payload's command lists, returning the latest fence value signaled for this queue.
	uint64 FinalizePayload(bool bRequiresSignal, FPayloadArray& PayloadsToHandDown);

	// Call the underlying ID3D12Queue::ExecuteCommandLists function
	void ExecuteCommandLists(TArrayView<ID3D12CommandList*> D3DCommandLists
#if ENABLE_RESIDENCY_MANAGEMENT
		, TArrayView<FD3D12ResidencySet*> ResidencySets
#endif
	);

	FD3D12Queue(FD3D12Device* Device, ED3D12QueueType QueueType, int32 QueueIndex);
	~FD3D12Queue();

	UE::RHI::GPUProfiler::FQueue GetProfilerQueue() const;

private:
	// Internal fence which may be used before calling ExecuteCommandLists
	FD3D12Fence ExecuteCommandListsFence;

};

class FD3D12Device final : public FD3D12SingleNodeGPUObject, public FNoncopyable, public FD3D12AdapterChild
{
public:
	FD3D12Device(FRHIGPUMask InGPUMask, FD3D12Adapter* InAdapter);
	~FD3D12Device();

	ID3D12Device* GetDevice();

	uint64 GetTimestampFrequency(ED3D12QueueType QueueType);

	// Misc
	void BlockUntilIdle();
	D3D12_RESOURCE_ALLOCATION_INFO GetResourceAllocationInfoUncached(const FD3D12ResourceDesc& InDesc);
	D3D12_RESOURCE_ALLOCATION_INFO GetResourceAllocationInfo(const FD3D12ResourceDesc& InDesc);

	// Specialized wrapper of ID3D12Device::CopyDescriptors for a common case of a single descriptor range. Similar to CopyDescriptorsSimple(), except source is provided as an array.
	static void CopyDescriptors(ID3D12Device* D3DDevice, D3D12_CPU_DESCRIPTOR_HANDLE Destination, const D3D12_CPU_DESCRIPTOR_HANDLE* Source, uint32 NumSourceDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type);
	void CopyDescriptors(D3D12_CPU_DESCRIPTOR_HANDLE Destination, const D3D12_CPU_DESCRIPTOR_HANDLE* Source, uint32 NumSourceDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type)
	{
		CopyDescriptors(GetDevice(), Destination, Source, NumSourceDescriptors, Type);
	}

	void									  InitExplicitDescriptorHeap();
	FD3D12ExplicitDescriptorHeapCache*		  GetExplicitDescriptorHeapCache() { return ExplicitDescriptorHeapCache; }

	// Ray Tracing
#if D3D12_RHI_RAYTRACING
	void									  InitRayTracing();
	void									  CleanupRayTracing();

	FD3D12RayTracingPipelineCache*			  GetRayTracingPipelineCache           () { return RayTracingPipelineCache;            }
	FD3D12Buffer*							  GetRayTracingDispatchRaysDescBuffer  (ED3D12QueueType QueueType) { return Queues[(uint32)QueueType].RayTracingDispatchRaysDescBuffer; }
	FD3D12RayTracingCompactionRequestHandler* GetRayTracingCompactionRequestHandler() { return RayTracingCompactionRequestHandler; }

	TRefCountPtr<ID3D12StateObject>			  DeserializeRayTracingStateObject(D3D12_SHADER_BYTECODE Bytecode, ID3D12RootSignature* RootSignature);

	void GetRaytracingAccelerationStructurePrebuildInfo(
		const D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS* pDesc,
		D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO* pInfo);

	// Queries ray tracing pipeline state object metrics such as VGPR usage (if available/supported). Returns true if query succeeded.
	bool GetRayTracingPipelineInfo(ID3D12StateObject* Pipeline, FD3D12RayTracingPipelineInfo* OutInfo);
#endif // D3D12_RHI_RAYTRACING

	// Heaps
	inline FD3D12GlobalOnlineSamplerHeap& GetGlobalSamplerHeap() { return GlobalSamplerHeap; }

	inline const D3D12_HEAP_PROPERTIES& GetConstantBufferPageProperties() { return ConstantBufferPageProperties; }

	// Descriptor Managers
	inline FD3D12DescriptorHeapManager&     GetDescriptorHeapManager    () { return DescriptorHeapManager;     }
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorAllocator& GetBindlessDescriptorAllocator() { return BindlessDescriptorAllocator; }
	FD3D12BindlessDescriptorManager& GetBindlessDescriptorManager() { return BindlessDescriptorManager; }
#endif
	inline FD3D12OnlineDescriptorManager&   GetOnlineDescriptorManager  () { return OnlineDescriptorManager;   }
	inline FD3D12OfflineDescriptorManager&  GetOfflineDescriptorManager (ERHIDescriptorHeapType InType)
	{
		check(InType < ERHIDescriptorHeapType::Count);
		return OfflineDescriptorManagers[static_cast<int>(InType)];
	}

	const FD3D12DefaultViews& GetDefaultViews() const { return DefaultViews; }

	// Memory Allocators
	inline FD3D12DefaultBufferAllocator& GetDefaultBufferAllocator() { return DefaultBufferAllocator; }
	inline FD3D12FastAllocator&          GetDefaultFastAllocator  () { return DefaultFastAllocator;   }
	inline FD3D12TextureAllocatorPool&   GetTextureAllocator      () { return TextureAllocator;       }

	// Residency
	inline FD3D12ResidencyManager& GetResidencyManager() { return ResidencyManager; }

	// Samplers
	FD3D12SamplerState* CreateSampler(const FSamplerStateInitializerRHI& Initializer, FD3D12SamplerState* FirstLinkedObject);
	void CreateSamplerInternal(const D3D12_SAMPLER_DESC& Desc, D3D12_CPU_DESCRIPTOR_HANDLE Descriptor);

	// Command Allocators
	FD3D12CommandAllocator* ObtainCommandAllocator (ED3D12QueueType QueueType);
	void                    ReleaseCommandAllocator(FD3D12CommandAllocator* Allocator);

	// Contexts
	FD3D12CommandContext&   GetDefaultCommandContext() { return *ImmediateCommandContext; }
	FD3D12ContextCommon*    ObtainContext           (ED3D12QueueType QueueType);
	FD3D12ContextCopy*      ObtainContextCopy       () { return static_cast<FD3D12ContextCopy*   >(ObtainContext(ED3D12QueueType::Copy  )); }
	FD3D12CommandContext*   ObtainContextCompute    () { return static_cast<FD3D12CommandContext*>(ObtainContext(ED3D12QueueType::Async )); }
	FD3D12CommandContext*   ObtainContextGraphics   () { return static_cast<FD3D12CommandContext*>(ObtainContext(ED3D12QueueType::Direct)); }
	void                    ReleaseContext          (FD3D12ContextCommon* Context);

	// Queries
	TRefCountPtr<FD3D12QueryHeap> ObtainQueryHeap (ED3D12QueueType QueueType, D3D12_QUERY_TYPE QueryType);
	void                          ReleaseQueryHeap(FD3D12QueryHeap* QueryHeap);
	
	// Command Lists
	FD3D12CommandList* ObtainCommandList (FD3D12CommandAllocator* CommandAllocator, FD3D12QueryAllocator* TimestampAllocator, FD3D12QueryAllocator* PipelineStatsAllocator);
	void               ReleaseCommandList(FD3D12CommandList* CommandList);

	// Queues
	FD3D12Queue& GetQueue(ED3D12QueueType QueueType) { return Queues[(uint32)QueueType]; }
	TArrayView<FD3D12Queue> GetQueues() { return Queues; }

	// shared code for different D3D12  devices (e.g. PC DirectX12 and XboxOne) called
	// after device creation and GRHISupportsAsyncTextureCreation was set and before resource init
	void SetupAfterDeviceCreation();
	void CleanupResources();

	// Wrapper of ID3D12Device::CreateCommandList
	HRESULT CreateCommandList(
		UINT                    nodeMask,
		D3D12_COMMAND_LIST_TYPE type,
		ID3D12CommandAllocator* pCommandAllocator,
		ID3D12PipelineState*    pInitialState,
		REFIID                  riid,
		void**                  ppCommandList
	);

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	void CreateUnorderedAccessViewAlias(
		ID3D12Resource* InResource,
		ID3D12Resource* InCounterResource,
		const D3D12_RESOURCE_DESC& InAliasResourceDesc,
		const D3D12_UNORDERED_ACCESS_VIEW_DESC& InAliasViewDesc,
		D3D12_CPU_DESCRIPTOR_HANDLE InOfflineCpuHandle);
#endif

	TRefCountPtr<ID3D12CommandQueue> TileMappingQueue;
	FD3D12Fence TileMappingFence;

private:
	// called by SetupAfterDeviceCreation() when the device gets initialized
	void CreateDefaultViews();
	void UpdateMSAASettings();
	void UpdateConstantBufferPageProperties();

	struct FResidencyManager : public FD3D12ResidencyManager
	{
		FResidencyManager(FD3D12Device& Parent);
		~FResidencyManager();
	} ResidencyManager;

	FD3D12DescriptorHeapManager     DescriptorHeapManager;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorAllocator& BindlessDescriptorAllocator;
	FD3D12BindlessDescriptorManager BindlessDescriptorManager;
#endif
	TArray<FD3D12OfflineDescriptorManager, TFixedAllocator<(uint32)ERHIDescriptorHeapType::Count>> OfflineDescriptorManagers;

	FD3D12GlobalOnlineSamplerHeap GlobalSamplerHeap;
	FD3D12OnlineDescriptorManager OnlineDescriptorManager;

	FD3D12DefaultViews DefaultViews;

	TStaticArray<TD3D12ObjectPool<FD3D12QueryHeap>, 4> QueryHeapPool { InPlace };
	
	FD3D12CommandContext* ImmediateCommandContext = nullptr;

	TArray<FD3D12Queue, TFixedAllocator<(uint32)ED3D12QueueType::Count>> Queues;

	TLruCache<D3D12_SAMPLER_DESC, TRefCountPtr<FD3D12SamplerState>> SamplerCache;
	uint32 SamplerID = 0;

	/** Hashmap used to cache resource allocation size information */
	FRWLock ResourceAllocationInfoMapMutex;
	TMap<uint64, D3D12_RESOURCE_ALLOCATION_INFO> ResourceAllocationInfoMap;

	// set by UpdateMSAASettings(), get by GetMSAAQuality()
	// [SampleCount] = Quality, 0xffffffff if not supported
	uint32 AvailableMSAAQualities[DX_MAX_MSAA_COUNT + 1];

	// set by UpdateConstantBufferPageProperties, get by GetConstantBufferPageProperties
	D3D12_HEAP_PROPERTIES ConstantBufferPageProperties;

	FD3D12DefaultBufferAllocator DefaultBufferAllocator;
	FD3D12FastAllocator          DefaultFastAllocator;
	FD3D12TextureAllocatorPool   TextureAllocator;

#if D3D12_RHI_RAYTRACING
	FD3D12RayTracingPipelineCache* RayTracingPipelineCache = nullptr;
	FD3D12RayTracingCompactionRequestHandler* RayTracingCompactionRequestHandler = nullptr;
// #dxr_todo UE-72158: unify RT descriptor cache with main FD3D12DescriptorCache
#endif

	FD3D12ExplicitDescriptorHeapCache* ExplicitDescriptorHeapCache = nullptr;
	void DestroyExplicitDescriptorCache();
};
