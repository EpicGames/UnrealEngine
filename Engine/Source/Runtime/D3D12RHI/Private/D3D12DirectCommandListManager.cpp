// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12DirectCommandListManager.h"
#include "D3D12RHIPrivate.h"
#include "Windows.h"

TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> FD3D12SyncPoint::MemoryPool;

FD3D12GPUFence::FD3D12GPUFence(FName InName)
	: FRHIGPUFence(InName)
{
	Clear();
}

void FD3D12GPUFence::Clear()
{
	SyncPoints.Reset();
	SyncPoints.AddDefaulted(FRHIGPUMask::All().GetNumActive());
}

bool FD3D12GPUFence::Poll() const
{
	return Poll(FRHIGPUMask::All());
}

bool FD3D12GPUFence::Poll(FRHIGPUMask GPUMask) const
{
	bool bHasAnySyncPoint = false;
	for (uint32 Index : GPUMask)
	{
		if (SyncPoints[Index])
		{
			if (!SyncPoints[Index]->IsComplete())
			{
				return false;
			}
			bHasAnySyncPoint = true;
		}
	}

	// Return "true" if we had sync points that all successfully completed, or "false" if we have no sync points (fence was never signaled)
	return bHasAnySyncPoint;
}

void FD3D12GPUFence::Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
{
	for (uint32 Index : GPUMask)
	{
		if (SyncPoints[Index] && !SyncPoints[Index]->IsComplete())
		{
			SyncPoints[Index]->Wait();
		}
	}
}

void FD3D12DynamicRHI::RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI)
{
	FD3D12GPUFence* Fence = FD3D12DynamicRHI::ResourceCast(FenceRHI);
	check(Fence);

	for (uint32 GPUIndex : RHICmdList.GetGPUMask())
	{
		checkf(Fence->SyncPoints[GPUIndex] == nullptr, TEXT("The fence for the current GPU node has already been issued."));

		Fence->SyncPoints[GPUIndex] = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, FenceRHI->GetFName());
	}

	Fence->NumPendingWriteCommands.Increment();
	RHICmdList.EnqueueLambda([Fence, SyncPoints = Fence->SyncPoints](FRHICommandListBase& ExecutingCmdList)
	{
		for (uint32 GPUIndex : ExecutingCmdList.GetGPUMask())
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
			Context.SignalSyncPoint(SyncPoints[GPUIndex]);
		}

		Fence->NumPendingWriteCommands.Decrement();
	});
}

FGPUFenceRHIRef FD3D12DynamicRHI::RHICreateGPUFence(const FName& Name)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateGPUFence"));
	return new FD3D12GPUFence(Name);
}

FStagingBufferRHIRef FD3D12DynamicRHI::RHICreateStagingBuffer()
{
	// Don't know the device yet - will be decided at copy time (lazy creation)
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateStagingBuffer"));
	return new FD3D12StagingBuffer(nullptr);
}

void* FD3D12DynamicRHI::RHILockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
{
	FD3D12StagingBuffer* StagingBuffer = FD3D12DynamicRHI::ResourceCast(StagingBufferRHI);
	check(StagingBuffer);

	return StagingBuffer->Lock(Offset, SizeRHI);
}

void FD3D12DynamicRHI::RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBufferRHI)
{
	FD3D12StagingBuffer* StagingBuffer = FD3D12DynamicRHI::ResourceCast(StagingBufferRHI);
	check(StagingBuffer);
	StagingBuffer->Unlock();
}

FD3D12StagingBuffer::~FD3D12StagingBuffer()
{
	ResourceLocation.Clear();
}

void* FD3D12StagingBuffer::Lock(uint32 Offset, uint32 NumBytes)
{
	check(!bIsLocked);
	bIsLocked = true;
	if (ResourceLocation.IsValid())
	{
		// readback resource are kept mapped after creation
		return reinterpret_cast<uint8*>(ResourceLocation.GetMappedBaseAddress()) + Offset;
	}
	else
	{
		return nullptr;
	}
}

void FD3D12StagingBuffer::Unlock()
{
	check(bIsLocked);
	bIsLocked = false;
}

// =============================================================================

FD3D12ManualFence::FD3D12ManualFence(FD3D12Adapter* InParent)
	: Parent(InParent)
{
	for (FD3D12Device* Device : Parent->GetDevices())
	{
		for (FD3D12Queue& Queue : Device->GetQueues())
		{
			TRefCountPtr<ID3D12Fence> Fence;
			VERIFYD3D12RESULT(Parent->GetD3DDevice()->CreateFence(0, D3D12_FENCE_FLAG_NONE, IID_PPV_ARGS(Fence.GetInitReference())));
			Fence->SetName(TEXT("Manual Fence"));

			Fences.Add(&Queue, MoveTemp(Fence));
		}
	}
}

uint64 FD3D12ManualFence::GetCompletedFenceValue(bool bUpdateCachedFenceValue)
{
	if (bUpdateCachedFenceValue)
	{
		uint64 MinFenceValue = TNumericLimits<uint64>::Max();

		for (auto& [Queue, Fence] : Fences)
		{
			MinFenceValue = FMath::Min(
				Fence->GetCompletedValue(),
				MinFenceValue
			);
		}

		CompletedFenceValue = MinFenceValue;
	}

	return CompletedFenceValue;
}

void FD3D12ManualFence::AdvanceTOP()
{
	check(IsInRenderingThread());
	NextFenceValueTOP.Increment();
}

void FD3D12ManualFence::AdvanceBOP()
{
	const uint64 NextValue = ++NextFenceValueBOP;

	TArray<FD3D12Payload*> Payloads;
	Payloads.Reserve(Fences.Num());

	for (auto& [Queue, Fence] : Fences)
	{
		FD3D12Payload* Payload = new FD3D12Payload(*Queue);
		Payload->ManualFencesToSignal.Emplace(Fence.GetReference(), NextValue, nullptr);

		Payloads.Add(Payload);
	}

	FD3D12DynamicRHI::GetD3DRHI()->SubmitPayloads(MoveTemp(Payloads));
}
