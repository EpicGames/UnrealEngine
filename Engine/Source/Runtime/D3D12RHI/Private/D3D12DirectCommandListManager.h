// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "D3D12ThirdParty.h"
#include "Templates/RefCounting.h"

class FD3D12Adapter;
class FD3D12CommandContext;
class FD3D12Queue;

extern int32 GEmitRgpFrameMarkers;

// A fence that is manually signaled on the graphics pipe (all graphics pipes in mGPU setups).
// @todo: Remove this. Systems that rely on this fence should be converted to use sync points instead.
class FD3D12ManualFence final
{
	FD3D12Adapter* const Parent;
	TMap<FD3D12Queue*, TRefCountPtr<ID3D12Fence>> Fences;

	FThreadSafeCounter NextFenceValueTOP = 0;
	uint64 NextFenceValueBOP = 0;
	uint64 CompletedFenceValue = 0;

	FD3D12ManualFence(FD3D12ManualFence const&) = delete;
	FD3D12ManualFence(FD3D12ManualFence&&) = delete;

public:
	FD3D12ManualFence(FD3D12Adapter* InParent);

	// Returned the fence value which has been signaled by the GPU.
	// If bUpdateCachedFenceValue is false, only the cached value is returned.
	// Otherwise, the latest fence value is queried from the driver, and the cached value is updated.
	uint64 GetCompletedFenceValue(bool bUpdateCachedFenceValue);

	// Determines if the given fence value has been signaled on the GPU.
	bool IsFenceComplete(uint64 FenceValue, bool bUpdateCachedFenceValue)
	{
		return GetCompletedFenceValue(bUpdateCachedFenceValue) >= FenceValue;
	}

	// Returns the next value to be signaled.
	uint64 GetNextFenceToSignal() const
	{
		return NextFenceValueTOP.GetValue() + 1;
	}

	void AdvanceTOP();
	void AdvanceBOP();
};
