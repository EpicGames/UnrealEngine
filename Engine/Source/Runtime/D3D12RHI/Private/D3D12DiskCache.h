// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PreprocessorHelpers.h"

#include COMPILED_PLATFORM_HEADER(D3D12DiskCache.h)
