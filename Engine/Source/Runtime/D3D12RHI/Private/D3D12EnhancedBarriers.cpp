// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12EnhancedBarriers.h"

#if D3D12RHI_SUPPORTS_ENHANCED_BARRIERS

#include "RHIPipeline.h"
#include "RHICoreTransitions.h"

#include "D3D12Query.h"
#include "D3D12Resources.h"
#include "D3D12Util.h"
#include "D3D12CommandList.h"
#include "D3D12Texture.h"
#include "D3D12CommandContext.h"
#include "D3D12Adapter.h"
#include "D3D12RHIPrivate.h"

#include "Algo/Contains.h"
#include "Templates/Tuple.h"
#include "Containers/StaticArray.h"

#include <bit>


// @TODO - The EB spec has a hole in it where there's no valid layout to use for simultaneous read access
//         from multiple pipes (e.g. async and gfx) that includes access bits that are gfx specific, like 
//         DEPTH_STENCIL_READ. Note that this makes the validation layers useless when the async pipe is used
//         and it's questionable if any given driver will do something sane.
#define PLATFORM_REQUIRES_ENHANCED_BARRIERS_GFX_ONLY_READ_BITS_HACK !(D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE)

// @TODO - EB spec is inconsistent about whether D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE
//         is compatible with D3D12_BARRIER_SYNC_RAYTRACING. Further, the validation layer in big windows
//         doesn't like that pair. So, we'll omit it for windows for now.
#define PLATFORM_REQUIRES_SYNC_RAYTRACING_NOT_COMPATIBLE_WITH_ACCESS_AS_WRITE (PLATFORM_WINDOWS)

// @TODO - Validation layer incorrectly complains that D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ isn't
//         compatible with D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE. Until this is fixed we omit the
//         read access bit if we're also setting the write access bit
#define PLATFORM_REQUIRES_LAYOUT_DEPTH_STENCIL_WRITE_NOT_COMPATIBLE_WITH_ACCESS_DEPTH_STENCIL_READ !(PLATFORM_WINDOWS)


// Each platform must provide its own implementation of this
extern D3D12_BARRIER_LAYOUT GetSkipFastClearEliminateLayoutFlags();

struct FD3D12EnhancedBarriersTransitionData
{
	ERHIPipeline SrcPipelines, DstPipelines;
	ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None;

	TArray<FRHITransitionInfo, TInlineAllocator<4, FConcurrentLinearArrayAllocator>> TransitionInfos;
	TArray<TRHIPipelineArray<FD3D12SyncPointRef>, TInlineAllocator<MAX_NUM_GPUS>> SyncPoints;
};

struct FD3D12BarrierValues
{
	D3D12_BARRIER_SYNC Sync;
	D3D12_BARRIER_ACCESS Access;
	D3D12_BARRIER_LAYOUT Layout;

	bool operator==(const FD3D12BarrierValues& other) const
	{
		return Sync == other.Sync &&
			   Access == other.Access &&
			   Layout == other.Layout;
	}
};

static bool operator==(
	const D3D12_GLOBAL_BARRIER& BarrierA,
	const D3D12_GLOBAL_BARRIER& BarrierB)
{
	return BarrierA.SyncBefore   == BarrierB.SyncBefore
		&& BarrierA.SyncAfter    == BarrierB.SyncAfter
		&& BarrierA.AccessBefore == BarrierB.AccessBefore
		&& BarrierA.AccessAfter  == BarrierB.AccessAfter;
}

static bool operator==(
	const D3D12_TEXTURE_BARRIER& BarrierA,
	const D3D12_TEXTURE_BARRIER& BarrierB)
{
	return BarrierA.SyncBefore   == BarrierB.SyncBefore
		&& BarrierA.SyncAfter    == BarrierB.SyncAfter
		&& BarrierA.AccessBefore == BarrierB.AccessBefore
		&& BarrierA.AccessAfter  == BarrierB.AccessAfter
		&& BarrierA.LayoutBefore == BarrierB.LayoutBefore
		&& BarrierA.LayoutAfter  == BarrierB.LayoutAfter
		&& BarrierA.pResource    == BarrierB.pResource
		&& BarrierA.Subresources.IndexOrFirstMipLevel == BarrierB.Subresources.IndexOrFirstMipLevel
		&& BarrierA.Subresources.NumMipLevels         == BarrierB.Subresources.NumMipLevels
		&& BarrierA.Subresources.FirstArraySlice      == BarrierB.Subresources.FirstArraySlice
		&& BarrierA.Subresources.NumArraySlices       == BarrierB.Subresources.NumArraySlices
		&& BarrierA.Subresources.FirstPlane           == BarrierB.Subresources.FirstPlane
		&& BarrierA.Subresources.NumPlanes            == BarrierB.Subresources.NumPlanes
		&& BarrierA.Flags        == BarrierB.Flags;
}

static bool operator==(
	const D3D12_BUFFER_BARRIER& BarrierA,
	const D3D12_BUFFER_BARRIER& BarrierB)
{
	return BarrierA.SyncBefore   == BarrierB.SyncBefore
		&& BarrierA.SyncAfter    == BarrierB.SyncAfter
		&& BarrierA.AccessBefore == BarrierB.AccessBefore
		&& BarrierA.AccessAfter  == BarrierB.AccessAfter
		&& BarrierA.pResource    == BarrierB.pResource
		&& BarrierA.Offset       == BarrierB.Offset
		&& BarrierA.Size         == BarrierB.Size;
}

template <typename T, typename TString, typename ...TRest>
static constexpr void ConvertFlagsToStringImpl(
	T InValue,
	FString& InFlagString,
	TPair<T, TString> InFirstFlag,
	TRest... InRestFlags)
{
	if (InValue & get<0>(InFirstFlag))
	{
		if (!InFlagString.IsEmpty())
		{
			InFlagString += TEXT("|");
		}
		InFlagString += get<1>(InFirstFlag);
	}

	if constexpr (sizeof...(InRestFlags))
	{
		ConvertFlagsToStringImpl(InValue, InFlagString, InRestFlags...);
	}
}

template <typename T, typename TString, typename ...TRest>
static FString ConvertFlagsToString(
	T InValue,
	TPair<T, TString> InFirstFlag,
	TRest... InRestFlags)
{
	FString FlagString;
	ConvertFlagsToStringImpl(InValue, FlagString, InFirstFlag, InRestFlags...);
	if (FlagString.IsEmpty())
	{
		// Assume the first provided enum value is "None"
		check(static_cast<std::underlying_type_t<T>>(get<0>(InFirstFlag)) == 0);
		return get<1>(InFirstFlag);
	}
	return FlagString;
}

template <typename T, typename TString, typename ...TArgs>
static FString ConvertEnumToString(
	T InValue,
	TPair<T, TString> FirstArg,
	TArgs... Args)
{
	if (InValue == get<0>(FirstArg))
	{
		return get<1>(FirstArg);
	}

	if constexpr (sizeof...(Args))
	{
		return ConvertEnumToString<T>(InValue, Args...);
	}
	else
	{
		checkNoEntry();
		return FString();
	}
}

#define ENUMVAL(EnumVal) MakeTuple(EnumVal, TEXT(#EnumVal))

static FString ConvertToString(D3D12_BARRIER_SYNC InSync)
{
	return ConvertFlagsToString(InSync,
		ENUMVAL(D3D12_BARRIER_SYNC_NONE),
		ENUMVAL(D3D12_BARRIER_SYNC_ALL),
		ENUMVAL(D3D12_BARRIER_SYNC_DRAW),
		ENUMVAL(D3D12_BARRIER_SYNC_INDEX_INPUT),
		ENUMVAL(D3D12_BARRIER_SYNC_VERTEX_SHADING),
		ENUMVAL(D3D12_BARRIER_SYNC_PIXEL_SHADING),
		ENUMVAL(D3D12_BARRIER_SYNC_DEPTH_STENCIL),
		ENUMVAL(D3D12_BARRIER_SYNC_RENDER_TARGET),
		ENUMVAL(D3D12_BARRIER_SYNC_COMPUTE_SHADING),
		ENUMVAL(D3D12_BARRIER_SYNC_RAYTRACING),
		ENUMVAL(D3D12_BARRIER_SYNC_COPY),
		ENUMVAL(D3D12_BARRIER_SYNC_RESOLVE),
		ENUMVAL(D3D12_BARRIER_SYNC_EXECUTE_INDIRECT),
		ENUMVAL(D3D12_BARRIER_SYNC_PREDICATION),
		ENUMVAL(D3D12_BARRIER_SYNC_ALL_SHADING),
		ENUMVAL(D3D12_BARRIER_SYNC_NON_PIXEL_SHADING),
		ENUMVAL(D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO),
		ENUMVAL(D3D12_BARRIER_SYNC_CLEAR_UNORDERED_ACCESS_VIEW),
		ENUMVAL(D3D12_BARRIER_SYNC_VIDEO_DECODE),
		ENUMVAL(D3D12_BARRIER_SYNC_VIDEO_PROCESS),
		ENUMVAL(D3D12_BARRIER_SYNC_VIDEO_ENCODE),
		ENUMVAL(D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE),
		ENUMVAL(D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE),
		ENUMVAL(D3D12_BARRIER_SYNC_SPLIT));
}

static FString ConvertToString(D3D12_BARRIER_ACCESS InAccess)
{
	return ConvertFlagsToString(InAccess,
		ENUMVAL(D3D12_BARRIER_ACCESS_COMMON),
		ENUMVAL(D3D12_BARRIER_ACCESS_VERTEX_BUFFER),
		ENUMVAL(D3D12_BARRIER_ACCESS_CONSTANT_BUFFER),
		ENUMVAL(D3D12_BARRIER_ACCESS_INDEX_BUFFER),
		ENUMVAL(D3D12_BARRIER_ACCESS_RENDER_TARGET),
		ENUMVAL(D3D12_BARRIER_ACCESS_UNORDERED_ACCESS),
		ENUMVAL(D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE),
		ENUMVAL(D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ),
		ENUMVAL(D3D12_BARRIER_ACCESS_SHADER_RESOURCE),
		ENUMVAL(D3D12_BARRIER_ACCESS_STREAM_OUTPUT),
		ENUMVAL(D3D12_BARRIER_ACCESS_INDIRECT_ARGUMENT),
		ENUMVAL(D3D12_BARRIER_ACCESS_PREDICATION),
		ENUMVAL(D3D12_BARRIER_ACCESS_COPY_DEST),
		ENUMVAL(D3D12_BARRIER_ACCESS_COPY_SOURCE),
		ENUMVAL(D3D12_BARRIER_ACCESS_RESOLVE_DEST),
		ENUMVAL(D3D12_BARRIER_ACCESS_RESOLVE_SOURCE),
		ENUMVAL(D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ),
		ENUMVAL(D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE),
		ENUMVAL(D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_DECODE_READ),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_DECODE_WRITE),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_PROCESS_READ),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_PROCESS_WRITE),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_ENCODE_READ),
		ENUMVAL(D3D12_BARRIER_ACCESS_VIDEO_ENCODE_WRITE),
		ENUMVAL(D3D12_BARRIER_ACCESS_NO_ACCESS));
}

static FString ConvertToString(D3D12_BARRIER_LAYOUT InLayout)
{
	return ConvertEnumToString(InLayout,
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMMON),
		ENUMVAL(D3D12_BARRIER_LAYOUT_GENERIC_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_RENDER_TARGET),
		ENUMVAL(D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_SHADER_RESOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COPY_SOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COPY_DEST),
		ENUMVAL(D3D12_BARRIER_LAYOUT_RESOLVE_SOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_RESOLVE_DEST),
		ENUMVAL(D3D12_BARRIER_LAYOUT_SHADING_RATE_SOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_DECODE_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_DECODE_WRITE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_PROCESS_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_PROCESS_WRITE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_ENCODE_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_ENCODE_WRITE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COMMON),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_UNORDERED_ACCESS),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_SHADER_RESOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_SOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_DEST),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COMMON),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_UNORDERED_ACCESS),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_SHADER_RESOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_SOURCE),
		ENUMVAL(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_DEST),
#if D3D12RHI_HAS_LAYOUT_VIDEO_QUEUE_COMMON
		ENUMVAL(D3D12_BARRIER_LAYOUT_VIDEO_QUEUE_COMMON),
#endif
#if D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
		ENUMVAL(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE),
#endif
#if D3D12RHI_COMMON_AND_PRESENT_ARE_DISTINCT
		ENUMVAL(D3D12_BARRIER_LAYOUT_PRESENT),
#endif
		ENUMVAL(D3D12_BARRIER_LAYOUT_UNDEFINED));
}

static bool ResourceIsSimultaneousAccess(
	const FD3D12ResourceDesc& InDesc)
{
	return EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS);
}

static bool ResourceIsSimultaneousAccess(
	const FD3D12Resource* InResource)
{
	if (InResource)
	{
		return ResourceIsSimultaneousAccess(InResource->GetDesc());
	}
	return false;
}

#undef ENUMVAL

template <typename TArray>
struct TNumAndArray
{
	using ArrayType = TArray;

	int32 Num;
	ArrayType Array;
};

template <TNumAndArray InNumAndArray>
static consteval auto ResizeArray()
{
	using OldArrayType = decltype(InNumAndArray)::ArrayType;
	using ElementType = OldArrayType::ElementType;

	using ArrayType = TStaticArray<ElementType, InNumAndArray.Num>;

	ArrayType NewArray = {};
	int32 ArrayIdx = 0;
	for (int32 i = 0; i < InNumAndArray.Num; ++i)
	{
		NewArray[i] = InNumAndArray.Array[i];
	}
	return NewArray;
}

template <typename FilterFunc, typename TElement, uint32 InArrayNum>
static consteval auto FilterArray(
	const TStaticArray<TElement, InArrayNum>& InArray,
	const FilterFunc& InFilterFunc)
{
	using OutArrayType = TStaticArray<TElement, InArrayNum>;

	int32 NumElems = 0;
	OutArrayType FilteredArray = {};
	for (uint32 i = 0; i < InArrayNum; ++i)
	{
		const TElement& Value = InArray[i];
		if (InFilterFunc(Value))
		{
			FilteredArray[NumElems++] = Value;
		}
	}
	return TNumAndArray<OutArrayType> { NumElems, FilteredArray };
}

template <TStaticArray InArrayA, TStaticArray InArrayB>
static consteval auto GetElementsExclusiveToA()
{
	using ElementType = decltype(InArrayA)::ElementType;

	constexpr TNumAndArray NumAndArray = FilterArray(
		InArrayA,
		[](const ElementType& Value)
		{
			return !Algo::Contains(InArrayB, Value);
		});

	return ResizeArray<NumAndArray>();
}

template <TStaticArray InArrayA, TStaticArray InArrayB>
static consteval auto GetIntersectionOf()
{
	using ElementType = decltype(InArrayA)::ElementType;

	constexpr TNumAndArray NumAndArray = FilterArray(
		InArrayA,
		[](const ElementType& Value)
		{
			return Algo::Contains(InArrayB, Value);
		});

	return ResizeArray<NumAndArray>();
}

// The below tables may be variably sized based on platform properties. This lets them 
// vary in size without the ugly arithmetic necessary to explicitly calculate and specify
// the size in the type based on those platform properties. It takes an array as input,
// rather than the values themselves, to allow for a trailing comma.
template <typename T, uint32 Size>
static constexpr TStaticArray<T, Size> MakeStaticArray(const T(&InValues)[Size])
{
	// Used to capture the indices
	return []
		<uint32... Indices>
		(const T(&InValues)[Size], TIntegerSequence<uint32, Indices...>)
		{
			return TStaticArray<T, Size>(InValues[Indices]...);
		}(InValues, TMakeIntegerSequence<uint32, Size>{});
}

//
// These tables are all copied directly from the DX spec
// and are unmodified (except where specifically noted)
// All additional lookups of this information should use
// these tables or tables derived from these during compilation
// so that if the spec is updated, it's easy to incorporate and
// validate those updates by inspection alone.
//

static constexpr
D3D12_BARRIER_SYNC DirectQueueCompatibleSync =
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_INDEX_INPUT
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_DEPTH_STENCIL
	| D3D12_BARRIER_SYNC_RENDER_TARGET
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	| D3D12_BARRIER_SYNC_RAYTRACING
	| D3D12_BARRIER_SYNC_COPY
	| D3D12_BARRIER_SYNC_RESOLVE
	| D3D12_BARRIER_SYNC_EXECUTE_INDIRECT
	| D3D12_BARRIER_SYNC_PREDICATION
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_NON_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO
	| D3D12_BARRIER_SYNC_CLEAR_UNORDERED_ACCESS_VIEW;

static constexpr
D3D12_BARRIER_SYNC ComputeQueueCompatibleSync =
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	| D3D12_BARRIER_SYNC_RAYTRACING
	| D3D12_BARRIER_SYNC_COPY
	| D3D12_BARRIER_SYNC_EXECUTE_INDIRECT
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_NON_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO
	| D3D12_BARRIER_SYNC_CLEAR_UNORDERED_ACCESS_VIEW
	| D3D12_BARRIER_SYNC_SPLIT;

static constexpr
TStaticArray AccessCompatibleSync = MakeStaticArray<D3D12_BARRIER_SYNC>({
	//D3D12_BARRIER_ACCESS_VERTEX_BUFFER
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_ALL_SHADING,

	//D3D12_BARRIER_ACCESS_CONSTANT_BUFFER
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_ALL_SHADING,

	//D3D12_BARRIER_ACCESS_INDEX_BUFFER
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_INDEX_INPUT
	| D3D12_BARRIER_SYNC_DRAW,

	//D3D12_BARRIER_ACCESS_RENDER_TARGET
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_RENDER_TARGET,

	//D3D12_BARRIER_ACCESS_UNORDERED_ACCESS
	D3D12_BARRIER_SYNC_ALL //-V501
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_CLEAR_UNORDERED_ACCESS_VIEW
	| D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO
	| D3D12_BARRIER_SYNC_RAYTRACING,

	//D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_DEPTH_STENCIL,

	//D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_DEPTH_STENCIL,

	//D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	// @TODO - This one isn't listed in the spec, but logic and validation tells us it's compatible
	| D3D12_BARRIER_SYNC_NON_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_RAYTRACING,

	//D3D12_BARRIER_ACCESS_STREAM_OUTPUT
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VERTEX_SHADING
	| D3D12_BARRIER_SYNC_DRAW
	| D3D12_BARRIER_SYNC_ALL_SHADING,

	//D3D12_BARRIER_ACCESS_INDIRECT_ARGUMENT
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_EXECUTE_INDIRECT,

#if 0
	// An Alias for INDIRECT_ARGUMENT
	//D3D12_BARRIER_ACCESS_PREDICATION
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_PREDICATION,
#endif

	//D3D12_BARRIER_ACCESS_COPY_DEST
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_COPY,

	//D3D12_BARRIER_ACCESS_COPY_SOURCE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_COPY,

	//D3D12_BARRIER_ACCESS_RESOLVE_DEST
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_RESOLVE,

	//D3D12_BARRIER_ACCESS_RESOLVE_SOURCE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_RESOLVE,

	//D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
	| D3D12_BARRIER_SYNC_RAYTRACING
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO,

	//D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_COMPUTE_SHADING
#if !PLATFORM_REQUIRES_SYNC_RAYTRACING_NOT_COMPATIBLE_WITH_ACCESS_AS_WRITE
	| D3D12_BARRIER_SYNC_RAYTRACING
#endif
	| D3D12_BARRIER_SYNC_ALL_SHADING
	| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
	| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE,

	//D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_PIXEL_SHADING
	| D3D12_BARRIER_SYNC_ALL_SHADING,

	//D3D12_BARRIER_ACCESS_VIDEO_DECODE_READ
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_DECODE,


	//D3D12_BARRIER_ACCESS_VIDEO_DECODE_WRITE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_DECODE,

	//D3D12_BARRIER_ACCESS_VIDEO_PROCESS_READ
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_PROCESS,

	//D3D12_BARRIER_ACCESS_VIDEO_PROCESS_WRITE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_PROCESS,

	//D3D12_BARRIER_ACCESS_VIDEO_ENCODE_READ
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_ENCODE,

	//D3D12_BARRIER_ACCESS_VIDEO_ENCODE_WRITE
	D3D12_BARRIER_SYNC_ALL
	| D3D12_BARRIER_SYNC_VIDEO_ENCODE,

	// Omitted since it's non-contigious with the other bits
	//D3D12_BARRIER_ACCESS_NO_ACCESS
});

// Simultaneous access textures can be accessed using any access bits except these.
// This info is rather hidden in the current version of the spec. It appears in the
// section titled "Equivalent D3D12_BARRIER_ACCESS bit for each D3D12_RESOURCE_STATES bit"
static constexpr
D3D12_BARRIER_ACCESS SimultaneousAccessIncompatibleAccess =
	D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE;

static constexpr
TStaticArray LayoutCompatibleAccess = MakeStaticArray<D3D12_BARRIER_ACCESS>({
	//D3D12_BARRIER_LAYOUT_COMMON
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_DEST
	| D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_GENERIC_READ
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_RENDER_TARGET
	D3D12_BARRIER_ACCESS_RENDER_TARGET,

	//D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS
	D3D12_BARRIER_ACCESS_UNORDERED_ACCESS,

	//D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE
	D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE,

	//D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_READ
	D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ,

	//D3D12_BARRIER_LAYOUT_SHADER_RESOURCE
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE,

	//D3D12_BARRIER_LAYOUT_COPY_SOURCE
	D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_COPY_DEST
	D3D12_BARRIER_ACCESS_COPY_DEST,

	//D3D12_BARRIER_LAYOUT_RESOLVE_SOURCE
	D3D12_BARRIER_ACCESS_RESOLVE_SOURCE,

	//D3D12_BARRIER_LAYOUT_RESOLVE_DEST
	D3D12_BARRIER_ACCESS_RESOLVE_DEST,

	//D3D12_BARRIER_LAYOUT_SHADING_RATE_SOURCE
	D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE,

	//D3D12_BARRIER_LAYOUT_VIDEO_DECODE_READ
	D3D12_BARRIER_ACCESS_VIDEO_DECODE_READ,

	//D3D12_BARRIER_LAYOUT_VIDEO_DECODE_WRITE
	D3D12_BARRIER_ACCESS_VIDEO_DECODE_WRITE,

	//D3D12_BARRIER_LAYOUT_VIDEO_PROCESS_READ
	D3D12_BARRIER_ACCESS_VIDEO_PROCESS_READ,

	//D3D12_BARRIER_LAYOUT_VIDEO_PROCESS_WRITE
	D3D12_BARRIER_ACCESS_VIDEO_PROCESS_WRITE,

	//D3D12_BARRIER_LAYOUT_VIDEO_ENCODE_READ
	D3D12_BARRIER_ACCESS_VIDEO_ENCODE_READ,

	//D3D12_BARRIER_LAYOUT_VIDEO_ENCODE_WRITE
	D3D12_BARRIER_ACCESS_VIDEO_ENCODE_WRITE,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COMMON
	D3D12_BARRIER_ACCESS_COPY_SOURCE
	|D3D12_BARRIER_ACCESS_COPY_DEST
	|D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	|D3D12_BARRIER_ACCESS_UNORDERED_ACCESS,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE
	| D3D12_BARRIER_ACCESS_RESOLVE_SOURCE,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_UNORDERED_ACCESS
	D3D12_BARRIER_ACCESS_UNORDERED_ACCESS,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_SHADER_RESOURCE
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_SOURCE
	D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_DEST
	D3D12_BARRIER_ACCESS_COPY_DEST,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COMMON
	D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_COPY_DEST
	| D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_UNORDERED_ACCESS,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_UNORDERED_ACCESS
	D3D12_BARRIER_ACCESS_UNORDERED_ACCESS,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_SHADER_RESOURCE
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_SOURCE
	D3D12_BARRIER_ACCESS_COPY_SOURCE,

	//D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_DEST
	D3D12_BARRIER_ACCESS_COPY_DEST,

#if D3D12RHI_HAS_LAYOUT_VIDEO_QUEUE_COMMON
	//D3D12_BARRIER_LAYOUT_VIDEO_QUEUE_COMMON,
	D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_COPY_DEST,
#else
	// RESERVED
	D3D12_BARRIER_ACCESS_NO_ACCESS,
#endif

#if D3D12RHI_COMMON_AND_PRESENT_ARE_DISTINCT
	//D3D12_BARRIER_LAYOUT_PRESENT
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_DEST
	| D3D12_BARRIER_ACCESS_COPY_SOURCE,
#endif

#if D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
	// D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
	D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE
	| D3D12_BARRIER_ACCESS_RESOLVE_SOURCE,
#endif
});

// These are not the same layout on some platforms but the compatible access should be the same
static_assert(
	LayoutCompatibleAccess[D3D12_BARRIER_LAYOUT_COMMON]
		== LayoutCompatibleAccess[D3D12_BARRIER_LAYOUT_PRESENT]);

static constexpr
D3D12_BARRIER_SYNC AllQueueCompatibleSync =
	DirectQueueCompatibleSync & ComputeQueueCompatibleSync;

static constexpr
D3D12_BARRIER_ACCESS DirectQueueCompatibleAccess =
	D3D12_BARRIER_ACCESS_VERTEX_BUFFER
	| D3D12_BARRIER_ACCESS_CONSTANT_BUFFER
	| D3D12_BARRIER_ACCESS_INDEX_BUFFER
	| D3D12_BARRIER_ACCESS_RENDER_TARGET
	| D3D12_BARRIER_ACCESS_UNORDERED_ACCESS
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_STREAM_OUTPUT
	| D3D12_BARRIER_ACCESS_INDIRECT_ARGUMENT
	| D3D12_BARRIER_ACCESS_COPY_DEST
	| D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_RESOLVE_DEST
	| D3D12_BARRIER_ACCESS_RESOLVE_SOURCE
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE
	| D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE
	| D3D12_BARRIER_ACCESS_PREDICATION
	// Not in spec
	| D3D12_BARRIER_ACCESS_NO_ACCESS;

static constexpr
D3D12_BARRIER_ACCESS ComputeQueueCompatibleAccess =
	// @TODO - Spec lists this but logic and validation disagree
	//D3D12_BARRIER_ACCESS_VERTEX_BUFFER
	D3D12_BARRIER_ACCESS_CONSTANT_BUFFER
	| D3D12_BARRIER_ACCESS_UNORDERED_ACCESS
	| D3D12_BARRIER_ACCESS_SHADER_RESOURCE
	| D3D12_BARRIER_ACCESS_INDIRECT_ARGUMENT
	| D3D12_BARRIER_ACCESS_COPY_DEST
	| D3D12_BARRIER_ACCESS_COPY_SOURCE
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ
	| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE
	| D3D12_BARRIER_ACCESS_PREDICATION
	// Not in spec
	| D3D12_BARRIER_ACCESS_NO_ACCESS
#if PLATFORM_REQUIRES_ENHANCED_BARRIERS_GFX_ONLY_READ_BITS_HACK
	// @TODO -
	// !!!!!!!!!! HUGE HACK! !!!!!!!!!!!!
	// This isn't officially compatible
	// But we have no other way to set certain read-only gfx bits
	// when a resource is being read by multiple pipes at once
	| D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ
	| D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE
	| D3D12_BARRIER_ACCESS_RESOLVE_SOURCE
#endif
	;

static constexpr
D3D12_BARRIER_ACCESS AllQueueCompatibleAccess =
	DirectQueueCompatibleAccess & ComputeQueueCompatibleAccess;

static constexpr
TStaticArray DirectQueueCompatibleLayouts = MakeStaticArray<D3D12_BARRIER_LAYOUT>({
	D3D12_BARRIER_LAYOUT_COMMON,
	D3D12_BARRIER_LAYOUT_GENERIC_READ,
	D3D12_BARRIER_LAYOUT_RENDER_TARGET,
	D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS,
	D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE,
	D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_READ,
	D3D12_BARRIER_LAYOUT_SHADER_RESOURCE,
	D3D12_BARRIER_LAYOUT_COPY_SOURCE,
	D3D12_BARRIER_LAYOUT_COPY_DEST,
	D3D12_BARRIER_LAYOUT_RESOLVE_SOURCE,
	D3D12_BARRIER_LAYOUT_RESOLVE_DEST,
	D3D12_BARRIER_LAYOUT_SHADING_RATE_SOURCE,
	// @TODO - Not listed in the spec, but logic tells us it's compatible
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COMMON,
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ,
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_UNORDERED_ACCESS,
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_SHADER_RESOURCE,
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_SOURCE,
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_DEST,
#if D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE,
#endif
	// Not in spec
	D3D12_BARRIER_LAYOUT_UNDEFINED,
});

static constexpr
TStaticArray ComputeQueueCompatibleLayouts = MakeStaticArray<D3D12_BARRIER_LAYOUT>({
	D3D12_BARRIER_LAYOUT_COMMON,
	D3D12_BARRIER_LAYOUT_GENERIC_READ,
	D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS,
	D3D12_BARRIER_LAYOUT_SHADER_RESOURCE,
	D3D12_BARRIER_LAYOUT_COPY_SOURCE,
	D3D12_BARRIER_LAYOUT_COPY_DEST,
	// @TODO - Not listed in the spec, but logic tells us it's compatible
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COMMON,
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ,
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_UNORDERED_ACCESS,
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_SHADER_RESOURCE,
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_SOURCE,
	D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_DEST,
#if D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE,
#endif
	// Not in spec
	D3D12_BARRIER_LAYOUT_UNDEFINED,
#if PLATFORM_REQUIRES_ENHANCED_BARRIERS_GFX_ONLY_READ_BITS_HACK
	// @TODO -
	// !!!!!!!!!! HUGE HACK! !!!!!!!!!!!!
	// This isn't officially compatible
	// But we have no other way to set certain read-only gfx bits
	// when a resource is being read by multiple pipes at once
	D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ,
#endif
});

static constexpr
TStaticArray AllQueueCompatibleLayouts =
	GetIntersectionOf<
		DirectQueueCompatibleLayouts,
		ComputeQueueCompatibleLayouts>();

static constexpr
TStaticArray DirectQueueSpecificLayouts =
	GetElementsExclusiveToA<
		DirectQueueCompatibleLayouts,
		ComputeQueueCompatibleLayouts>();

static constexpr
TStaticArray ComputeQueueSpecificLayouts =
	GetElementsExclusiveToA<
		ComputeQueueCompatibleLayouts,
		DirectQueueCompatibleLayouts>();


static bool LayoutIsCompatibleWithQueue(
	D3D12_BARRIER_LAYOUT InLayout,
	ERHIPipeline InPipe)
{
	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return Algo::Contains(DirectQueueCompatibleLayouts, InLayout);
	case ERHIPipeline::AsyncCompute:
		return Algo::Contains(ComputeQueueCompatibleLayouts, InLayout);
	case ERHIPipeline::All:
		return Algo::Contains(AllQueueCompatibleLayouts, InLayout);
	default:
		checkNoEntry();
		return false;
	}
}

static bool LayoutIsQueueSpecific(
	D3D12_BARRIER_LAYOUT InLayout,
	ERHIPipeline InPipe)
{
	check(LayoutIsCompatibleWithQueue(InLayout, InPipe));

	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return Algo::Contains(DirectQueueSpecificLayouts, InLayout);
	case ERHIPipeline::AsyncCompute:
		return Algo::Contains(ComputeQueueSpecificLayouts, InLayout);
	case ERHIPipeline::All:
		return false;
	default:
		checkNoEntry();
		return false;
	}
}

static bool SyncIsCompatibleWithQueue(
	D3D12_BARRIER_SYNC InSync,
	ERHIPipeline InPipe)
{
	if (InSync == D3D12_BARRIER_SYNC_NONE)
	{
		return true;
	}

	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return EnumOnlyContainsFlags(InSync, DirectQueueCompatibleSync);
	case ERHIPipeline::AsyncCompute:
		return EnumOnlyContainsFlags(InSync, ComputeQueueCompatibleSync);
	case ERHIPipeline::All:
		return EnumOnlyContainsFlags(InSync, AllQueueCompatibleSync);
	default:
		checkNoEntry();
		return false;
	}
}

static D3D12_BARRIER_SYNC FilterToQueueCompatibleSync(
	D3D12_BARRIER_SYNC InSync,
	ERHIPipeline InPipe)
{
	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return InSync & DirectQueueCompatibleSync;
	case ERHIPipeline::AsyncCompute:
		return InSync & ComputeQueueCompatibleSync;
	case ERHIPipeline::All:
		return InSync & AllQueueCompatibleSync;
	default:
		checkNoEntry();
		return {};
	}
}

static bool AccessIsCompatibleWithQueue(
	D3D12_BARRIER_ACCESS InAccess,
	ERHIPipeline InPipe)
{
	if (InAccess == D3D12_BARRIER_ACCESS_NO_ACCESS)
	{
		return true;
	}

	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return EnumOnlyContainsFlags(InAccess, DirectQueueCompatibleAccess);
	case ERHIPipeline::AsyncCompute:
		return EnumOnlyContainsFlags(InAccess, ComputeQueueCompatibleAccess);
	case ERHIPipeline::All:
		return EnumOnlyContainsFlags(InAccess, AllQueueCompatibleAccess);
	default:
		checkNoEntry();
		return false;
	}
}

static bool AccessIsCompatibleWithLayout(
	D3D12_BARRIER_ACCESS InAccess,
	D3D12_BARRIER_LAYOUT InLayout,
	const FD3D12Resource* InResource)
{
	if (InAccess == D3D12_BARRIER_ACCESS_NO_ACCESS)
	{
		return true;
	}

	if (ResourceIsSimultaneousAccess(InResource))
	{
		return !EnumHasAnyFlags(InAccess, SimultaneousAccessIncompatibleAccess);
	}

	if (static_cast<int32>(InLayout) >= LayoutCompatibleAccess.Num())
	{
		return false;
	}
	return EnumOnlyContainsFlags(InAccess, LayoutCompatibleAccess[InLayout]);
}

static D3D12_BARRIER_ACCESS FilterToQueueCompatibleAccess(
	D3D12_BARRIER_ACCESS InAccess,
	ERHIPipeline InPipe)
{
	switch (InPipe)
	{
	case ERHIPipeline::Graphics:
		return InAccess & DirectQueueCompatibleAccess;
	case ERHIPipeline::AsyncCompute:
		return InAccess & ComputeQueueCompatibleAccess;
	case ERHIPipeline::All:
		return InAccess & AllQueueCompatibleAccess;
	default:
		checkNoEntry();
		return {};
	}
}

static D3D12_BARRIER_LAYOUT GetQueueAgnosticVersionOfLayout(
	D3D12_BARRIER_LAYOUT InLayout)
{
	switch (InLayout)
	{
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COMMON:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COMMON:
		return D3D12_BARRIER_LAYOUT_COMMON;
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ:
		return D3D12_BARRIER_LAYOUT_GENERIC_READ;
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_UNORDERED_ACCESS:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_UNORDERED_ACCESS:
		return D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS;
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_SHADER_RESOURCE:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_SHADER_RESOURCE:
		return D3D12_BARRIER_LAYOUT_SHADER_RESOURCE;
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_SOURCE:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_SOURCE:
		return D3D12_BARRIER_LAYOUT_COPY_SOURCE;
	case D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_DEST:
	case D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_DEST:
		return D3D12_BARRIER_LAYOUT_COPY_DEST;
	default:
		return InLayout;
	}
}

static bool BarrierValuesAreCompatibleWithQueue(
	D3D12_BARRIER_SYNC InSync,
	D3D12_BARRIER_ACCESS InAccess,
	D3D12_BARRIER_LAYOUT InLayout,
	ERHIPipeline InPipe)
{
	return SyncIsCompatibleWithQueue(InSync, InPipe)
		&& AccessIsCompatibleWithQueue(InAccess, InPipe)
		&& LayoutIsCompatibleWithQueue(InLayout, InPipe);
}

static bool BarrierValuesAreCompatibleWithQueue(
	FD3D12BarrierValues InBarrierValues,
	ERHIPipeline InPipe)
{
	return BarrierValuesAreCompatibleWithQueue(
		InBarrierValues.Sync,
		InBarrierValues.Access,
		InBarrierValues.Layout,
		InPipe);
}

static constexpr
D3D12_BARRIER_SYNC GetAccessCompatibleSync(
	D3D12_BARRIER_ACCESS InAccess)
{
	using AccessUnsignedUnderlyingType =
		std::make_unsigned_t<
			std::underlying_type_t<D3D12_BARRIER_ACCESS>>;
	
	using SyncUnsignedUnderlyingType =
		std::make_unsigned_t<
			std::underlying_type_t<D3D12_BARRIER_SYNC>>;

	if (InAccess == D3D12_BARRIER_ACCESS_COMMON
		|| InAccess == D3D12_BARRIER_ACCESS_NO_ACCESS)
	{
		return BitCast<D3D12_BARRIER_SYNC>(~AccessUnsignedUnderlyingType(0));
	}

	const uint32 Bits = sizeof(AccessUnsignedUnderlyingType) * 8;
	const auto UnsignedAccessValue = static_cast<AccessUnsignedUnderlyingType>(InAccess);

	SyncUnsignedUnderlyingType CompatibleSync = 0;
	uint32 LeadingZeros = std::countl_zero(UnsignedAccessValue);
	while (LeadingZeros < Bits)
	{
		const uint32 FirstSetBitIdx = Bits - LeadingZeros - 1;
		const auto Mask = ((AccessUnsignedUnderlyingType(1)) << FirstSetBitIdx) - 1;
		CompatibleSync |= AccessCompatibleSync[FirstSetBitIdx];
		LeadingZeros = std::countl_zero(UnsignedAccessValue & Mask);
	}

	return static_cast<D3D12_BARRIER_SYNC>(CompatibleSync);
}

static bool AccessIsCompatibleWithSync(
	D3D12_BARRIER_ACCESS InAccess,
	D3D12_BARRIER_SYNC InSync)
{
	// Note that this is more expensive than the other checks,
	// try not to use it other than for validation
	return EnumHasAllFlags(GetAccessCompatibleSync(InAccess), InSync);
}

static bool SyncAndAccessAreComputeWrite(
	D3D12_BARRIER_SYNC InSync,
	D3D12_BARRIER_ACCESS InAccess)
{
	// These mean compute write access
	static constexpr D3D12_BARRIER_ACCESS ComputeWriteAccessFlags =
		D3D12_BARRIER_ACCESS_UNORDERED_ACCESS
		| D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE;

	static constexpr D3D12_BARRIER_SYNC ComputeSyncFlags =
		GetAccessCompatibleSync(ComputeWriteAccessFlags);


	return EnumHasAnyFlags(InSync, ComputeSyncFlags)
		// ACCESS_COMMON is used to describe all access bits compatible with the sync scope
		// so if it's used with one of the compute work sync'ing bits then it implies compute write
		&& ((InAccess == D3D12_BARRIER_ACCESS_COMMON)
			// Otherwise, we can explicitly look for the compute write access bits
			|| EnumHasAnyFlags(InAccess, ComputeWriteAccessFlags));
}

template <typename T>
static bool CheckBarrierValuesAreCompatible(
	const TArrayView<T>& InBarriers,
	ERHIPipeline InPipeline)
{
	for (const T& Barrier : InBarriers)
	{
		check(SyncIsCompatibleWithQueue(Barrier.SyncBefore, InPipeline));
		check(SyncIsCompatibleWithQueue(Barrier.SyncAfter, InPipeline));
		check(AccessIsCompatibleWithQueue(Barrier.AccessBefore, InPipeline));
		check(AccessIsCompatibleWithQueue(Barrier.AccessAfter, InPipeline));

		check(AccessIsCompatibleWithSync(Barrier.AccessBefore, Barrier.SyncBefore));
		check(AccessIsCompatibleWithSync(Barrier.AccessAfter, Barrier.SyncAfter));

		if constexpr (std::is_same_v<D3D12_TEXTURE_BARRIER, T>)
		{
			check(LayoutIsCompatibleWithQueue(Barrier.LayoutBefore, InPipeline));
			check(LayoutIsCompatibleWithQueue(Barrier.LayoutAfter, InPipeline));

			check(AccessIsCompatibleWithLayout(Barrier.AccessBefore, Barrier.LayoutBefore, Barrier.pResource));
			check(AccessIsCompatibleWithLayout(Barrier.AccessAfter, Barrier.LayoutAfter, Barrier.pResource));
		}
	}

	return true;
}

static bool BarrierCanBeDiscarded(
	D3D12_BARRIER_SYNC InBeforeSync,
	D3D12_BARRIER_SYNC InAfterSync,
	D3D12_BARRIER_ACCESS InBeforeAccess,
	D3D12_BARRIER_ACCESS InAfterAccess,
	D3D12_BARRIER_LAYOUT InBeforeLayout,
	D3D12_BARRIER_LAYOUT InAfterLayout)
{
	return (InBeforeSync   == InAfterSync)
		&& (InBeforeAccess == InAfterAccess)
		&& (InBeforeLayout == InAfterLayout)
		// ComputeWrite->ComputeWrite can't be skipped because
		// each compute unit may have its own caches
		&& !SyncAndAccessAreComputeWrite(InBeforeSync, InBeforeAccess);
}

static bool BarrierCanBeDiscarded(
	const FD3D12BarrierValues& InBeforeValues,
	const FD3D12BarrierValues& InAfterValues)
{
	return BarrierCanBeDiscarded(
		InBeforeValues.Sync,
		InAfterValues.Sync,
		InBeforeValues.Access,
		InAfterValues.Access,
		InBeforeValues.Layout,
		InAfterValues.Layout);
}

static D3D12_BARRIER_SYNC GetEBSync(
	ED3D12Access InD3D12Access,
	ERHIPipeline InPipe)
{
	if (InPipe == ERHIPipeline::None)
	{
		checkNoEntry();
		return D3D12_BARRIER_SYNC_NONE;
	}

	if (InD3D12Access == ED3D12Access::Unknown)
	{
		return D3D12_BARRIER_SYNC_ALL;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Common))
	{
		return D3D12_BARRIER_SYNC_ALL;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::GenericRead))
	{
		return D3D12_BARRIER_SYNC_ALL;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Discard))
	{
		check(InD3D12Access == ED3D12Access::Discard);
		return D3D12_BARRIER_SYNC_NONE;
	}

	D3D12_BARRIER_SYNC EBSync = {};
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CPURead))
	{
		EBSync |= D3D12_BARRIER_SYNC_NONE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Present))
	{
#if PLATFORM_WINDOWS
		EBSync |= D3D12_BARRIER_SYNC_ALL;
#else
		EBSync |= D3D12_BARRIER_SYNC_NONE;
#endif
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::IndirectArgs))
	{
		EBSync |= D3D12_BARRIER_SYNC_EXECUTE_INDIRECT;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::VertexOrIndexBuffer))
	{
		// @TODO - This sucks... need more specific RHI bits or to pass in a resource description
		EBSync |= D3D12_BARRIER_SYNC_VERTEX_SHADING
				| D3D12_BARRIER_SYNC_INDEX_INPUT
				// Needed to cover constant buffers
				| D3D12_BARRIER_SYNC_ALL_SHADING;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::SRVCompute))
	{
		EBSync |= D3D12_BARRIER_SYNC_COMPUTE_SHADING;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::SRVGraphicsPixel))
	{
		EBSync |= D3D12_BARRIER_SYNC_PIXEL_SHADING;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::SRVGraphicsNonPixel))
	{
		EBSync |= D3D12_BARRIER_SYNC_NON_PIXEL_SHADING;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CopySrc))
	{
		EBSync |= D3D12_BARRIER_SYNC_COPY;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ResolveSrc))
	{
		EBSync |= D3D12_BARRIER_SYNC_RESOLVE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::DSVRead))
	{
		EBSync |= D3D12_BARRIER_SYNC_DEPTH_STENCIL;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::UAVCompute))
	{
		EBSync |= D3D12_BARRIER_SYNC_COMPUTE_SHADING;;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::UAVGraphics))
	{
		EBSync |= D3D12_BARRIER_SYNC_VERTEX_SHADING
				| D3D12_BARRIER_SYNC_PIXEL_SHADING;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::RTV))
	{
		EBSync |= D3D12_BARRIER_SYNC_RENDER_TARGET;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CopyDest))
	{
		EBSync |= D3D12_BARRIER_SYNC_COPY;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ResolveDst))
	{
		EBSync |= D3D12_BARRIER_SYNC_RESOLVE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::DSVWrite))
	{
		EBSync |= D3D12_BARRIER_SYNC_DEPTH_STENCIL;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::BVHRead))
	{
		EBSync |= D3D12_BARRIER_SYNC_RAYTRACING
				| D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
				| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE
				| D3D12_BARRIER_SYNC_EMIT_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::BVHWrite))
	{
		EBSync |= 
#if !PLATFORM_REQUIRES_SYNC_RAYTRACING_NOT_COMPATIBLE_WITH_ACCESS_AS_WRITE
			D3D12_BARRIER_SYNC_RAYTRACING |
#endif
			D3D12_BARRIER_SYNC_BUILD_RAYTRACING_ACCELERATION_STRUCTURE
			| D3D12_BARRIER_SYNC_COPY_RAYTRACING_ACCELERATION_STRUCTURE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ShadingRateSource))
	{
		EBSync |= D3D12_BARRIER_SYNC_PIXEL_SHADING;
	}

	// Make sure we at least set one of the bits or it's
	// really only CPURead which requires no GPU sync
	static_assert(D3D12_BARRIER_SYNC_NONE == 0);
	static constexpr ED3D12Access AccessBitsThatCanBeNone =
		ED3D12Access::Present |
		ED3D12Access::CPURead;

	check((EBSync != D3D12_BARRIER_SYNC_NONE)
		|| EnumHasAnyOneFlag(InD3D12Access, AccessBitsThatCanBeNone));

	if (InPipe != ERHIPipeline::All)
	{
		check(EnumHasOneFlag(InPipe));
		return FilterToQueueCompatibleSync(EBSync, InPipe);
	}
	else
	{
		return EBSync;
	}
}

static D3D12_BARRIER_SYNC GetEBSync(
	ERHIAccess InRHIAccess,
	ERHIPipeline InPipe)
{
	return GetEBSync(ConvertToD3D12Access(InRHIAccess), InPipe);
}

static D3D12_BARRIER_ACCESS GetEBAccess(
	ED3D12Access InD3D12Access,
	ERHIPipeline InPipe)
{
	if (InPipe == ERHIPipeline::None)
	{
		checkNoEntry();
		return D3D12_BARRIER_ACCESS_NO_ACCESS;
	}

	if (InD3D12Access == ED3D12Access::Unknown)
	{
		return D3D12_BARRIER_ACCESS_COMMON;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Common))
	{
		return D3D12_BARRIER_ACCESS_COMMON;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::GenericRead))
	{
		return D3D12_BARRIER_ACCESS_COMMON;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Discard))
	{
		check(InD3D12Access == ED3D12Access::Discard);
		return D3D12_BARRIER_ACCESS_NO_ACCESS;
	}

	D3D12_BARRIER_ACCESS EBAccess = {};
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CPURead))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_COMMON;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::Present))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_COMMON;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::IndirectArgs))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_INDIRECT_ARGUMENT;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::VertexOrIndexBuffer))
	{
		// @TODO - This sucks... need more specific RHI bits or to pass in a resource description
		EBAccess |= D3D12_BARRIER_ACCESS_VERTEX_BUFFER
				  | D3D12_BARRIER_ACCESS_INDEX_BUFFER
				  | D3D12_BARRIER_ACCESS_CONSTANT_BUFFER;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::SRVMask))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_SHADER_RESOURCE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CopySrc))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_COPY_SOURCE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ResolveSrc))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_RESOLVE_SOURCE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::DSVRead))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::UAVMask))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_UNORDERED_ACCESS;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::RTV))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_RENDER_TARGET;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CopyDest))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_COPY_DEST;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ResolveDst))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_RESOLVE_DEST;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::DSVWrite))
	{
#if PLATFORM_REQUIRES_LAYOUT_DEPTH_STENCIL_WRITE_NOT_COMPATIBLE_WITH_ACCESS_DEPTH_STENCIL_READ
		// @TODO - The validation layer claims READ isn't compatible with the WRITE layout
		//         so for now hack out the READ bit if we're also setting WRITE
		EBAccess &= ~D3D12_BARRIER_ACCESS_DEPTH_STENCIL_READ;
#endif
		EBAccess |= D3D12_BARRIER_ACCESS_DEPTH_STENCIL_WRITE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::BVHRead))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_READ;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::BVHWrite))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_RAYTRACING_ACCELERATION_STRUCTURE_WRITE;
	}
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::ShadingRateSource))
	{
		EBAccess |= D3D12_BARRIER_ACCESS_SHADING_RATE_SOURCE;
	}

	if (InPipe != ERHIPipeline::All)
	{
		check(EnumHasOneFlag(InPipe));
		return FilterToQueueCompatibleAccess(EBAccess, InPipe);
	}
	else
	{
		return EBAccess;
	}
}

static D3D12_BARRIER_ACCESS GetEBAccess(
	ERHIAccess InRHIAccess,
	ERHIPipeline InPipe)
{
	return GetEBAccess(ConvertToD3D12Access(InRHIAccess), InPipe);
}

static D3D12_BARRIER_LAYOUT GetEBLayout(
	ED3D12Access InD3D12Access,
	ERHIPipeline InRHIPipe,
	const FD3D12Texture* InTexture)
{
	if (InD3D12Access == ED3D12Access::Unknown)
	{
		checkNoEntry();
		return D3D12_BARRIER_LAYOUT_UNDEFINED;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::IndirectArgs
									   | ED3D12Access::VertexOrIndexBuffer
									   | ED3D12Access::BVHRead
									   | ED3D12Access::BVHWrite))
	{
		// These are all buffer flags and the resource does not have a layout
		checkNoEntry();
		return D3D12_BARRIER_LAYOUT_UNDEFINED;
	}

	if (InTexture && ResourceIsSimultaneousAccess(InTexture->GetResource()))
	{
		// Simultaneous access textures have an immutable layout
		return D3D12_BARRIER_LAYOUT_UNDEFINED;
	}

	// Special cases
	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::CPURead | ED3D12Access::Common))
	{
		return D3D12_BARRIER_LAYOUT_COMMON;
	}

	if (EnumHasAnyFlags(InD3D12Access, ED3D12Access::GenericRead))
	{
		switch (InRHIPipe)
		{
		case ERHIPipeline::Graphics:
			return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ;
		case ERHIPipeline::AsyncCompute:
			return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ;
		default:
			return D3D12_BARRIER_LAYOUT_GENERIC_READ;
		}
	}

	if (EnumHasOneFlag(InD3D12Access))
	{
		// First check the 1:1 translations
		switch (InD3D12Access)
		{
		case ED3D12Access::Present:
			return D3D12_BARRIER_LAYOUT_PRESENT;
		case ED3D12Access::CopySrc:
			switch (InRHIPipe)
			{
			case ERHIPipeline::Graphics:
				return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_SOURCE;
			case ERHIPipeline::AsyncCompute:
				return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_SOURCE;
			default:
				return D3D12_BARRIER_LAYOUT_COPY_SOURCE;
			}
		case ED3D12Access::ResolveSrc:
			return D3D12_BARRIER_LAYOUT_RESOLVE_SOURCE;
		case ED3D12Access::DSVRead:
			return D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_READ;
		case ED3D12Access::RTV:
			return D3D12_BARRIER_LAYOUT_RENDER_TARGET;
		case ED3D12Access::CopyDest:
			switch (InRHIPipe)
			{
			case ERHIPipeline::Graphics:
				return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COPY_DEST;
			case ERHIPipeline::AsyncCompute:
				return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COPY_DEST;
			default:
				return D3D12_BARRIER_LAYOUT_COPY_DEST;
			}
		case ED3D12Access::ResolveDst:
			return D3D12_BARRIER_LAYOUT_RESOLVE_DEST;
		case ED3D12Access::DSVWrite:
			return D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE;
		case ED3D12Access::Discard:
			return D3D12_BARRIER_LAYOUT_UNDEFINED;
		}
	}

	// Special read+write case for depth stencil
	if (InD3D12Access == (ED3D12Access::DSVRead | ED3D12Access::DSVWrite))
	{
		return D3D12_BARRIER_LAYOUT_DEPTH_STENCIL_WRITE;
	}

	// Now try the sets of flags that have the same layout translations
	if (EnumOnlyContainsFlags(InD3D12Access, ED3D12Access::SRVMask))
	{
		D3D12_BARRIER_LAYOUT ExtraLayoutBits = {};
		if (InTexture && InTexture->SkipsFastClearFinalize())
		{
			ExtraLayoutBits = GetSkipFastClearEliminateLayoutFlags();
		}

		switch (InRHIPipe)
		{
		case ERHIPipeline::Graphics:
			return static_cast<D3D12_BARRIER_LAYOUT>(D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_SHADER_RESOURCE | ExtraLayoutBits);
		case ERHIPipeline::AsyncCompute:
			return static_cast<D3D12_BARRIER_LAYOUT>(D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_SHADER_RESOURCE | ExtraLayoutBits);
		default:
			return D3D12_BARRIER_LAYOUT_SHADER_RESOURCE;
		}
	}
	if (EnumOnlyContainsFlags(InD3D12Access, ED3D12Access::UAVMask))
	{
		switch (InRHIPipe)
		{
		case ERHIPipeline::Graphics:
			return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_UNORDERED_ACCESS;
		case ERHIPipeline::AsyncCompute:
			return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_UNORDERED_ACCESS;
		default:
			return D3D12_BARRIER_LAYOUT_UNORDERED_ACCESS;
		}
	}

	// And finally check for multiple read states
	if (EnumOnlyContainsFlags(InD3D12Access, ED3D12Access::ReadOnlyMask))
	{
		static constexpr
		ED3D12Access GfxOnlyGenericReadBits =
			// Other gfx-only bits excluded by the compute compatible versions are
			// for buffer resources which have no defined layout so won't get here.
			ED3D12Access::DSVRead
			| ED3D12Access::ShadingRateSource
		  	| ED3D12Access::ResolveSrc;

		switch (InRHIPipe)
		{
		case ERHIPipeline::Graphics:
			return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ;
		case ERHIPipeline::AsyncCompute:
			check(!EnumHasAnyFlags(InD3D12Access, GfxOnlyGenericReadBits));
			return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_GENERIC_READ;
		case ERHIPipeline::All:
			if (EnumHasAnyFlags(InD3D12Access, GfxOnlyGenericReadBits))
			{
#if PLATFORM_REQUIRES_ENHANCED_BARRIERS_GFX_ONLY_READ_BITS_HACK
				// @TODO - This is to work around a hole in the API that won't allow DSVRead
				//         access with any compute queue compatible layout.
				return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ;
#elif D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE
				return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE;
#endif
			}
			return D3D12_BARRIER_LAYOUT_GENERIC_READ;
		default:
			checkNoEntry();
			return D3D12_BARRIER_LAYOUT_GENERIC_READ;
		}
	}

	// Must be a combination of read and write flags
	check(EnumHasAnyFlags(InD3D12Access, ED3D12Access::ReadableMask) &&
		  EnumHasAnyFlags(InD3D12Access, ED3D12Access::WritableMask));

	switch (InRHIPipe)
	{
		case ERHIPipeline::Graphics:
			return D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_COMMON;
		case ERHIPipeline::AsyncCompute:
			return D3D12_BARRIER_LAYOUT_COMPUTE_QUEUE_COMMON;
		default:
			check(!EnumHasAnyFlags(InD3D12Access, ED3D12Access::UAVMask));
			return D3D12_BARRIER_LAYOUT_COMMON;
	}
}

static D3D12_BARRIER_LAYOUT GetEBLayout(
	ERHIAccess InRHIAccess,
	ERHIPipeline InRHIPipe,
	const FD3D12Texture* InTexture)
{
	return GetEBLayout(ConvertToD3D12Access(InRHIAccess), InRHIPipe, InTexture);
}

#if D3D12RHI_BARRIER_LOGGING

static void LogGlobalBarrier(
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter)
{
#define ENUM_STRING_AND_VALUE(EnumVal) \
	*ConvertToString(EnumVal), static_cast<uint32>(EnumVal)

	UE_LOG(LogD3D12RHI, Log,
TEXT(R"(
D3D12 Global Barrier
  |      SyncBefore: %s (0x%08x)
  |       SyncAfter: %s (0x%08x)
  |    AccessBefore: %s (0x%08x)
  |     AccessAfter: %s (0x%08x)
)"),
	ENUM_STRING_AND_VALUE(SyncBefore),
	ENUM_STRING_AND_VALUE(SyncAfter),
	ENUM_STRING_AND_VALUE(AccessBefore),
	ENUM_STRING_AND_VALUE(AccessAfter));

#undef ENUM_STRING_AND_VALUE
}

static const ID3D12Resource* GetID3D12ResourcePtr(
	const FD3D12Resource* ResourcePtr)
{
	return ResourcePtr->GetResource();
}

static const ID3D12Resource* GetID3D12ResourcePtr(
	const ID3D12Resource* ResourcePtr)
{
	return ResourcePtr;
}

static FString GetResourceName(
	const FD3D12Resource* ResourcePtr)
{
	return ResourcePtr->GetName().ToString();
}

static FString GetResourceName(
	const ID3D12Resource* ResourcePtr)
{
	return GetD312ObjectName(const_cast<ID3D12Resource*>(ResourcePtr));
}

static FString ConvertSubresourceToString(
	uint32 Subresource)
{
	return (Subresource == D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES) ?
		TEXT("All") :
		FString::FromInt(Subresource);
}

static bool ShouldLogBarrier(
	const FString& ResourceName)
{
	const FString& Filter = GD3D12LogBarriersResourceFilter;
	return Filter.IsEmpty() || ResourceName.Contains(Filter);
}

template <typename TResourcePtr>
static void LogTextureBarrier(
	const TResourcePtr pResource,
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter,
	D3D12_BARRIER_LAYOUT LayoutBefore,
	D3D12_BARRIER_LAYOUT LayoutAfter,
	uint32 Subresource,
	bool bDiscard)
{
	const FString ResourceName = GetResourceName(pResource);
	if (!ShouldLogBarrier(ResourceName))
	{
		return;
	}

#define ENUM_STRING_AND_VALUE(EnumVal) \
	*ConvertToString(EnumVal), static_cast<uint32>(EnumVal)

	UE_LOG(LogD3D12RHI, Log,
TEXT(R"(
D3D12 Texture Barrier
  |        Resource: %s (0x%p)
  |      SyncBefore: %s (0x%08x)
  |       SyncAfter: %s (0x%08x)
  |    AccessBefore: %s (0x%08x)
  |     AccessAfter: %s (0x%08x)
  |    LayoutBefore: %s (0x%08x)
  |     LayoutAfter: %s (0x%08x)
  |     Subresource: %s
  |         Discard: %u
)"),
	*ResourceName, GetID3D12ResourcePtr(pResource),
	ENUM_STRING_AND_VALUE(SyncBefore),
	ENUM_STRING_AND_VALUE(SyncAfter),
	ENUM_STRING_AND_VALUE(AccessBefore),
	ENUM_STRING_AND_VALUE(AccessAfter),
	ENUM_STRING_AND_VALUE(LayoutBefore),
	ENUM_STRING_AND_VALUE(LayoutAfter),
	*ConvertSubresourceToString(Subresource),
	static_cast<uint32>(bDiscard));

#undef ENUM_STRING_AND_VALUE
}

template <typename TResourcePtr>
static void LogBufferBarrier(
	const TResourcePtr pResource,
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter)
{
	const FString ResourceName = GetResourceName(pResource);
	if (!ShouldLogBarrier(ResourceName))
	{
		return;
	}

#define ENUM_STRING_AND_VALUE(EnumVal) \
	*ConvertToString(EnumVal), static_cast<uint32>(EnumVal)

	UE_LOG(LogD3D12RHI, Log,
TEXT(R"(
D3D12 Buffer Barrier
  |        Resource: %s (0x%p)
  |      SyncBefore: %s (0x%08x)
  |       SyncAfter: %s (0x%08x)
  |    AccessBefore: %s (0x%08x)
  |     AccessAfter: %s (0x%08x)
)"),
	*ResourceName, GetID3D12ResourcePtr(pResource),
	ENUM_STRING_AND_VALUE(SyncBefore),
	ENUM_STRING_AND_VALUE(SyncAfter),
	ENUM_STRING_AND_VALUE(AccessBefore),
	ENUM_STRING_AND_VALUE(AccessAfter));

#undef ENUM_STRING_AND_VALUE
}

template <typename TBarrierType>
static void LogBarriers(
	TArrayView<const TBarrierType> Barriers)
{
	using NonCVBarrierType = std::remove_cv_t<TBarrierType>;

	for (int32 BarrierIdx = 0; BarrierIdx < Barriers.Num(); ++BarrierIdx)
	{
		const TBarrierType& Barrier = Barriers[BarrierIdx];

		if constexpr (std::is_same_v<D3D12_BARRIER_GROUP, NonCVBarrierType>)
		{
			switch (Barrier.Type)
			{
			case D3D12_BARRIER_TYPE_GLOBAL:
				LogBarriers(MakeArrayView(Barrier.pGlobalBarriers, Barrier.NumBarriers));
				break;
			case D3D12_BARRIER_TYPE_BUFFER:
				LogBarriers(MakeArrayView(Barrier.pBufferBarriers, Barrier.NumBarriers));
				break;
			case D3D12_BARRIER_TYPE_TEXTURE:
				LogBarriers(MakeArrayView(Barrier.pTextureBarriers, Barrier.NumBarriers));
				break;
			default:
				checkNoEntry();
				break;
			}
		}
		else if constexpr (std::is_same_v<D3D12_GLOBAL_BARRIER, NonCVBarrierType>)
		{
			LogGlobalBarrier(
				Barrier.SyncBefore,
				Barrier.SyncAfter,
				Barrier.AccessBefore,
				Barrier.AccessAfter);
		}
		else if constexpr (std::is_same_v<D3D12_BUFFER_BARRIER, NonCVBarrierType>)
		{
			LogBufferBarrier(
				Barrier.pResource,
				Barrier.SyncBefore,
				Barrier.SyncAfter,
				Barrier.AccessBefore,
				Barrier.AccessAfter);
		}
		else if constexpr (std::is_same_v<D3D12_TEXTURE_BARRIER, NonCVBarrierType>)
		{
			// If this isn't zero then the Subresources struct represents a range
			// and we're not setup to log that. If you hit this, add the needed code.
			check(Barrier.Subresources.NumMipLevels == 0);

			LogTextureBarrier(
				Barrier.pResource,
				Barrier.SyncBefore,
				Barrier.SyncAfter,
				Barrier.AccessBefore,
				Barrier.AccessAfter,
				Barrier.LayoutBefore,
				Barrier.LayoutAfter,
				Barrier.Subresources.IndexOrFirstMipLevel,
				!!(Barrier.Flags & D3D12_TEXTURE_BARRIER_FLAG_DISCARD));
		}
		else
		{
			static_assert(std::is_same_v<D3D12_TEXTURE_BARRIER, NonCVBarrierType>);
		}
	}
}

#endif // D3D12RHI_BARRIER_LOGGING

// Use the top bit of the flags enum to mark transitions as "idle" time (used to remove the swapchain wait time for back buffers).
static const D3D12_BARRIER_TYPE BarrierType_CountAsIdleTime = D3D12_BARRIER_TYPE(1ull << ((sizeof(D3D12_BARRIER_TYPE) * 8) - 1));
static const D3D12_BARRIER_TYPE D3D12_BARRIER_TYPE_GLOBAL_COUNT_AS_IDLE = D3D12_BARRIER_TYPE(D3D12_BARRIER_TYPE_GLOBAL | BarrierType_CountAsIdleTime);
static const D3D12_BARRIER_TYPE D3D12_BARRIER_TYPE_TEXTURE_COUNT_AS_IDLE = D3D12_BARRIER_TYPE(D3D12_BARRIER_TYPE_TEXTURE | BarrierType_CountAsIdleTime);
static const D3D12_BARRIER_TYPE D3D12_BARRIER_TYPE_BUFFER_COUNT_AS_IDLE = D3D12_BARRIER_TYPE(D3D12_BARRIER_TYPE_BUFFER | BarrierType_CountAsIdleTime);

class FD3D12EnhancedBarriersBatcher
{
	struct FD3D12BarrierGroup : public D3D12_BARRIER_GROUP
	{
		FD3D12BarrierGroup() = default;
		FD3D12BarrierGroup(D3D12_BARRIER_GROUP&& BarrierGroup) : D3D12_BARRIER_GROUP(MoveTemp(BarrierGroup)) {}

		bool HasIdleFlag() const { return !!(Type & BarrierType_CountAsIdleTime); }
		void ClearIdleFlag() { Type = static_cast<D3D12_BARRIER_TYPE>(Type & ~BarrierType_CountAsIdleTime); }
	};
	static_assert(sizeof(FD3D12BarrierGroup) == sizeof(D3D12_BARRIER_GROUP), "FD3D12ResourceBarrier is a wrapper to add helper functions. Do not add members.");

public:
	template <typename T>
	void AddBarriers(
		FD3D12ContextCommon& Context,
		TArrayView<T>&& Barriers,
		bool IdleTime);

	void AddGlobalBarrier(
		FD3D12ContextCommon& Context,
		D3D12_BARRIER_SYNC SyncBefore,
		D3D12_BARRIER_SYNC SyncAfter,
		D3D12_BARRIER_ACCESS AccessBefore,
		D3D12_BARRIER_ACCESS AccessAfter);

	void AddTextureBarrier(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		D3D12_BARRIER_SYNC SyncBefore,
		D3D12_BARRIER_SYNC SyncAfter,
		D3D12_BARRIER_ACCESS AccessBefore,
		D3D12_BARRIER_ACCESS AccessAfter,
		D3D12_BARRIER_LAYOUT LayoutBefore,
		D3D12_BARRIER_LAYOUT LayoutAfter,
		uint32 Subresource,
		bool bDiscard);

	void AddBufferBarrier(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		D3D12_BARRIER_SYNC SyncBefore,
		D3D12_BARRIER_SYNC SyncAfter,
		D3D12_BARRIER_ACCESS AccessBefore,
		D3D12_BARRIER_ACCESS AccessAfter);

	void FlushIntoCommandList(
		class FD3D12CommandList& CommandList,
		class FD3D12QueryAllocator& TimestampAllocator);

	int32 Num() const
	{
		return BarrierGroups.Num();
	}

private:
	template <D3D12_BARRIER_TYPE TBarrierType>
	static auto GetMemberArrayForBarrierType();

	TArray<FD3D12BarrierGroup>    BarrierGroups;

	// @TODO - Shared Allocation/Allocator? 
	TArray<D3D12_TEXTURE_BARRIER> TextureBarriers;
	TArray<D3D12_BUFFER_BARRIER>  BufferBarriers;
	TArray<D3D12_GLOBAL_BARRIER>  GlobalBarriers;
};

template <typename T>
static consteval
D3D12_BARRIER_TYPE GetBarrierType()
{
	using TNonQualifiedType = std::remove_cv_t<T>;
	if constexpr (std::is_same_v<D3D12_GLOBAL_BARRIER, TNonQualifiedType>)
	{
		return D3D12_BARRIER_TYPE_GLOBAL;
	}
	else if constexpr (std::is_same_v<D3D12_TEXTURE_BARRIER, TNonQualifiedType>)
	{
		return D3D12_BARRIER_TYPE_TEXTURE;
	}
	else if constexpr (std::is_same_v<D3D12_BUFFER_BARRIER, TNonQualifiedType>)
	{
		return D3D12_BARRIER_TYPE_BUFFER;
	}
	else
	{
		static_assert(std::is_same_v<D3D12_BUFFER_BARRIER, T>, "Unknown barrier type");
	}
} //-V591

template <D3D12_BARRIER_TYPE TBarrierType>
static auto FD3D12EnhancedBarriersBatcher::GetMemberArrayForBarrierType()
{
	if constexpr (TBarrierType == D3D12_BARRIER_TYPE_GLOBAL)
	{
		return &FD3D12EnhancedBarriersBatcher::GlobalBarriers;
	}
	else if constexpr (TBarrierType == D3D12_BARRIER_TYPE_TEXTURE)
	{
		return &FD3D12EnhancedBarriersBatcher::TextureBarriers;
	}
	else if constexpr (TBarrierType == D3D12_BARRIER_TYPE_BUFFER)
	{
		return &FD3D12EnhancedBarriersBatcher::BufferBarriers;
	}
}

template <typename T>
void FD3D12EnhancedBarriersBatcher::AddBarriers(
	FD3D12ContextCommon& Context,
	TArrayView<T>&& Barriers,
	bool IdleTime)
{
	using TNonQualifiedType = std::remove_cv_t<T>;

	check(!Barriers.IsEmpty());

	checkSlow(CheckBarrierValuesAreCompatible(Barriers, Context.GetRHIPipeline()));

#if D3D12RHI_BARRIER_LOGGING
	if (GD3D12LogBarriers & D3D12LogBarriersWhenBatched)
	{
		LogBarriers(Barriers);
	}
#endif

	static constexpr D3D12_BARRIER_TYPE BaseBarrierType = GetBarrierType<TNonQualifiedType>();
	TArray<TNonQualifiedType>& BarrierArray = this->*GetMemberArrayForBarrierType<BaseBarrierType>();

	const D3D12_BARRIER_TYPE AdditionalTypeFlags = static_cast<D3D12_BARRIER_TYPE>(IdleTime ? BarrierType_CountAsIdleTime : 0);
	const D3D12_BARRIER_TYPE BarrierType = static_cast<D3D12_BARRIER_TYPE>(BaseBarrierType | AdditionalTypeFlags);

	const uint32 NumBarriers = Barriers.Num();
	const bool BarrierArrayIsEmpty = BarrierArray.IsEmpty();

	BarrierArray.Append(MoveTemp(Barriers));

	if (!BarrierArrayIsEmpty && (BarrierGroups.Last().Type == BarrierType))
	{
		BarrierGroups.Last().NumBarriers += NumBarriers;
	}
	else
	{
		// Smuggle an offset in the pointer field of each group. We'll fix it before submitting
		const T* FirstBarrierInGroupIdx = reinterpret_cast<T*>(static_cast<uintptr_t>(BarrierArray.Num()) - NumBarriers);
		if constexpr (std::is_same_v<D3D12_GLOBAL_BARRIER, TNonQualifiedType>)
		{
			BarrierGroups.Add(FD3D12BarrierGroup({
				.Type = BarrierType,
				.NumBarriers = NumBarriers,
				.pGlobalBarriers = FirstBarrierInGroupIdx,
			}));
		}
		else if constexpr (std::is_same_v<D3D12_TEXTURE_BARRIER, TNonQualifiedType>)
		{
			BarrierGroups.Add(FD3D12BarrierGroup({
				.Type = BarrierType,
				.NumBarriers = NumBarriers,
				.pTextureBarriers = FirstBarrierInGroupIdx,
			}));
		}
		else if constexpr (std::is_same_v<D3D12_BUFFER_BARRIER, TNonQualifiedType>)
		{
			BarrierGroups.Add(FD3D12BarrierGroup({
				.Type = BarrierType,
				.NumBarriers = NumBarriers,
				.pBufferBarriers = FirstBarrierInGroupIdx,
			}));
		}
		else
		{
			static_assert(std::is_same_v<D3D12_BUFFER_BARRIER, TNonQualifiedType>, "Unknown barrier type");
		}
	}

	if (!GD3D12BatchResourceBarriers)
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}
}

void FD3D12EnhancedBarriersBatcher::AddGlobalBarrier(
	FD3D12ContextCommon& Context,
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter)
{
	if (!BarrierCanBeDiscarded(
			SyncBefore,
			SyncAfter,
			AccessBefore,
			AccessAfter,
			D3D12_BARRIER_LAYOUT_UNDEFINED,
			D3D12_BARRIER_LAYOUT_UNDEFINED))
	{
		AddBarriers(
			Context, 
			MakeArrayView<D3D12_GLOBAL_BARRIER>({{
				.SyncBefore = SyncBefore,
				.SyncAfter = SyncAfter,
				.AccessBefore = AccessBefore,
				.AccessAfter = AccessAfter,
			}}),
			false);
	}
}

static bool IsBackBufferWriteTransition(
	const FD3D12Resource* InResource,
	D3D12_BARRIER_ACCESS AccessAfter,
	D3D12_BARRIER_LAYOUT LayoutBefore)
{
	static constexpr D3D12_BARRIER_ACCESS BackBufferBarrierWriteAccess =
		D3D12_BARRIER_ACCESS_RENDER_TARGET
		| D3D12_BARRIER_ACCESS_UNORDERED_ACCESS
		| D3D12_BARRIER_ACCESS_STREAM_OUTPUT
		| D3D12_BARRIER_ACCESS_COPY_DEST
		| D3D12_BARRIER_ACCESS_RESOLVE_DEST;

	static constexpr bool bCommonIsDistinctFromPresent =
		D3D12_BARRIER_LAYOUT_COMMON != D3D12_BARRIER_LAYOUT_PRESENT;

	const bool bIsBackBufferWriteTransition =
		InResource->IsBackBuffer()
		&& EnumHasAnyFlags(AccessAfter, BackBufferBarrierWriteAccess);

	if constexpr (bCommonIsDistinctFromPresent)
	{
		return bIsBackBufferWriteTransition 
			&& (LayoutBefore == D3D12_BARRIER_LAYOUT_PRESENT);
	}
	else
	{
		return bIsBackBufferWriteTransition;
	}
}

void FD3D12EnhancedBarriersBatcher::AddTextureBarrier(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* pResource,
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter,
	D3D12_BARRIER_LAYOUT LayoutBefore,
	D3D12_BARRIER_LAYOUT LayoutAfter,
	uint32 Subresource,
	bool bDiscard)
{
	// This is not the barrier you're looking for
	check(pResource->GetDesc().Dimension != D3D12_RESOURCE_DIMENSION_BUFFER);

	check(pResource->GetResource());

	// EB spec says discard flag can only be used when LayoutBefore is UNDEFINED
	check(!bDiscard || (LayoutBefore == D3D12_BARRIER_LAYOUT_UNDEFINED));

	if (!BarrierCanBeDiscarded(
		SyncBefore,
		SyncAfter,
		AccessBefore,
		AccessAfter,
		LayoutBefore,
		LayoutAfter))
	{
		const D3D12_TEXTURE_BARRIER_FLAGS Flags = 
			bDiscard ?
				D3D12_TEXTURE_BARRIER_FLAG_DISCARD :
				D3D12_TEXTURE_BARRIER_FLAG_NONE;

		const bool bIsBackBufferWriteTransition =
			IsBackBufferWriteTransition(
				pResource,
				AccessAfter,
				LayoutBefore);

		AddBarriers(
			Context,
			MakeArrayView<D3D12_TEXTURE_BARRIER>({{
				.SyncBefore   = SyncBefore,
				.SyncAfter    = SyncAfter,
				.AccessBefore = AccessBefore,
				.AccessAfter  = AccessAfter,
				.LayoutBefore = LayoutBefore,
				.LayoutAfter  = LayoutAfter,
				.pResource    = pResource->GetResource(),
				.Subresources = {
					.IndexOrFirstMipLevel = Subresource,
					.NumMipLevels = 0, // 0 indicates a subresource index
					.FirstArraySlice = {},
					.NumArraySlices = {},
					.FirstPlane = {},
					.NumPlanes = {},
				},
				.Flags = Flags,
			}}),
			bIsBackBufferWriteTransition);
	}
}

void FD3D12EnhancedBarriersBatcher::AddBufferBarrier(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* pResource,
	D3D12_BARRIER_SYNC SyncBefore,
	D3D12_BARRIER_SYNC SyncAfter,
	D3D12_BARRIER_ACCESS AccessBefore,
	D3D12_BARRIER_ACCESS AccessAfter)
{
	check(pResource->GetDesc().Dimension == D3D12_RESOURCE_DIMENSION_BUFFER);

	if (!BarrierCanBeDiscarded(
		SyncBefore,
		SyncAfter,
		AccessBefore,
		AccessAfter,
		D3D12_BARRIER_LAYOUT_UNDEFINED,
		D3D12_BARRIER_LAYOUT_UNDEFINED))
	{

		AddBarriers(
			Context,
			MakeArrayView<D3D12_BUFFER_BARRIER>({{
				.SyncBefore   = SyncBefore,
				.SyncAfter    = SyncAfter,
				.AccessBefore = AccessBefore,
				.AccessAfter  = AccessAfter,
				.pResource    = pResource->GetResource(),
				.Offset       = 0,
				.Size         = UINT64_MAX,
			}}),
			false);
	}
}

void FD3D12EnhancedBarriersBatcher::FlushIntoCommandList(
	FD3D12CommandList& CommandList,
	FD3D12QueryAllocator& TimestampAllocator)
{
	auto InsertTimestamp = [&](bool bBegin)
	{
		if (bBegin)
		{
			auto& Event = CommandList.EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
			CommandList.EndQuery(TimestampAllocator.Allocate(ED3D12QueryType::ProfilerTimestampBOP, &Event.GPUTimestampBOP));
		}
		else
		{
			// CPUTimestamp is filled in at submission time in FlushProfilerEvents
			auto& Event = CommandList.EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0);
			CommandList.EndQuery(TimestampAllocator.Allocate(ED3D12QueryType::ProfilerTimestampTOP, &Event.GPUTimestampTOP));
		}
	};

	for (int32 BatchStart = 0, BatchEnd = 0; BatchStart < BarrierGroups.Num(); BatchStart = BatchEnd)
	{
		// Gather a range of barriers that all have the same idle flag
		bool const bIdle = BarrierGroups[BatchEnd].HasIdleFlag();

		while (BatchEnd < BarrierGroups.Num() 
			&& bIdle == BarrierGroups[BatchEnd].HasIdleFlag())
		{
			// Clear the idle flag since it's not a valid D3D bit.
			BarrierGroups[BatchEnd++].ClearIdleFlag();
		}

		// Insert an idle begin/end timestamp around the barrier batch if required.
		if (bIdle)
		{
			InsertTimestamp(true);
		}

		//
		// Convert the offset in each barrier group to a pointer now that we can be sure the memory won't move around
		//

		for (int32 GroupIdx = BatchStart; GroupIdx < BatchEnd; ++GroupIdx)
		{
			D3D12_BARRIER_GROUP& BarrierGroup = BarrierGroups[GroupIdx];
			uintptr_t FirstBarrierIdx = {};
			switch (BarrierGroup.Type)
			{
				case D3D12_BARRIER_TYPE_GLOBAL:
					FirstBarrierIdx = reinterpret_cast<uintptr_t>(BarrierGroup.pGlobalBarriers);
					BarrierGroup.pGlobalBarriers = &GlobalBarriers[FirstBarrierIdx];
					break;
				case D3D12_BARRIER_TYPE_TEXTURE:
					FirstBarrierIdx = reinterpret_cast<uintptr_t>(BarrierGroup.pTextureBarriers);
					BarrierGroup.pTextureBarriers = &TextureBarriers[FirstBarrierIdx];
					break;
				case D3D12_BARRIER_TYPE_BUFFER:
					FirstBarrierIdx = reinterpret_cast<uintptr_t>(BarrierGroup.pBufferBarriers);
					BarrierGroup.pBufferBarriers = &BufferBarriers[FirstBarrierIdx];
					break;
				default:
					checkNoEntry();
					break;
			}
		}

		const int32 BatchNumGroups = BatchEnd - BatchStart;
#if D3D12RHI_BARRIER_LOGGING
		if (GD3D12LogBarriers & D3D12LogBarriersWhenFlushed)
		{
			LogBarriers(MakeArrayView<const D3D12_BARRIER_GROUP>(&BarrierGroups[BatchStart], BatchNumGroups));
		}
#endif
		CommandList.GraphicsCommandList8()->Barrier(BatchNumGroups, &BarrierGroups[BatchStart]);

		if (bIdle)
		{
			InsertTimestamp(false);
		}
	}

	BarrierGroups.Reset();
	GlobalBarriers.Reset();
	TextureBarriers.Reset();
	BufferBarriers.Reset();
}

D3D12_BARRIER_LAYOUT FD3D12EnhancedBarriersForAdapterImpl::GetInitialLayout(
	ED3D12Access InD3D12Access,
	const FD3D12ResourceDesc& InDesc)
{
	const bool bIsBuffer = InDesc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER;
	if (bIsBuffer)
	{
		// Buffers don't have a layout
		return D3D12_BARRIER_LAYOUT_UNDEFINED;
	}

	return ResourceIsSimultaneousAccess(InDesc) ?
		// Simultaneous access textures must be created in the COMMON layout
		D3D12_BARRIER_LAYOUT_COMMON :
		// This makes the assumption that all resources begin life on the gfx pipe
		GetEBLayout(InD3D12Access, ERHIPipeline::Graphics, nullptr);
}

void FD3D12EnhancedBarriersForAdapterImpl::ConfigureDevice(
	ID3D12Device* Device,
	bool InWithD3DDebug)
{
	FD3D12DynamicRHI::SetFormatAliasedTexturesMustBeCreatedUsingCommonLayout(false);
	GRHIGlobals.NeedsTransientDiscardStateTracking = false;
	GRHIGlobals.NeedsTransientDiscardOnGraphicsWorkaround = false;
}

uint64 FD3D12EnhancedBarriersForAdapterImpl::GetTransitionDataSizeBytes()
{
	return sizeof(FD3D12EnhancedBarriersTransitionData);
}

uint64 FD3D12EnhancedBarriersForAdapterImpl::GetTransitionDataAlignmentBytes()
{
	return alignof(FD3D12EnhancedBarriersTransitionData);
}

void FD3D12EnhancedBarriersForAdapterImpl::CreateTransition(
	FRHITransition* Transition,
	const FRHITransitionCreateInfo& CreateInfo)
{
	// Construct the data in-place on the transition instance
	FD3D12EnhancedBarriersTransitionData* Data = 
		new (Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>())
			FD3D12EnhancedBarriersTransitionData;

	Data->SrcPipelines = CreateInfo.SrcPipelines;
	Data->DstPipelines = CreateInfo.DstPipelines;
	Data->CreateFlags = CreateInfo.Flags;

	const bool bCreateFence =
		(CreateInfo.SrcPipelines != CreateInfo.DstPipelines)
		&& (!EnumHasAnyFlags(Data->CreateFlags, ERHITransitionCreateFlags::NoFence));

	if (bCreateFence)
	{
		// Create one sync point per device, per source pipe
		for (uint32 Index : FRHIGPUMask::All())
		{
			TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints.Emplace_GetRef();
			for (ERHIPipeline Pipeline : MakeFlagsRange(CreateInfo.SrcPipelines))
			{
				DeviceSyncPoints[Pipeline] = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("Transition"));
			}
		}
	}

	Data->TransitionInfos = CreateInfo.TransitionInfos;
}

void FD3D12EnhancedBarriersForAdapterImpl::ReleaseTransition(
	FRHITransition* Transition)
{
	// Destruct the transition data
	Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>()->~FD3D12EnhancedBarriersTransitionData();
}

HRESULT FD3D12EnhancedBarriersForAdapterImpl::CreateCommittedResource(
	FD3D12Adapter& Adapter,
	const D3D12_HEAP_PROPERTIES& InHeapProps,
	D3D12_HEAP_FLAGS InHeapFlags,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource)
{
	// @TODO - Ask Intel if they want to provide Layout based extensions like we use for
	//         for the legacy barriers

	// Convert the desc to the version required by CreateCommittedResource3
	const CD3DX12_RESOURCE_DESC1 LocalDesc1(InDesc);
	DXGI_FORMAT* CastableFormatsPtr = nullptr;
	UINT32 NumCastableFormats = 0;
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();
	CastableFormatsPtr =
		CastableFormats.IsEmpty() ? nullptr : const_cast<DXGI_FORMAT*>(CastableFormats.GetData());
	NumCastableFormats = CastableFormats.Num();
#endif

	ID3D12ProtectedResourceSession* ProtectedSession = nullptr;
	const D3D12_BARRIER_LAYOUT InitialLayout = GetInitialLayout(InInitialD3D12Access, InDesc);

	return Adapter.GetD3DDevice10()->CreateCommittedResource3(
		&InHeapProps,
		InHeapFlags,
		&LocalDesc1,
		InitialLayout,
		InClearValue,
		ProtectedSession,
		NumCastableFormats,
		CastableFormatsPtr,
		IID_PPV_ARGS(OutResource.GetInitReference()));
}

HRESULT FD3D12EnhancedBarriersForAdapterImpl::CreateReservedResource(
	FD3D12Adapter& Adapter,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource)
{
	DXGI_FORMAT* CastableFormatsPtr = nullptr;
	UINT32 NumCastableFormats = 0;
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();
	CastableFormatsPtr =
		CastableFormats.IsEmpty() ? nullptr : const_cast<DXGI_FORMAT*>(CastableFormats.GetData());
	NumCastableFormats = CastableFormats.Num();
#endif

	ID3D12ProtectedResourceSession* ProtectedSession = nullptr;
	const D3D12_BARRIER_LAYOUT InitialLayout = GetInitialLayout(InInitialD3D12Access, InDesc);

	return Adapter.GetD3DDevice10()->CreateReservedResource2(
		&InDesc,
		InitialLayout,
		InClearValue,
		ProtectedSession,
		NumCastableFormats,
		CastableFormatsPtr,
		IID_PPV_ARGS(OutResource.GetInitReference()));
}

HRESULT FD3D12EnhancedBarriersForAdapterImpl::CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 InHeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource)
{
		// @TODO - Ask Intel if they want to provide Layout based extensions like we use for
		//         for the legacy barriers

		// Convert the desc to the version required by CreatePlacedResource2
		CD3DX12_RESOURCE_DESC1 LocalDesc(InDesc);
		DXGI_FORMAT* CastableFormatsPtr = nullptr;
		UINT32 NumCastableFormats = 0;
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
		const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();
		CastableFormatsPtr =
			CastableFormats.IsEmpty() ? nullptr : const_cast<DXGI_FORMAT*>(CastableFormats.GetData());
		NumCastableFormats = CastableFormats.Num();
#endif

		const D3D12_BARRIER_LAYOUT InitialLayout = GetInitialLayout(InInitialD3D12Access, InDesc);

		return Adapter.GetD3DDevice10()->CreatePlacedResource2(
			Heap,
			InHeapOffset,
			&LocalDesc,
			InitialLayout,
			InClearValue,
			NumCastableFormats,
			CastableFormatsPtr,
			IID_PPV_ARGS(OutResource.GetInitReference()));
}

FD3D12EnhancedBarriersForAdapter::~FD3D12EnhancedBarriersForAdapter()
{}

void FD3D12EnhancedBarriersForAdapter::ConfigureDevice(
	ID3D12Device* Device,
	bool InWithD3DDebug) const
		
{
	return
		FD3D12EnhancedBarriersForAdapterImpl::ConfigureDevice(
			Device,
			InWithD3DDebug);
}

uint64 FD3D12EnhancedBarriersForAdapter::GetTransitionDataSizeBytes() const
{
	return FD3D12EnhancedBarriersForAdapterImpl::GetTransitionDataSizeBytes();
}

uint64 FD3D12EnhancedBarriersForAdapter::GetTransitionDataAlignmentBytes() const
{
	return FD3D12EnhancedBarriersForAdapterImpl::GetTransitionDataAlignmentBytes();
}

void FD3D12EnhancedBarriersForAdapter::CreateTransition(
	FRHITransition* Transition,
	const FRHITransitionCreateInfo& CreateInfo) const
{
	return 
		FD3D12EnhancedBarriersForAdapterImpl::CreateTransition(
			Transition,
			CreateInfo);
}

void FD3D12EnhancedBarriersForAdapter::ReleaseTransition(
	FRHITransition* Transition) const
{
	return
		FD3D12EnhancedBarriersForAdapterImpl::ReleaseTransition(
			Transition);
}

HRESULT FD3D12EnhancedBarriersForAdapter::CreateCommittedResource(
	FD3D12Adapter& Adapter,
	const D3D12_HEAP_PROPERTIES& InHeapProps,
	D3D12_HEAP_FLAGS InHeapFlags,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource) const
{
	return 
		FD3D12EnhancedBarriersForAdapterImpl::CreateCommittedResource(
			Adapter,
			InHeapProps,
			InHeapFlags,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

HRESULT FD3D12EnhancedBarriersForAdapter::CreateReservedResource(
	FD3D12Adapter& Adapter,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource) const
{	
	return 
		FD3D12EnhancedBarriersForAdapterImpl::CreateReservedResource(
			Adapter,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

HRESULT FD3D12EnhancedBarriersForAdapter::CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 InHeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const
{
	return 
		FD3D12EnhancedBarriersForAdapterImpl::CreatePlacedResource(
			Adapter,
			Heap,
			InHeapOffset,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

const TCHAR* FD3D12EnhancedBarriersForAdapter::GetImplementationName() const
{
	return TEXT("D3D12EnhancedBarriers");
}


static TTuple<const FD3D12Resource*, const FD3D12Buffer*, const FD3D12Texture*>
GetResourceBufferAndTexture(FD3D12CommandContext& Context, const FRHITransitionInfo& Info)
{
	switch (Info.Type)
	{
	case FRHITransitionInfo::EType::UAV:
		{
			const FD3D12UnorderedAccessView* UAV = Context.RetrieveObject<FD3D12UnorderedAccessView_RHI>(Info.UAV);
			check(UAV);
			return {
				UAV ? UAV->GetResource() : nullptr,
				Info.UAV->IsBuffer() ? Context.RetrieveObject<FD3D12Buffer>(Info.UAV->GetBuffer()) : nullptr,
				Info.UAV->IsTexture() ? Context.RetrieveTexture(Info.UAV->GetTexture()) : nullptr };
		}
	case FRHITransitionInfo::EType::Buffer:
		{
			// Resource may be null if this is a multi-GPU resource not present on the current GPU
			const FD3D12Buffer* Buffer = Context.RetrieveObject<FD3D12Buffer>(Info.Buffer);
			return { Buffer ? Buffer->GetResource() : nullptr, Buffer, nullptr };
		}
	case FRHITransitionInfo::EType::Texture:
		{
			// Resource may be null if this is a multi-GPU resource not present on the current GPU
			const FD3D12Texture* Texture = Context.RetrieveTexture(Info.Texture);
			return { Texture ? Texture->GetResource() : nullptr, nullptr, Texture };
		}
	case FRHITransitionInfo::EType::BVH:
		// Nothing special required for BVH transitions - handled inside d3d12 raytracing directly via UAV barriers and don't need explicit state changes
		return { nullptr, nullptr, nullptr };
	default:
		checkNoEntry();
		return { nullptr, nullptr, nullptr };
	}
}

class VARangeCollection
{
public:
	using VARange = TInterval<D3D12_GPU_VIRTUAL_ADDRESS>;

	void AddRange(
		const VARangeCollection::VARange& InVARange)
	{
		check(InVARange.IsValid());
		VARanges.Add(InVARange);
	}

	bool OverlapsAny(
		const VARangeCollection::VARange& InVARange) const
	{
		check(InVARange.IsValid());
		return VARanges.ContainsByPredicate(
			[this, &InVARange] (const VARange& InExistingVARange) { 
				return Overlaps(InVARange, InExistingVARange);
		});
	}

private:
	static bool Overlaps(
		const VARangeCollection::VARange& A,
		const VARangeCollection::VARange& B)
	{
		return (A.Min <= B.Max) && (B.Min <= A.Max);
	}

	TArray<VARange> VARanges;
};

static VARangeCollection BuildVARangeTable(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> InTransitions)
{
	VARangeCollection VARanges;
	for (const FRHITransition* Transition : InTransitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();
		for (const FRHITransitionInfo& Info : Data->TransitionInfos)
		{
			if ((Info.AccessBefore == ERHIAccess::Discard)
				&& (Info.AccessAfter != ERHIAccess::Discard))
			{
				const auto [Resource, Buffer, Texture] = GetResourceBufferAndTexture(Context, Info);
				if (Texture)
				{
					VARanges.AddRange({
						Texture->ResourceLocation.GetGPUVirtualAddress(),
						Texture->ResourceLocation.GetGPUVirtualAddress() + Texture->ResourceLocation.GetSize() });
				}
			}
		}
	}
	return VARanges;
}

FD3D12EnhancedBarriersForContext::FD3D12EnhancedBarriersForContext()
: ID3D12BarriersForContext()
, Batcher(MakeUnique<FD3D12EnhancedBarriersBatcher>())
{}

FD3D12EnhancedBarriersForContext::~FD3D12EnhancedBarriersForContext()
{}

//
// The RHI takes an abstracted view of transitions and so will ask for transitions across
// pipes that can't be expressed via a single barrier. In those cases an intermediate 
// state is necessary. The layout used for that intermediate state must be compatible
// with both the source and destination pipes as well as any sync and access bits that
// will compose the intermediate state. This function calculates which layout to use.
//
static FD3D12BarrierValues ChooseIntermediateBarrierValues(
	D3D12_BARRIER_SYNC InSyncBefore,
	D3D12_BARRIER_SYNC InSyncAfter,
	D3D12_BARRIER_ACCESS InAccessBefore,
	D3D12_BARRIER_ACCESS InAccessAfter,
	D3D12_BARRIER_LAYOUT InLayoutBefore,
	D3D12_BARRIER_LAYOUT InLayoutAfter,
	ERHIPipeline InPipeBefore,
	ERHIPipeline InPipeAfter)
{
	// @TODO - Should we use other information to decide on which side of the 
	//         transition to perform a layout change if we have multiple options?

	check(EnumHasOneFlag(InPipeBefore));
	check(!AccessIsCompatibleWithQueue(InAccessAfter, InPipeBefore)
		|| !LayoutIsCompatibleWithQueue(InLayoutAfter, InPipeBefore));

	if (LayoutIsCompatibleWithQueue(InLayoutBefore, InPipeAfter))
	{
		check(!AccessIsCompatibleWithQueue(InAccessBefore, InPipeAfter)
			|| !LayoutIsCompatibleWithQueue(InLayoutAfter, InPipeBefore));

		// In this case we only need to shed some access *or* we need to keep the
		// current layout during the intermediate barrier for the transfer to the 
		// new pipe and perform the layout change on the destination pipe
		return {
			.Sync = FilterToQueueCompatibleSync(InSyncBefore, InPipeAfter),
			.Access = FilterToQueueCompatibleAccess(InAccessBefore, InPipeAfter),
			.Layout = InLayoutBefore,
		};
	}

	// Layout must also be a problem so try and resolve that
	check(!LayoutIsCompatibleWithQueue(InLayoutAfter, InPipeBefore)
		|| !LayoutIsCompatibleWithQueue(InLayoutBefore, InPipeAfter));

	const D3D12_BARRIER_LAYOUT QueueAgnosticLayoutAfter =
		GetQueueAgnosticVersionOfLayout(InLayoutAfter);
	if (LayoutIsCompatibleWithQueue(QueueAgnosticLayoutAfter, InPipeBefore))
	{
		check(QueueAgnosticLayoutAfter != InLayoutAfter);
		return {
			.Sync = FilterToQueueCompatibleSync(InSyncAfter, ERHIPipeline::All),
			.Access = FilterToQueueCompatibleAccess(InAccessAfter, ERHIPipeline::All),
			.Layout = QueueAgnosticLayoutAfter,
		};
	}

	const D3D12_BARRIER_LAYOUT QueueAgnosticLayoutBefore =
		GetQueueAgnosticVersionOfLayout(InLayoutBefore);

	if (LayoutIsCompatibleWithQueue(QueueAgnosticLayoutBefore, InPipeAfter))
	{
		check (QueueAgnosticLayoutBefore != InLayoutBefore);

		// This should always be the case but to document the assumption...
		check(LayoutIsCompatibleWithQueue(QueueAgnosticLayoutBefore, InPipeBefore));

		// In this case we can change the layout to a queue independent one
		// and shed the sync+access incompatible with the new layout at the same time
		return {
			.Sync = FilterToQueueCompatibleSync(InSyncBefore, ERHIPipeline::All),
			.Access = FilterToQueueCompatibleAccess(InAccessBefore, ERHIPipeline::All),
			.Layout = QueueAgnosticLayoutBefore,
		};
	}

	// @TODO - Do we ever get here? This will work but it's not great...
	checkSlow(false);
	return {
		.Sync = D3D12_BARRIER_SYNC_ALL,
		.Access = D3D12_BARRIER_ACCESS_COMMON,
		.Layout = D3D12_BARRIER_LAYOUT_COMMON,
	};
}

static bool ProcessTransitionDuringBegin(const FD3D12EnhancedBarriersTransitionData* Data)
{
	// If we're entering a new pipe then we might need to do work in begin 
	return !EnumOnlyContainsFlags(Data->DstPipelines, Data->SrcPipelines);
}

static bool ProcessTransitionInfoDuringBegin(
	const UE::RHICore::FResourceState& InResourceState,
	const FD3D12Texture* InTexture)
{
	if (!EnumOnlyContainsFlags(InResourceState.DstPipelines, InResourceState.SrcPipelines))
	{
		if (InResourceState.SrcPipelines != ERHIPipeline::All)
		{
			// Are we going to a pipe that can't deal with one or more of our sync bits?
			// If that's the case we'll need to perform the incompatible sync before 
			// leaving the source pipe
			const D3D12_BARRIER_SYNC CurrentSync = GetEBSync(InResourceState.AccessBefore, InResourceState.SrcPipelines);
			if (!SyncIsCompatibleWithQueue(CurrentSync, InResourceState.DstPipelines))
			{
				return true;
			}

			// Are we going to a pipe that can't deal with one or more of our access bits?
			// If that's the case we'll need to perform the incompatible access changes before
			// leaving the source pipe
			const D3D12_BARRIER_ACCESS CurrentAccess = GetEBAccess(InResourceState.AccessBefore, InResourceState.SrcPipelines);
			if (!AccessIsCompatibleWithQueue(CurrentAccess, InResourceState.DstPipelines))
			{
				return true;
			}

			// Check if we have to make any layout changes for the cross-pipe barrier
			// Only textures have layouts
			if (InTexture)
			{
				const D3D12_BARRIER_LAYOUT CurrentLayout =
					GetEBLayout(
						InResourceState.AccessBefore,
						InResourceState.SrcPipelines,
						InTexture);

				return !LayoutIsCompatibleWithQueue(CurrentLayout, InResourceState.DstPipelines);
			}
		}
		else
		{
			// If we're already on all pipes then all bits should be compatible with the destination
			// If not then something has changed or it's a bug somewhere else
			check(SyncIsCompatibleWithQueue(GetEBSync(InResourceState.AccessBefore, InResourceState.DstPipelines), InResourceState.DstPipelines));
			check(AccessIsCompatibleWithQueue(GetEBAccess(InResourceState.AccessBefore, InResourceState.DstPipelines), InResourceState.DstPipelines));
			if (InTexture)
			{
				check(
					LayoutIsCompatibleWithQueue(
						GetEBLayout(
							InResourceState.AccessBefore,
							InResourceState.SrcPipelines,
							InTexture),
						InResourceState.DstPipelines));
			}
		}
	}

	return false;
}

void FD3D12EnhancedBarriersForContext::BeginTransitions(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> Transitions)
{
	// Build barriers
	AddBarriersForTransitions(Context, Transitions, EBarrierPhase::Begin);

	// Signal fences
	const ERHIPipeline CurrentPipeline = Context.GetPipeline();
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();
		if (!Data->SyncPoints.IsEmpty())
		{
			const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];
			if (DeviceSyncPoints[CurrentPipeline])
			{
				Context.SignalSyncPoint(DeviceSyncPoints[CurrentPipeline]);
			}
		}
	}
}

void FD3D12EnhancedBarriersForContext::EndTransitions(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> Transitions)
{
	const ERHIPipeline CurrentPipeline = Context.GetPipeline();

	// Wait for fences
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();
		if (!Data->SyncPoints.IsEmpty())
		{
			const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];
			for (ERHIPipeline SrcPipeline : MakeFlagsRange(Data->SrcPipelines))
			{
				if (SrcPipeline != CurrentPipeline && DeviceSyncPoints[SrcPipeline])
				{
					Context.WaitSyncPoint(DeviceSyncPoints[SrcPipeline]);
				}
			}
		}
	}

	// Update reserved resource memory mapping
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();

		HandleReservedResourceCommits(Context, Data);
	}

	// Build barriers
	AddBarriersForTransitions(Context, Transitions, EBarrierPhase::End);
}

void FD3D12EnhancedBarriersForContext::AddGlobalBarrier(
	FD3D12ContextCommon& Context,
	ED3D12Access D3D12AccessBefore,
	ED3D12Access D3D12AccessAfter)
{
	const ERHIPipeline Pipe = Context.GetRHIPipeline();

	Batcher->AddGlobalBarrier(
		Context,
		GetEBSync(D3D12AccessBefore, Pipe),
		GetEBSync(D3D12AccessAfter, Pipe),
		GetEBAccess(D3D12AccessBefore, Pipe),
		GetEBAccess(D3D12AccessAfter, Pipe));
}

void FD3D12EnhancedBarriersForContext::AddBarrier(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* pResource,
	ED3D12Access D3D12AccessBefore,
	ED3D12Access D3D12AccessAfter,
	uint32 Subresource)
{
	check(pResource);

	const ERHIPipeline Pipe = Context.GetRHIPipeline();

	const D3D12_BARRIER_SYNC SyncBefore = GetEBSync(D3D12AccessBefore, Pipe);
	const D3D12_BARRIER_SYNC SyncAfter = GetEBSync(D3D12AccessAfter, Pipe);
	const D3D12_BARRIER_ACCESS AccessBefore = GetEBAccess(D3D12AccessBefore, Pipe);
	const D3D12_BARRIER_ACCESS AccessAfter = GetEBAccess(D3D12AccessAfter, Pipe);

	if (pResource->GetDesc().Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
	{
		Batcher->AddBufferBarrier(
			Context,
			pResource,
			SyncBefore,
			SyncAfter,
			AccessBefore,
			AccessAfter);
	}
	else
	{
		D3D12_BARRIER_LAYOUT LayoutBefore = {};
		D3D12_BARRIER_LAYOUT LayoutAfter = {};

		if (!pResource->RequiresResourceStateTracking()
			&& (pResource->GetInitialAccess() == ED3D12Access::Common))
		{
			// Don't ever move an untracked resource out of a common layout if that's its initial access
			// This fixes problems with upload textures that need to be left in the common layout for the copy queue
			// See FD3D12DynamicRHI::RHIAsyncCreateTexture2D()
			LayoutBefore = D3D12_BARRIER_LAYOUT_COMMON;
			LayoutAfter = D3D12_BARRIER_LAYOUT_COMMON;
		}
		else
		{
			LayoutBefore = GetEBLayout(D3D12AccessBefore, Pipe, nullptr);
			LayoutAfter = GetEBLayout(D3D12AccessAfter, Pipe, nullptr);
		}

		Batcher->AddTextureBarrier(
			Context,
			pResource,
			SyncBefore,
			SyncAfter,
			AccessBefore,
			AccessAfter,
			LayoutBefore,
			LayoutAfter,
			Subresource,
			false);
	}
}

void FD3D12EnhancedBarriersForContext::FlushIntoCommandList(
	FD3D12CommandList& CommandList,
	FD3D12QueryAllocator& TimestampAllocator)
{
	Batcher->FlushIntoCommandList(CommandList, TimestampAllocator);
}

int32 FD3D12EnhancedBarriersForContext::GetNumPendingBarriers() const
{
	return Batcher->Num();
}

static bool NeedToProcessTransitionEarlyToAvoidVAConflicts(
	ERHIAccess InRHIAccessBefore,
	ERHIAccess InRHIAccessAfter,
	const FD3D12Resource* InResource,
	const FD3D12Buffer* InBuffer,
	const FD3D12Texture* InTexture,
	const VARangeCollection& InVARangesToBeInitialized)
{
	// It's against the enhanced barriers rules to discard a resource 
	// (in the sense that it's put into the NO_ACCESS state) and then
	// initialize an aliasing resource (using the DISCARD flag) in the
	// same barrier batch. So here we check to see if we're both discarding
	// and initializing overlapping VA ranges and if we are, we need to
	// submit the discard to the driver separate from the initialization.

	if ((InRHIAccessBefore != ERHIAccess::Discard)
		&& (InRHIAccessAfter == ERHIAccess::Discard))
	{
		check(InBuffer || InTexture);

		const D3D12_GPU_VIRTUAL_ADDRESS ResourceVARangeStart =
			InTexture ?
				InTexture->ResourceLocation.GetGPUVirtualAddress() :
				InBuffer->ResourceLocation.GetGPUVirtualAddress();
		
		const D3D12_GPU_VIRTUAL_ADDRESS ResourceVARangeEnd =
			ResourceVARangeStart
			+ (InTexture ?
					InTexture->ResourceLocation.GetSize() :
					InBuffer->ResourceLocation.GetSize());

		const TInterval<D3D12_GPU_VIRTUAL_ADDRESS> ResourceVARange =
			{ ResourceVARangeStart, ResourceVARangeEnd };

		return InVARangesToBeInitialized.OverlapsAny(ResourceVARange);
	}
	return false;
}

void FD3D12EnhancedBarriersForContext::AddBarriersForTransitions(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> InTransitions,
	EBarrierPhase InBarrierPhase)
{
	// @TODO - Experiment with this ordering. Is it better to do the "discard" or "acquire" portion
	//         of the operation on its own? This will totally depend on the engine behavior.

	const bool bIsBegin = (InBarrierPhase == EBarrierPhase::Begin);

	// Build a list of VA ranges that may be initialized by the barriers
	const VARangeCollection VARanges = BuildVARangeTable(Context, InTransitions);

	// Handle early discards first
	bool bHadEarlyDiscards = false;
	for (const FRHITransition* Transition : InTransitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();
		if (!bIsBegin || ProcessTransitionDuringBegin(Data))
		{
			bHadEarlyDiscards |=
				AddBarriersForTransitionData(
					Context,
					Data,
					VARanges,
					EProcessEarlyTransitions::Yes,
					InBarrierPhase);
		}
	}

	if (bHadEarlyDiscards)
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}

	// Now everything else
	for (const FRHITransition* Transition : InTransitions)
	{
		const FD3D12EnhancedBarriersTransitionData* Data = Transition->GetPrivateData<FD3D12EnhancedBarriersTransitionData>();
		if (!bIsBegin || ProcessTransitionDuringBegin(Data))
		{
			AddBarriersForTransitionData(Context, Data, VARanges, EProcessEarlyTransitions::No, InBarrierPhase);
		}
	}
}

bool FD3D12EnhancedBarriersForContext::AddBarriersForTransitionData(
	FD3D12CommandContext& Context,
	const FD3D12EnhancedBarriersTransitionData* InTransitionData,
	const VARangeCollection& InVARangesToBeInitialized,
	EProcessEarlyTransitions InProcessEarlyTransitions,
	EBarrierPhase InBarrierPhase)
{
	bool bAddedBarriers = false;
	const bool bIsBegin = (InBarrierPhase == EBarrierPhase::Begin);
	const bool bProcessEarlyTransitions = (InProcessEarlyTransitions == EProcessEarlyTransitions::Yes);

	for (const FRHITransitionInfo& Info : InTransitionData->TransitionInfos)
	{
		if (Info.Resource)
		{
			const auto [Resource, Buffer, Texture] = GetResourceBufferAndTexture(Context, Info);

			// @TODO - Why do we have to filter these out? Should we ever see this???
			//check(Resource);
			//check(Resource->RequiresResourceStateTracking());
			if (Resource == nullptr || !Resource->RequiresResourceStateTracking())
			{
				check(Resource || Info.Type == FRHITransitionInfo::EType::BVH);
				continue;
			}

			const UE::RHICore::FResourceState ResourceState(
				Context,
				InTransitionData->SrcPipelines,
				InTransitionData->DstPipelines,
				Info);

			const bool bProcessDuringBegin =
				ProcessTransitionInfoDuringBegin(
					ResourceState,
					Texture);

			if (bIsBegin && !bProcessDuringBegin)
			{
				continue;
			}

			if (NeedToProcessTransitionEarlyToAvoidVAConflicts(
					ResourceState.AccessBefore,
					ResourceState.AccessAfter,
					Resource,
					Buffer,
					Texture,
					InVARangesToBeInitialized)
				!= bProcessEarlyTransitions)
			{
				continue;
			}

			/*
			const bool bLeavingThisPipe =
				!EnumHasAnyFlags(InTransitionData->DstPipelines, GetPipeline());

			const bool bEnteringThisPipe =
				!EnumHasAnyFlags(InTransitionData->SrcPipelines, GetPipeline());
			*/

			// @TODO - Ask MS to allow SYNC_NONE for cases like this?
			//         We know that the sync is already handled by fences
			D3D12_BARRIER_SYNC SyncBefore =
				/*
				bEnteringThisPipe ?
					D3D12_BARRIER_SYNC_NONE :
				*/
					GetEBSync(ResourceState.AccessBefore, ResourceState.SrcPipelines);
			D3D12_BARRIER_SYNC SyncAfter =
				/*
				bLeavingThisPipe ?
					D3D12_BARRIER_SYNC_NONE :
				*/
					GetEBSync(ResourceState.AccessAfter, ResourceState.DstPipelines);

			D3D12_BARRIER_ACCESS AccessBefore =
				GetEBAccess(ResourceState.AccessBefore, ResourceState.SrcPipelines);
			D3D12_BARRIER_ACCESS AccessAfter =
				GetEBAccess(ResourceState.AccessAfter, ResourceState.DstPipelines);

			D3D12_BARRIER_LAYOUT LayoutBefore =
				Texture ?
					GetEBLayout(
						ResourceState.AccessBefore,
						ResourceState.SrcPipelines,
						Texture) :
					D3D12_BARRIER_LAYOUT_UNDEFINED;

			D3D12_BARRIER_LAYOUT LayoutAfter =
				Texture ?
					GetEBLayout(
						ResourceState.AccessAfter,
						ResourceState.DstPipelines,
						Texture) :
					D3D12_BARRIER_LAYOUT_UNDEFINED;

			// Need an intermediate barrier if we can't complete
			// the entire barrier on the source pipe during begin
			// Note that sync is taken care of by the following fence
			// to move the resource to the new pipe(s) so we don't have
			// to test for sync compat on the after side
			const bool bCreateIntermediateBarrier =
				bProcessDuringBegin
				&& (!AccessIsCompatibleWithQueue(
						AccessAfter,
						ResourceState.SrcPipelines)
					|| !LayoutIsCompatibleWithQueue(
							LayoutAfter,
							ResourceState.SrcPipelines));

			if (!bIsBegin && bProcessDuringBegin && !bCreateIntermediateBarrier)
			{
				// In this case the barrier was completely handled in begin
				continue;
			}

			if (bCreateIntermediateBarrier)
			{
				// If we're here, then we need to do some kind of intermediate
				// transition before the resource gets fenced over to the other pipe

				const FD3D12BarrierValues IntermediateBarrierValues =
					ChooseIntermediateBarrierValues(
						SyncBefore,
						SyncAfter,
						AccessBefore,
						AccessAfter,
						LayoutBefore,
						LayoutAfter,
						ResourceState.SrcPipelines,
						ResourceState.DstPipelines);
						
				if (bIsBegin)
				{
					SyncAfter = IntermediateBarrierValues.Sync;
					AccessAfter = IntermediateBarrierValues.Access;
					LayoutAfter = IntermediateBarrierValues.Layout;
				}
				else
				{
					SyncBefore = IntermediateBarrierValues.Sync;
					AccessBefore = IntermediateBarrierValues.Access;
					LayoutBefore = IntermediateBarrierValues.Layout;
				}
			}

			const ERHIPipeline ThisPipe = Context.GetPipeline();
			SyncBefore = FilterToQueueCompatibleSync(SyncBefore, ThisPipe);
			SyncAfter = FilterToQueueCompatibleSync(SyncAfter, ThisPipe);
			AccessBefore = FilterToQueueCompatibleAccess(AccessBefore, ThisPipe);
			AccessAfter = FilterToQueueCompatibleAccess(AccessAfter, ThisPipe);

			// Can't use SYNC_NONE when there's access work to be done so have to choose something
			if (ResourceState.AccessBefore != ERHIAccess::Present
				&& SyncBefore == D3D12_BARRIER_SYNC_NONE
				&& AccessBefore != D3D12_BARRIER_ACCESS_NO_ACCESS)
			{
				SyncBefore = FilterToQueueCompatibleSync(GetAccessCompatibleSync(AccessBefore), ThisPipe);
			}
#if PLATFORM_REQUIRES_ENHANCED_BARRIERS_GFX_ONLY_READ_BITS_HACK
			// @TODO - This is necessary because of the hack that makes D3D12_BARRIER_LAYOUT_DIRECT_QUEUE_GENERIC_READ
			//         compatible with the compute pipe in order to support gfx specific read access when a resource is
			//         available on multiple pipes at once. It should be removed when MS fixes the EB API.
			//         Although we've made the layout and access bits compatible, there's no change to the sync bits so
			//         we can end up in situations where the sync filters down to none when moving a resource to/from gfx
			//         to compute.
			if (ResourceState.AccessAfter != ERHIAccess::Present
				&& SyncAfter == D3D12_BARRIER_SYNC_NONE
				&& AccessAfter != D3D12_BARRIER_ACCESS_NO_ACCESS)
			{
				SyncAfter = FilterToQueueCompatibleSync(GetAccessCompatibleSync(AccessAfter), ThisPipe);
			}
#endif
			check(!(ResourceState.AccessAfter != ERHIAccess::Present
					&& SyncAfter == D3D12_BARRIER_SYNC_NONE
					&& AccessAfter != D3D12_BARRIER_ACCESS_NO_ACCESS));

			if (Texture)
			{	
				const bool bIsReallyWholeResource =
					Info.IsWholeResource() ||
					(Resource->GetSubresourceCount() == 1);

				if (!bIsReallyWholeResource)
				{
					// high level rendering is controlling transition ranges, at this level this is an index not a range
					check(Info.MipIndex != FRHISubresourceRange::kAllSubresources);
					check(Info.ArraySlice != FRHISubresourceRange::kAllSubresources);
					check(Info.PlaneSlice != FRHISubresourceRange::kAllSubresources);
				}

				const uint32 SubresourceIdx =
					bIsReallyWholeResource ?
						D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES :
						D3D12CalcSubresource(
							Info.MipIndex,
							Info.ArraySlice,
							Info.PlaneSlice,
							Resource->GetMipLevels(),
							Resource->GetArraySize());

				const bool bDiscard =
					LayoutBefore == D3D12_BARRIER_LAYOUT_UNDEFINED
					&& LayoutAfter != D3D12_BARRIER_LAYOUT_UNDEFINED
					&& !(GD3D12DisableDiscardOfDepthResources && Resource->IsDepthStencilResource())
					&& GD3D12AllowDiscardResources;

				Batcher->AddTextureBarrier(
					Context,
					Resource,
					SyncBefore,
					SyncAfter,
					AccessBefore,
					AccessAfter,
					LayoutBefore,
					LayoutAfter,
					SubresourceIdx,
					bDiscard);
			}
			else
			{
				Batcher->AddBufferBarrier(
					Context,
					Resource,
					SyncBefore,
					SyncAfter,
					AccessBefore,
					AccessAfter);
			}

			bAddedBarriers = true;
		}
	}

	return bAddedBarriers;
}

void FD3D12EnhancedBarriersForContext::HandleReservedResourceCommits(
	FD3D12CommandContext& Context,
	const FD3D12EnhancedBarriersTransitionData* TransitionData)
{
	for (const FRHITransitionInfo& Info : TransitionData->TransitionInfos)
	{
		if (const FRHICommitResourceInfo* CommitInfo = Info.CommitInfo.GetPtrOrNull())
		{
			if (Info.Type == FRHITransitionInfo::EType::Buffer)
			{
				FD3D12Buffer* Buffer = Context.RetrieveObject<FD3D12Buffer>(Info.Buffer);
				Context.SetReservedBufferCommitSize(Buffer, CommitInfo->SizeInBytes);
			}
			else
			{
				checkNoEntry();
			}
		}
	}
}

#endif // D3D12RHI_SUPPORTS_ENHANCED_BARRIERS