// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIDefinitions.h"

#if D3D12RHI_SUPPORTS_ENHANCED_BARRIERS

#include "ID3D12Barriers.h"

class FD3D12EnhancedBarriersBatcher;
struct FD3D12EnhancedBarriersTransitionData;

struct FRHITransitionInfo;
enum class ERHIPipeline : uint8;

class FD3D12Texture;

//
// Separate the implementation details from the fulfillment of
// the ID3D12BarriersForAdapter interface so that other platforms
// which need to further specialize both the interface and the 
// implementation don't run into the diamond inheritance problem
// with the interface or are forced to have multiple v-tables
//

namespace FD3D12EnhancedBarriersForAdapterImpl
{
	D3D12_BARRIER_LAYOUT GetInitialLayout(
		ED3D12Access InD3D12Access,
		const FD3D12ResourceDesc& InDesc);

	void ConfigureDevice(
		ID3D12Device* Device,
		bool InWithD3DDebug);

	uint64 GetTransitionDataSizeBytes();
	uint64 GetTransitionDataAlignmentBytes();

	void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo);

	void ReleaseTransition(
		FRHITransition* Transition);

	HRESULT CreateCommittedResource(
		FD3D12Adapter& Adapter,
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);
	
	HRESULT CreateReservedResource(
		FD3D12Adapter& Adapter,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);

	HRESULT CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 HeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);

} // namespace FD3D12EnhancedBarriersForAdapterImpl

class FD3D12EnhancedBarriersForAdapter final
	: public ID3D12BarriersForAdapter
{
public:
	virtual ~FD3D12EnhancedBarriersForAdapter();

	virtual void ConfigureDevice(
		ID3D12Device* Device,
		bool InWithD3DDebug) const override final;

	virtual uint64 GetTransitionDataSizeBytes() const override final;
	virtual uint64 GetTransitionDataAlignmentBytes() const override final;

	virtual void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo) const override final;

	virtual void ReleaseTransition(
		FRHITransition* Transition) const override final;

	virtual HRESULT CreateCommittedResource(
		FD3D12Adapter& Adapter,
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual HRESULT CreateReservedResource(
		FD3D12Adapter& Adapter,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual HRESULT CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 HeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual const TCHAR* GetImplementationName() const override final;
};

class FD3D12EnhancedBarriersForContext final
	: public ID3D12BarriersForContext
{
public:
	FD3D12EnhancedBarriersForContext();
	virtual ~FD3D12EnhancedBarriersForContext();

	virtual void BeginTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) override final;

	virtual void EndTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) override final;

	virtual void AddGlobalBarrier(
		FD3D12ContextCommon& Context,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter) override final;

	virtual void AddBarrier(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter,
		uint32 Subresource) override final;

	virtual void FlushIntoCommandList(
		class FD3D12CommandList& CommandList,
		class FD3D12QueryAllocator& TimestampAllocator) override final;

	virtual int32 GetNumPendingBarriers() const override final;

private:
	enum class EProcessEarlyTransitions : bool
	{
		No = false,
		Yes = true,
	};

	enum class EBarrierPhase : bool
	{
		Begin,
		End,
	};

	bool AddBarriersForTransitionData(
		FD3D12CommandContext& Context,
		const FD3D12EnhancedBarriersTransitionData* InTransitionData,
		const class VARangeCollection& InVARangesToBeInitialized,
		EProcessEarlyTransitions InProcessEarlyTransitions,
		EBarrierPhase InBarrierPhase);

	void AddBarriersForTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> InTransitions,
		EBarrierPhase InBarrierPhase);

	static void HandleReservedResourceCommits(
		FD3D12CommandContext& Context,
		const FD3D12EnhancedBarriersTransitionData* TransitionData);

	TUniquePtr<FD3D12EnhancedBarriersBatcher> Batcher;
};

#endif // D3D12RHI_SUPPORTS_ENHANCED_BARRIERS