// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12ExplicitDescriptorCache.h"
#include "D3D12Adapter.h"
#include "D3D12Device.h"
#include "D3D12DirectCommandListManager.h"
#include "D3D12RHIPrivate.h"
#include "D3D12Stats.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/LowLevelMemTracker.h"
#include "Hash/xxhash.h"

// Whether to compare the full descriptor table on cache lookup or only use FXxHash64 digest.
#ifndef EXPLICIT_DESCRIPTOR_CACHE_FULL_COMPARE
#define EXPLICIT_DESCRIPTOR_CACHE_FULL_COMPARE 0
#endif

static int32 GD3D12ExplicitDeduplicateSamplers = 1;
static FAutoConsoleVariableRef CVarD3D12ExplicitDeduplicateSamplers(
	TEXT("r.D3D12.ExplicitDescriptorHeap.DeduplicateSamplers"),
	GD3D12ExplicitDeduplicateSamplers,
	TEXT("Use an exhaustive search to deduplicate sampler descriptors when generating shader binding tables. Reduces sampler heap usage at the cost of some CPU time. (default = 1)")
);

int32 GD3D12ExplicitViewDescriptorHeapSize = 250'000;
int32 GD3D12ExplicitViewDescriptorHeapOverflowReported = 0;
static FAutoConsoleVariableRef CVarD3D12ExplicitViewDescriptorHeapSize(
	TEXT("r.D3D12.ExplicitDescriptorHeap.ViewDescriptorHeapSize"),
	GD3D12ExplicitViewDescriptorHeapSize,
	TEXT("Maximum number of descriptors per explicit view descriptor heap. (default = 250k, ~8MB per heap)\n")
	TEXT("Typical measured descriptor heap usage in large scenes is ~50k. An error is reported when this limit is reached and shader bindings for subsequent objects are skipped.\n"),
	ECVF_ReadOnly
);

FD3D12ExplicitDescriptorHeap::FD3D12ExplicitDescriptorHeap(FD3D12Device* Device)
	: FD3D12DeviceChild(Device)
	, D3DDevice(Device->GetDevice())
{
}

FD3D12ExplicitDescriptorHeapCache::~FD3D12ExplicitDescriptorHeapCache()
{
	check(NumAllocatedEntries == 0);

	FScopeLock Lock(&CriticalSection);
	for (const FEntry& It : FreeList)
	{
		if (It.Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV)
		{
			DEC_DWORD_STAT(STAT_ExplicitViewDescriptorHeaps);
			DEC_DWORD_STAT_BY(STAT_ExplicitViewDescriptors, It.NumDescriptors);
		}
		else
		{
			DEC_DWORD_STAT(STAT_ExplicitSamplerDescriptorHeaps);
			DEC_DWORD_STAT_BY(STAT_ExplicitSamplerDescriptors, It.NumDescriptors);
		}

		It.Heap->Release();
	}
	FreeList.Empty();
}

void FD3D12ExplicitDescriptorHeapCache::DeferredReleaseHeap(FD3D12ExplicitDescriptorHeapCache::FEntry&& Entry)
{
	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(
		[Cache = this, Entry = MoveTemp(Entry)]() mutable
		{
			Cache->ReleaseHeap(MoveTemp(Entry));
		});
}

void FD3D12ExplicitDescriptorHeapCache::ReleaseHeap(FD3D12ExplicitDescriptorHeapCache::FEntry&& Entry)
{
	FScopeLock Lock(&CriticalSection);

	check(NumAllocatedEntries != 0);

	Entry.LastUsedFrame = GetParentDevice()->GetParentAdapter()->GetFrameFence().GetNextFenceToSignal();
	Entry.LastUsedTime = FPlatformTime::Seconds();

	FreeList.Add(MoveTemp(Entry));

	--NumAllocatedEntries;
}

FD3D12ExplicitDescriptorHeapCache::FEntry FD3D12ExplicitDescriptorHeapCache::AllocateHeap(D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 NumDescriptors)
{
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ExplicitDescriptorHeap"));

	FScopeLock Lock(&CriticalSection);

	// Align request to enable greater reusue in the cache.
	const uint32 MaxDescriptors = (Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER) ? D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE : D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_1;
	NumDescriptors = FMath::Clamp(FMath::RoundUpToPowerOfTwo(NumDescriptors), 0, MaxDescriptors);

	++NumAllocatedEntries;

	FEntry Result = {};

	for (int32 EntryIndex = 0; EntryIndex < FreeList.Num(); ++EntryIndex)
	{
		const FEntry& It = FreeList[EntryIndex];
		if (It.Type == Type && It.NumDescriptors >= NumDescriptors)
		{
			Result = It;
			FreeList.RemoveAtSwap(EntryIndex, EAllowShrinking::No);

			return Result;
		}
	}

	// Compatible heap was not found in cache, so create a new one.

	// Release heaps that were not used for a while before allocating new.
	ReleaseStaleEntries(100 /*MaxAgeInFrames*/, 5.0f /*MaxAgeInSeconds*/);

	D3D12_DESCRIPTOR_HEAP_DESC Desc = {};

	Desc.Flags = D3D12_DESCRIPTOR_HEAP_FLAG_SHADER_VISIBLE;
	Desc.Type = Type;
	Desc.NumDescriptors = NumDescriptors;
	Desc.NodeMask = GetParentDevice()->GetGPUMask().GetNative();

	ID3D12DescriptorHeap* D3D12Heap = nullptr;

	const TCHAR* HeapName = Desc.Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV ? TEXT("Explicit View Heap") : TEXT("Explicit Sampler Heap");
	UE_LOGF(LogD3D12RHI, Log, "Creating %ls with %d entries", HeapName, NumDescriptors);

	VERIFYD3D12RESULT(GetParentDevice()->GetDevice()->CreateDescriptorHeap(&Desc, IID_PPV_ARGS(&D3D12Heap)));
	SetD3D12ObjectName(D3D12Heap, Desc.Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV ? L"Explicit View Heap" : L"Explicit Sampler Heap");

	Result.NumDescriptors = NumDescriptors;
	Result.Type = Type;
	Result.Heap = D3D12Heap;

	if (Desc.Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV)
	{
		INC_DWORD_STAT(STAT_ExplicitViewDescriptorHeaps);
		INC_DWORD_STAT_BY(STAT_ExplicitViewDescriptors, NumDescriptors);
	}
	else
	{
		INC_DWORD_STAT(STAT_ExplicitSamplerDescriptorHeaps);
		INC_DWORD_STAT_BY(STAT_ExplicitSamplerDescriptors, NumDescriptors);
	}

	return Result;
}

void FD3D12ExplicitDescriptorHeapCache::ReleaseStaleEntries(uint32 MaxAgeInFrames, float MaxAgeInSeconds)
{
	const uint64 CurrentFrame = GetParentDevice()->GetParentAdapter()->GetFrameFence().GetNextFenceToSignal();
	const double CurrentTime = FPlatformTime::Seconds();

	int32 EntryIndex = 0;
	while (EntryIndex < FreeList.Num())
	{
		FEntry& It = FreeList[EntryIndex];

		if ((It.LastUsedFrame + MaxAgeInFrames) <= CurrentFrame
			|| (It.LastUsedTime + MaxAgeInSeconds) <= CurrentTime)
		{
			if (It.Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV)
			{
				DEC_DWORD_STAT(STAT_ExplicitViewDescriptorHeaps);
				DEC_DWORD_STAT_BY(STAT_ExplicitViewDescriptors, It.NumDescriptors);
			}
			else
			{
				DEC_DWORD_STAT(STAT_ExplicitSamplerDescriptorHeaps);
				DEC_DWORD_STAT_BY(STAT_ExplicitSamplerDescriptors, It.NumDescriptors);
			}

			FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(It.Heap);

			FreeList.RemoveAtSwap(EntryIndex, EAllowShrinking::No);
		}
		else
		{
			EntryIndex++;
		}
	}
}

void FD3D12ExplicitDescriptorHeapCache::FlushFreeList()
{
	FScopeLock Lock(&CriticalSection);

	for (const FEntry& It : FreeList)
	{
		FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(It.Heap);
	}
	FreeList.Empty();
}

///

FD3D12ExplicitDescriptorHeap::~FD3D12ExplicitDescriptorHeap()
{
	if (D3D12Heap)
	{
		GetParentDevice()->GetExplicitDescriptorHeapCache()->DeferredReleaseHeap(MoveTemp(HeapCacheEntry));
	}
}

void FD3D12ExplicitDescriptorHeap::Init(uint32 InMaxNumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE InType)
{
	check(D3D12Heap == nullptr);

	Type = InType;
	HeapCacheEntry = GetParentDevice()->GetExplicitDescriptorHeapCache()->AllocateHeap(Type, InMaxNumDescriptors);

	MaxNumDescriptors = HeapCacheEntry.NumDescriptors;
	D3D12Heap = HeapCacheEntry.Heap;

	CPUBase = D3D12Heap->GetCPUDescriptorHandleForHeapStart();
	GPUBase = D3D12Heap->GetGPUDescriptorHandleForHeapStart();

	checkf(CPUBase.ptr, TEXT("Explicit descriptor heap of type %d returned from descriptor heap cache is invalid."), Type);

	DescriptorSize = GetParentDevice()->GetDevice()->GetDescriptorHandleIncrementSize(Type);

	bExhaustiveSamplerDeduplication = GD3D12ExplicitDeduplicateSamplers == 1;

	if (bExhaustiveSamplerDeduplication && InType == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER)
	{
		// When exhaustive descriptor deduplication is active, all shadow descriptor table entries must be initialized.
		// Deduplication works by looping over all elements, but they may be written out of order by worker threads.
		// Initializing descriptors to 0 avoids accidentally matching wrong descriptor entries.
		Descriptors.SetNumZeroed(MaxNumDescriptors);
	}
#if EXPLICIT_DESCRIPTOR_CACHE_FULL_COMPARE
	else
	{
		// If exhaustive sampler deduplication is not used, then descriptor table copy is only used for validation.
		// Entries are guaranteed to be written before they're read, so we don't have to spend the time on initialization.
		Descriptors.SetNumUninitialized(MaxNumDescriptors);
	}
#endif
}

// Returns descriptor heap base index or -1 if allocation is not possible.
// Thread-safe (uses atomic linear allocation).
int32 FD3D12ExplicitDescriptorHeap::Allocate(uint32 InNumDescriptors)
{
	int32 Result = FPlatformAtomics::InterlockedAdd(&NumAllocatedDescriptors, InNumDescriptors);

	if (Result + InNumDescriptors > MaxNumDescriptors)
	{
		if (Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER)
		{
			UE_LOGF(LogD3D12RHI, Fatal,
			       "Explicit sampler descriptor heap overflow. " "It is not possible to recover from this error, as maximum D3D12 sampler heap size is 2048.");
		}
		else if ((uint32)GD3D12ExplicitViewDescriptorHeapSize <= MaxNumDescriptors && FPlatformAtomics::InterlockedOr(&GD3D12ExplicitViewDescriptorHeapOverflowReported, 1) == 0)
		{
			// NOTE: GD3D12RayTracingViewDescriptorHeapOverflowReported is set atomically because multiple 
			// allocations may be happening simultaneously, but we only want to report the error once.

			UE_LOGF(LogD3D12RHI, Error,
			       "Explicit view descriptor heap overflow. Current frame will not be rendered correctly. " "Increase r.D3D12.RayTracingViewDescriptorHeapSize to at least %d to fix this issue.",
			       MaxNumDescriptors * 2);
		}

		Result = -1;
	}

	return Result;
}

void FD3D12ExplicitDescriptorHeap::CopyDescriptors(int32 BaseIndex, const D3D12_CPU_DESCRIPTOR_HANDLE* InDescriptors, uint32 InNumDescriptors)
{
	D3D12_CPU_DESCRIPTOR_HANDLE DestDescriptor = GetDescriptorCPU(BaseIndex);
	FD3D12Device::CopyDescriptors(D3DDevice, DestDescriptor, InDescriptors, InNumDescriptors, Type);

#if EXPLICIT_DESCRIPTOR_CACHE_FULL_COMPARE
	for (uint32 i = 0; i < InNumDescriptors; ++i)
	{
		Descriptors[BaseIndex + i].ptr = InDescriptors[i].ptr;
	}
#endif
}

bool FD3D12ExplicitDescriptorHeap::CompareDescriptors(int32 BaseIndex, const D3D12_CPU_DESCRIPTOR_HANDLE* InDescriptors, uint32 InNumDescriptors)
{
	for (uint32 i = 0; i < InNumDescriptors; ++i)
	{
		if (Descriptors[BaseIndex + i].ptr != InDescriptors[i].ptr)
		{
			return false;
		}
	}
	return true;
}

D3D12_CPU_DESCRIPTOR_HANDLE FD3D12ExplicitDescriptorHeap::GetDescriptorCPU(uint32 Index) const
{
	checkSlow(Index < MaxNumDescriptors);
	D3D12_CPU_DESCRIPTOR_HANDLE Result = { CPUBase.ptr + Index * DescriptorSize };
	return Result;
}

D3D12_GPU_DESCRIPTOR_HANDLE FD3D12ExplicitDescriptorHeap::GetDescriptorGPU(uint32 Index) const
{
	checkSlow(Index < MaxNumDescriptors);
	D3D12_GPU_DESCRIPTOR_HANDLE Result = { GPUBase.ptr + Index * DescriptorSize };
	return Result;
}

///

void FD3D12ExplicitDescriptorCache::Init(uint32 NumConstantDescriptors, uint32 NumViewDescriptors, uint32 NumSamplerDescriptors, ERHIBindlessConfiguration BindlessConfig)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorManager& BindlessManager = GetParentDevice()->GetBindlessDescriptorManager();

	BindlessConfiguration = BindlessConfig;
	bBindless = (BindlessConfig != ERHIBindlessConfiguration::Disabled && BindlessManager.GetConfiguration() >= BindlessConfig);
#else
	const bool bBindless = false;
#endif

	const uint32 TotalViewDescriptors = NumConstantDescriptors + (bBindless ? 0 : NumViewDescriptors);
	if (TotalViewDescriptors)
	{
		ViewHeap.Init(TotalViewDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);
	}
		
	if (!bBindless)
	{
		SamplerHeap.Init(NumSamplerDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER);
	}
}

// Returns descriptor heap base index for this descriptor table allocation (checking for duplicates and reusing existing tables) or -1 if allocation failed.
int32 FD3D12ExplicitDescriptorCache::AllocateDeduplicated(const uint32* DescriptorVersions, const D3D12_CPU_DESCRIPTOR_HANDLE* Descriptors, uint32 NumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 WorkerIndex)
{
	checkSlow(Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV || Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER);

	FD3D12ExplicitDescriptorHeap& Heap = (Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV) ? ViewHeap : SamplerHeap;
	TDescriptorHashMap& Map = (Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV) 
		? WorkerData[WorkerIndex].ViewDescriptorTableCache
		: WorkerData[WorkerIndex].SamplerDescriptorTableCache;

	const uint64 VersionHash = FXxHash64::HashBuffer(DescriptorVersions, sizeof(DescriptorVersions[0]) * NumDescriptors).Hash;
	const uint64 DescriptorHash = FXxHash64::HashBuffer(Descriptors, sizeof(Descriptors[0]) * NumDescriptors).Hash;
	const uint64 Key = VersionHash ^ DescriptorHash;

	int32& DescriptorTableBaseIndex = Map.FindOrAdd(Key, INDEX_NONE);

	if (DescriptorTableBaseIndex != INDEX_NONE)
	{
	#if EXPLICIT_DESCRIPTOR_CACHE_FULL_COMPARE
		if (ensureMsgf(Heap.CompareDescriptors(DescriptorTableBaseIndex, Descriptors, NumDescriptors),
			TEXT("Explicit descriptor cache hash collision detected!")))
	#endif
		{
			return DescriptorTableBaseIndex;
		}
	}

	if (Heap.bExhaustiveSamplerDeduplication && Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER && WorkerData.Num() > 1)
	{
		// Exhaustive search for a sampler table:
		// We have to do this because sampler heap space is precious (hard limit of 2048 total entries).
		// Per-thread descriptor table deduplication hash tables introduce a lot of redundancy in the heap
		// which is reduced by looking for global matches on hash table lookup miss.

		int32 FoundIndex = INDEX_NONE;
		int32 SearchEndPos = Heap.NumWrittenSamplerDescriptors;
		for (int32 SearchIndex = 0; SearchIndex + int32(NumDescriptors) < SearchEndPos; ++SearchIndex)
		{
			if (Heap.CompareDescriptors(SearchIndex, Descriptors, NumDescriptors))
			{
				FoundIndex = SearchIndex;
				break;
			}
		}

		if (FoundIndex != INDEX_NONE)
		{
			DescriptorTableBaseIndex = FoundIndex;
			return DescriptorTableBaseIndex;
		}
	}

	DescriptorTableBaseIndex = Allocate(Descriptors, NumDescriptors, Type, WorkerIndex);
	return DescriptorTableBaseIndex;
}

// Returns descriptor heap base index for this descriptor table allocation or -1 if allocation failed.
int32 FD3D12ExplicitDescriptorCache::Allocate(const D3D12_CPU_DESCRIPTOR_HANDLE* Descriptors, uint32 NumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 WorkerIndex)
{
	checkSlow(Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV || Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER);

	FD3D12ExplicitDescriptorHeap& Heap = (Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV) ? ViewHeap : SamplerHeap;

	int32 DescriptorTableBaseIndex = INDEX_NONE; 
	
	if (Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV)
	{
		DescriptorTableBaseIndex = WorkerData[WorkerIndex].ReservedViewDescriptors.Allocate(NumDescriptors);
		if (DescriptorTableBaseIndex == INDEX_NONE)
		{
			DescriptorTableBaseIndex = Heap.Allocate(NumDescriptors);
		}
	}
	else
	{
		DescriptorTableBaseIndex = Heap.Allocate(NumDescriptors);
	}

	if (DescriptorTableBaseIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	Heap.CopyDescriptors(DescriptorTableBaseIndex, Descriptors, NumDescriptors);

	if (Heap.bExhaustiveSamplerDeduplication && Type == D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER)
	{
		FPlatformAtomics::InterlockedAdd(&Heap.NumWrittenSamplerDescriptors, NumDescriptors);
	}

	if (Type == D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV)
	{
		INC_DWORD_STAT_BY(STAT_ExplicitUsedViewDescriptors, NumDescriptors);
	}
	else
	{
		INC_DWORD_STAT_BY(STAT_ExplicitUsedSamplerDescriptors, NumDescriptors);

		static uint64 GMaxNumUsedSamplerDescriptors = 0;
		if (Heap.NumAllocatedDescriptors > GMaxNumUsedSamplerDescriptors)
		{
			GMaxNumUsedSamplerDescriptors = Heap.NumAllocatedDescriptors;
		}
		SET_DWORD_STAT(STAT_ExplicitMaxUsedSamplerDescriptors, GMaxNumUsedSamplerDescriptors);
	}

	return DescriptorTableBaseIndex;
}
