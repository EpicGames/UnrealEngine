// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#include "Experimental/Containers/SherwoodHashTable.h"

class FD3D12DynamicRHI;
struct FD3D12DefaultViews;
class FD3D12CommandContext;
class FD3D12DescriptorCache;
struct FD3D12VertexBufferCache;
struct FD3D12IndexBufferCache;
struct FD3D12ConstantBufferCache;
struct FD3D12ShaderResourceViewCache;
struct FD3D12UnorderedAccessViewCache;
struct FD3D12SamplerStateCache;

// #dxr_todo UE-72158: FD3D12Device::GlobalViewHeap/GlobalSamplerHeap should be used instead of ad-hoc heaps here.
// Unfortunately, this requires a major refactor of how global heaps work.
// FD3D12CommandContext-s should not get static chunks of the global heap, but instead should dynamically allocate
// chunks on as-needed basis and release them when possible.
// This would allow calling code to sub-allocate heap blocks from the same global heap.
class FD3D12ExplicitDescriptorHeapCache : FD3D12DeviceChild
{
public:

	UE_NONCOPYABLE(FD3D12ExplicitDescriptorHeapCache)

	struct FEntry
	{
		ID3D12DescriptorHeap* Heap = nullptr;
		uint32 NumDescriptors = 0;
		D3D12_DESCRIPTOR_HEAP_TYPE Type = D3D12_DESCRIPTOR_HEAP_TYPE_NUM_TYPES;

		// Information for stale entry release, updated upon adding the entry to the free list
		uint64 LastUsedFrame = 0;
		double LastUsedTime = 0.0;
	};

	FD3D12ExplicitDescriptorHeapCache(FD3D12Device* Device)
	: FD3D12DeviceChild(Device)
	{
	}

	~FD3D12ExplicitDescriptorHeapCache();

	FEntry AllocateHeap(D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 NumDescriptors);
	void DeferredReleaseHeap(FEntry&& Entry);

	void FlushFreeList();

private:

	void ReleaseHeap(FEntry&& Entry);

	// Assumes CriticalSection is already locked
	void ReleaseStaleEntries(uint32 MaxAgeInFrames, float MaxAgeInSeconds);

	FCriticalSection CriticalSection;
	TArray<FEntry> FreeList;
	uint32 NumAllocatedEntries = 0;
};

struct FD3D12ExplicitDescriptorHeap : public FD3D12DeviceChild
{
	UE_NONCOPYABLE(FD3D12ExplicitDescriptorHeap)

	FD3D12ExplicitDescriptorHeap(FD3D12Device* Device);
	~FD3D12ExplicitDescriptorHeap();

	void Init(uint32 InMaxNumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE InType);

	// Returns descriptor heap base index or -1 if allocation is not possible.
	// Thread-safe (uses atomic linear allocation).
	int32 Allocate(uint32 InNumDescriptors);

	void CopyDescriptors(int32 BaseIndex, const D3D12_CPU_DESCRIPTOR_HANDLE* InDescriptors, uint32 InNumDescriptors);

	bool CompareDescriptors(int32 BaseIndex, const D3D12_CPU_DESCRIPTOR_HANDLE* InDescriptors, uint32 InNumDescriptors);

	D3D12_CPU_DESCRIPTOR_HANDLE GetDescriptorCPU(uint32 Index) const;
	D3D12_GPU_DESCRIPTOR_HANDLE GetDescriptorGPU(uint32 Index) const;

	// Cache D3D device pointer, as it's frequently accessed on the hot path in CopyDescriptors
	ID3D12Device* D3DDevice = nullptr;

	D3D12_DESCRIPTOR_HEAP_TYPE Type = D3D12_DESCRIPTOR_HEAP_TYPE_NUM_TYPES;
	ID3D12DescriptorHeap* D3D12Heap = nullptr;
	uint32 MaxNumDescriptors = 0;

	int32 NumAllocatedDescriptors = 0;

	// Marks the valid range of the heap when exhaustive sampler deduplication is enabled. Not used otherwise.
	int32 NumWrittenSamplerDescriptors = 0;

	uint32 DescriptorSize = 0;
	D3D12_CPU_DESCRIPTOR_HANDLE CPUBase = {};
	D3D12_GPU_DESCRIPTOR_HANDLE GPUBase = {};

	FD3D12ExplicitDescriptorHeapCache::FEntry HeapCacheEntry;

	TArray<D3D12_CPU_DESCRIPTOR_HANDLE> Descriptors;

	bool bExhaustiveSamplerDeduplication = false;
};

class FD3D12ExplicitDescriptorCache : public FD3D12DeviceChild
{
public:

	UE_NONCOPYABLE(FD3D12ExplicitDescriptorCache)

	FD3D12ExplicitDescriptorCache(FD3D12Device* Device, uint32 MaxWorkerCount)
	: FD3D12DeviceChild(Device)
	, ViewHeap(Device)
	, SamplerHeap(Device)
	{
		check(MaxWorkerCount > 0u);
		WorkerData.SetNum(MaxWorkerCount);
	}

	void Init(uint32 NumConstantDescriptors, uint32 NumViewDescriptors, uint32 NumSamplerDescriptors, ERHIBindlessConfiguration BindlessConfig);

	// Returns descriptor heap base index for this descriptor table allocation or -1 if allocation failed.
	int32 Allocate(const D3D12_CPU_DESCRIPTOR_HANDLE* Descriptors, uint32 NumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 WorkerIndex);

	// Returns descriptor heap base index for this descriptor table allocation (checking for duplicates and reusing existing tables) or -1 if allocation failed.
	int32 AllocateDeduplicated(const uint32* DescriptorVersions, const D3D12_CPU_DESCRIPTOR_HANDLE* Descriptors, uint32 NumDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type, uint32 WorkerIndex);

	FD3D12ExplicitDescriptorHeap ViewHeap;
	FD3D12ExplicitDescriptorHeap SamplerHeap;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	ERHIBindlessConfiguration BindlessConfiguration{};
	bool bBindless = false;
#endif

	template<typename KeyType>
	struct TIdentityHash
	{
		static FORCEINLINE bool Matches(KeyType A, KeyType B)
		{
			return A == B;
		}
		static FORCEINLINE uint32 GetKeyHash(KeyType Key)
		{
			return (uint32)Key;
		}
	};

	using TDescriptorHashMap = Experimental::TSherwoodMap<uint64, int32, TIdentityHash<uint64>>;

	struct FDescriptorSlotRange
	{
		FDescriptorSlotRange() = default;
		FDescriptorSlotRange(int32 BaseIndex, int32 Count)
		: Begin(BaseIndex)
		, Cursor(BaseIndex)
		, End(BaseIndex + Count)
		{
		}

		int32 Begin = 0;
		int32 Cursor = 0;
		int32 End = 0;

		int32 Allocate(int32 Count)
		{
			int32 Result = INDEX_NONE;
			if (Cursor + Count <= End)
			{
				Result = Cursor;
				Cursor += Count;
			}
			return Result;
		}
	};

	void ReserveViewDescriptors(uint32 Count, uint32 WorkerIndex)
	{
		const int32 BaseIndex = ViewHeap.Allocate(Count);
		if (BaseIndex != INDEX_NONE)
		{
			WorkerData[WorkerIndex].ReservedViewDescriptors = FDescriptorSlotRange(BaseIndex, Count);
		}
	}

	struct alignas(PLATFORM_CACHE_LINE_SIZE) FWorkerThreadData
	{
		TDescriptorHashMap ViewDescriptorTableCache;
		TDescriptorHashMap SamplerDescriptorTableCache;

		FDescriptorSlotRange ReservedViewDescriptors;
	};

	TArray<FWorkerThreadData> WorkerData;
};