// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12IntelExtensions.h"
#include "D3D12RHICommon.h"
#include "D3D12RHIPrivate.h"

#if INTEL_EXTENSIONS

#if INTEL_GPU_CRASH_DUMPS
namespace UE::RHICore::Intel::GPUCrashDumps
{
	extern RHICORE_API TAutoConsoleVariable<int32> CVarIntelCrashDumps;
}
#endif // INTEL_GPU_CRASH_DUMPS

bool MatchIntelExtensionVersion(const INTCExtensionVersion& ExtensionsVersion)
{
	// Make sure these are sorted with the highest version first (most features)
	const INTCSupportedVersion SupportedExtensionsVersions[] =
	{
#if INTEL_GPU_CRASH_DUMPS
		// Intel GPU crash dumps
		{
			{ 4, 14, 0 },
			&(*UE::RHICore::Intel::GPUCrashDumps::CVarIntelCrashDumps)
		},
#endif // INTEL_GPU_CRASH_DUMPS

		// Emulated Typed 64bit Atomics - this is required to run Nanite on ACM (DG2)
		{
			{ 4, 8, 0 },
			nullptr
		},
	};

	for (const INTCSupportedVersion& SupportedVersion : SupportedExtensionsVersions)
	{
		if (SupportedVersion.CVar && SupportedVersion.CVar->GetInt() == 0)
		{
			continue;			// skip disabled feature
		}

		if (ExtensionsVersion.HWFeatureLevel == SupportedVersion.Version.HWFeatureLevel &&
			ExtensionsVersion.APIVersion == SupportedVersion.Version.APIVersion &&
			ExtensionsVersion.Revision == SupportedVersion.Version.Revision)
		{
			// First match wins
			return true;
		}
	}

	// No matches
	return false;
}

#if INTEL_GPU_CRASH_DUMPS
namespace UE::RHICore::Intel::GPUCrashDumps::D3D12
{
	inline INTCExtensionContext* GetIntelExtensionContext()
	{
		return FD3D12DynamicRHI::GetD3DRHI()->GetIntelExtensionContext();
	}

	inline constexpr uint32 GetEventMarkerBreadcrumbType()
	{
#if INTEL_BREADCRUMBS_USE_BREADCRUMB_PTRS
		return INTC_EVENT_MARKER_PTR;
#else
		return INTC_EVENT_MARKER_WSTRING;
#endif
	}

	uint64_t RegisterCommandList(ID3D12CommandList* pCommandList)
	{
		if (!IsEnabled())
		{
			return 0;
		}

		return INTC_D3D12_GetCommandListHandle(GetIntelExtensionContext(), pCommandList);
	}

#if WITH_RHI_BREADCRUMBS
	void BeginBreadcrumb(ID3D12GraphicsCommandList* pCommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		if (!IsEnabled())
		{
			return;
		}

		FMarker Marker(Breadcrumb);
		if (Marker)
		{
			INTC_D3D12_SetEventMarker(GetIntelExtensionContext(), pCommandList, INTC_EVENT_MARKER_BEGIN | GetEventMarkerBreadcrumbType(), Marker.GetPtr(), Marker.GetSize());
		}
	}

	void EndBreadcrumb(ID3D12GraphicsCommandList* pCommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		if (!IsEnabled())
		{
			return;
		}

		FMarker Marker(Breadcrumb);
		if (Marker)
		{
			INTC_D3D12_SetEventMarker(GetIntelExtensionContext(), pCommandList, INTC_EVENT_MARKER_END | GetEventMarkerBreadcrumbType(), Marker.GetPtr(), Marker.GetSize());
		}
	}
#endif
}
#endif // INTEL_GPU_CRASH_DUMPS

#endif // INTEL_EXTENSIONS
