// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if INTEL_EXTENSIONS

	#include "RHICoreIntelBreadcrumbs.h"
	#include "Windows/D3DIntelExtensions.h"

	extern bool GDX12INTCAtomicUInt64Emulation;

	struct INTCExtensionContext;
	struct INTCExtensionInfo;

	struct INTCSupportedVersion
	{
		INTCExtensionVersion				Version;			// Required version
		IConsoleVariable*					CVar;				// Console variable that controls this feature
	};
	extern INTCExtensionVersion IntelExtensionsVersion;

	// Offsets for the version structure
	#define INTEL_EXTENSION_VERSION_GENERIC			0		// Generic version for all UE5 targets
	#define INTEL_EXTENSION_VERSION_BREADCRUMBS		1		// Intel Breadcrumbs support

	bool MatchIntelExtensionVersion(const INTCExtensionVersion& Version);

	INTCExtensionContext* CreateIntelExtensionsContext(ID3D12Device* Device, INTCExtensionInfo& INTCExtensionInfo, uint32 DeviceId = 0);
	void DestroyIntelExtensionsContext(INTCExtensionContext* IntelExtensionContext);

	bool EnableIntelAtomic64Support(INTCExtensionContext* IntelExtensionContext, INTCExtensionInfo& INTCExtensionInfo);
	bool TryCheckIntelAsyncCompute(ID3D12Device* Device, uint32 DeviceId, bool &bOutSupported);

#if INTEL_GPU_CRASH_DUMPS
	namespace UE::RHICore::Intel::GPUCrashDumps::D3D12
	{
		uint64_t RegisterCommandList( ID3D12CommandList* pCommandList );
		//void UnregisterCommandList( FCommandList CommandList );

#if WITH_RHI_BREADCRUMBS
		void BeginBreadcrumb( ID3D12GraphicsCommandList* pCommandList, FRHIBreadcrumbNode* Breadcrumb );
		void EndBreadcrumb( ID3D12GraphicsCommandList* pCommandList, FRHIBreadcrumbNode* Breadcrumb );
#endif
	}
#endif

#endif //INTEL_EXTENSIONS
