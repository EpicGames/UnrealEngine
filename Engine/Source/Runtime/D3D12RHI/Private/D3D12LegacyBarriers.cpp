// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12LegacyBarriers.h"

#if D3D12RHI_SUPPORTS_LEGACY_BARRIERS

#include "RHIResources.h"
#include "RHICoreTransitions.h"

#include "D3D12Adapter.h"
#include "D3D12RHIPrivate.h"

#if INTEL_EXTENSIONS
#include "D3D12RHIPrivate.h"
#endif


// Each platform must provide its own implementation of this
extern D3D12_RESOURCE_STATES GetSkipFastClearEliminateStateFlags();

// Custom resource states
// To Be Determined (TBD) means we need to fill out a resource barrier before the command list is executed.
#define D3D12_RESOURCE_STATE_TBD D3D12_RESOURCE_STATES(-1 ^ (1 << 31))
#define D3D12_RESOURCE_STATE_CORRUPT D3D12_RESOURCE_STATES(-2 ^ (1 << 31))

static bool IsValidD3D12ResourceState(D3D12_RESOURCE_STATES InState)
{
	return (InState != D3D12_RESOURCE_STATE_TBD && InState != D3D12_RESOURCE_STATE_CORRUPT);
}

static bool IsDirectQueueExclusiveD3D12State(D3D12_RESOURCE_STATES InState)
{
	return EnumHasAnyFlags(InState, 
		D3D12_RESOURCE_STATE_RENDER_TARGET
		| D3D12_RESOURCE_STATE_DEPTH_WRITE
		| D3D12_RESOURCE_STATE_DEPTH_READ
		| D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE);
}

struct FD3D12LegacyBarriersTransitionData
{
	ERHIPipeline SrcPipelines, DstPipelines;
	ERHITransitionCreateFlags CreateFlags = ERHITransitionCreateFlags::None;

	TArray<FRHITransitionInfo, TInlineAllocator<4, FConcurrentLinearArrayAllocator>> TransitionInfos;
	TArray<FRHITransientAliasingInfo, TInlineAllocator<4, FConcurrentLinearArrayAllocator>> AliasingInfos;
	TArray<FRHITransientAliasingOverlap, TInlineAllocator<4, FConcurrentLinearArrayAllocator>> AliasingOverlaps;

	TArray<TRHIPipelineArray<FD3D12SyncPointRef>, TInlineAllocator<MAX_NUM_GPUS>> SyncPoints;

	bool bCrossPipeline = false;
	bool bAsyncToAllPipelines = false;
};

static FString ConvertToResourceStateString(
	uint32 ResourceState)
{
	if (ResourceState == 0)
	{
		return TEXT("D3D12_RESOURCE_STATE_COMMON");
	}

	if (ResourceState == D3D12_RESOURCE_STATE_TBD)
	{
		return TEXT("D3D12_RESOURCE_STATE_TBD");
	}

	const TCHAR* ResourceStateNames[] =
	{
		TEXT("D3D12_RESOURCE_STATE_VERTEX_AND_CONSTANT_BUFFER"),
		TEXT("D3D12_RESOURCE_STATE_INDEX_BUFFER"),
		TEXT("D3D12_RESOURCE_STATE_RENDER_TARGET"),
		TEXT("D3D12_RESOURCE_STATE_UNORDERED_ACCESS"),
		TEXT("D3D12_RESOURCE_STATE_DEPTH_WRITE"),
		TEXT("D3D12_RESOURCE_STATE_DEPTH_READ"),
		TEXT("D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE"),
		TEXT("D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE"),
		TEXT("D3D12_RESOURCE_STATE_STREAM_OUT"),
		TEXT("D3D12_RESOURCE_STATE_INDIRECT_ARGUMENT"),
		TEXT("D3D12_RESOURCE_STATE_COPY_DEST"),
		TEXT("D3D12_RESOURCE_STATE_COPY_SOURCE"),
		TEXT("D3D12_RESOURCE_STATE_RESOLVE_DEST"),
		TEXT("D3D12_RESOURCE_STATE_RESOLVE_SOURCE"),
	};

	FString ResourceStateString;
	uint16 NumStates = 0;
	for (uint16 i = 0; ResourceState && i < ARRAYSIZE(ResourceStateNames); i++)
	{
		if (ResourceState & 1)
		{
			if (NumStates > 0)
			{
				ResourceStateString += " | ";
			}

			ResourceStateString += ResourceStateNames[i];
			NumStates++;
		}
		ResourceState = ResourceState >> 1;
	}
	return ResourceStateString;
}

#if D3D12RHI_BARRIER_LOGGING

static void LogResourceBarriers(
	TConstArrayView<D3D12_RESOURCE_BARRIER> Barriers,
	ID3D12CommandList* const pCommandList,
	ED3D12QueueType QueueType)
{
	// Configure what resource barriers are logged.
	const FString& Filter = GD3D12LogBarriersResourceFilter;
	const bool bLogAll = true;
	const bool bLogTransitionDepth = true;
	const bool bLogTransitionRenderTarget = true;
	const bool bLogTransitionUAV = true;
	const bool bCheckResourceName = !Filter.IsEmpty();

	// Create the state bit mask to indicate what barriers should be logged.
	uint32 ShouldLogMask = bLogAll ? static_cast<uint32>(-1) : 0;
	ShouldLogMask |= bLogTransitionDepth ? D3D12_RESOURCE_STATE_DEPTH_READ | D3D12_RESOURCE_STATE_DEPTH_WRITE : 0;
	ShouldLogMask |= bLogTransitionRenderTarget ? D3D12_RESOURCE_STATE_RENDER_TARGET : 0;
	ShouldLogMask |= bLogTransitionUAV ? D3D12_RESOURCE_STATE_UNORDERED_ACCESS : 0;

	for (int32 i = 0; i < Barriers.Num(); i++)
	{
		const D3D12_RESOURCE_BARRIER& currentBarrier = Barriers[i];

		switch (currentBarrier.Type)
		{
		case D3D12_RESOURCE_BARRIER_TYPE_TRANSITION:
		{
			const FString StateBefore = ConvertToResourceStateString(static_cast<uint32>(currentBarrier.Transition.StateBefore));
			const FString StateAfter = ConvertToResourceStateString(static_cast<uint32>(currentBarrier.Transition.StateAfter));

			bool bShouldLog = bLogAll;
			if (!bShouldLog)
			{
				// See if we should log this transition.
				for (uint32 j = 0; (j < 2) && !bShouldLog; j++)
				{
					const D3D12_RESOURCE_STATES& State = (j == 0) ? currentBarrier.Transition.StateBefore : currentBarrier.Transition.StateAfter;
					bShouldLog = (State & ShouldLogMask) > 0;
				}
			}

			if (bShouldLog)
			{
				const FString BarrierResourceName = GetD312ObjectName(currentBarrier.Transition.pResource);

				if (!bCheckResourceName || BarrierResourceName.Contains(Filter))
				{
					UE_LOGF(LogD3D12RHI, Log, "*** BARRIER (CmdList: %016llX, Queue: %ls) %u/%u: %ls %016llX (Sub: %u), %ls -> %ls", pCommandList, GetD3DCommandQueueTypeName(QueueType), i + 1, Barriers.Num(),
						*BarrierResourceName,
						currentBarrier.Transition.pResource,
						currentBarrier.Transition.Subresource,
						*StateBefore,
						*StateAfter);
				}
			}
			break;
		}

		case D3D12_RESOURCE_BARRIER_TYPE_UAV:
			{
				const FString BarrierResourceName = GetD312ObjectName(currentBarrier.UAV.pResource);
				if (!bCheckResourceName || BarrierResourceName.Contains(Filter))
				{
					UE_LOGF(LogD3D12RHI, Log, "*** BARRIER (CmdList: %016llX, Queue: %ls) %u/%u: UAV Barrier %ls", pCommandList, GetD3DCommandQueueTypeName(QueueType), i + 1, Barriers.Num(), *BarrierResourceName);
				}
			}
			break;

		case D3D12_RESOURCE_BARRIER_TYPE_ALIASING:
			{
				const FString BarrierResourceNameBefore = GetD312ObjectName(currentBarrier.Aliasing.pResourceBefore);
				const FString BarrierResourceNameAfter = GetD312ObjectName(currentBarrier.Aliasing.pResourceAfter);
				
				if (!bCheckResourceName || BarrierResourceNameBefore.Contains(Filter) || BarrierResourceNameAfter.Contains(Filter))
				{
					UE_LOGF(LogD3D12RHI, Log, "*** BARRIER (CmdList: %016llX, Queue: %ls) %u/%u: Aliasing Barrier, %016llX %ls -> %016llX %ls", pCommandList, GetD3DCommandQueueTypeName(QueueType), i + 1, Barriers.Num(), currentBarrier.Aliasing.pResourceBefore, *BarrierResourceNameBefore, currentBarrier.Aliasing.pResourceAfter, *BarrierResourceNameAfter);
				}
			}
			break;

		default:
			check(false);
			break;
		}
	}
}

#endif // D3D12RHI_BARRIER_LOGGING

static D3D12_RESOURCE_STATES GetDiscardedResourceState(
	const D3D12_RESOURCE_DESC& InDesc,
	ED3D12QueueType QueueType)
{
	// Validate the creation state
	D3D12_RESOURCE_STATES State = D3D12_RESOURCE_STATE_COMMON;
	if (EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET) && QueueType == ED3D12QueueType::Direct)
	{
		State = D3D12_RESOURCE_STATE_RENDER_TARGET;
	}
	else if (EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL) && QueueType == ED3D12QueueType::Direct)
	{
		State = D3D12_RESOURCE_STATE_DEPTH_WRITE;
	}
	else if (EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS))
	{
		State = D3D12_RESOURCE_STATE_UNORDERED_ACCESS;
	}
	return State;
}

static D3D12_RESOURCE_STATES GetD3D12ResourceState(
	ED3D12Access InD3D12Access,
	ED3D12QueueType InQueueType,
	const FD3D12ResourceDesc& InResourceDesc,
	const FD3D12Texture* InRHID3D12Texture)
{
	if (InD3D12Access == ED3D12Access::Discard)
	{
		return GetDiscardedResourceState(InResourceDesc, InQueueType);
	}

	const ED3D12Access D3D12AccessWithoutDiscard = InD3D12Access & ~ED3D12Access::Discard;

	// Add switch for common states (should cover all writeable states)
	switch (D3D12AccessWithoutDiscard)
	{
		// Common is a state all its own
	case ED3D12Access::Common:				return D3D12_RESOURCE_STATE_COMMON;

		// All single write states
	case ED3D12Access::RTV:					return D3D12_RESOURCE_STATE_RENDER_TARGET;
#if D3D12_RHI_RAYTRACING
	case ED3D12Access::BVHRead:
	case ED3D12Access::BVHWrite:				return D3D12_RESOURCE_STATE_RAYTRACING_ACCELERATION_STRUCTURE;
#endif
	case ED3D12Access::UAVMask:
	case ED3D12Access::UAVCompute:
	case ED3D12Access::UAVGraphics:			return D3D12_RESOURCE_STATE_UNORDERED_ACCESS;
	case ED3D12Access::DSVWrite:				return D3D12_RESOURCE_STATE_DEPTH_WRITE;
	case ED3D12Access::CopyDest:				return D3D12_RESOURCE_STATE_COPY_DEST;
	case ED3D12Access::ResolveDst:			return D3D12_RESOURCE_STATE_RESOLVE_DEST;
	case ED3D12Access::Present:				return D3D12_RESOURCE_STATE_PRESENT;

		// Generic read for mask read states
	case ED3D12Access::GenericRead:
	case ED3D12Access::ReadOnlyMask:
	case ED3D12Access::ReadOnlyExclusiveMask:	return D3D12_RESOURCE_STATE_GENERIC_READ;
	default:
	{
		D3D12_RESOURCE_STATES ExtraReadState = {};

		if (InRHID3D12Texture)
		{
// 			if (InRHID3D12Texture->GetResource()->IsDepthStencilResource())
// 			{
// 				ExtraReadState |= D3D12_RESOURCE_STATE_DEPTH_READ;
// 			}
			 
			if (InRHID3D12Texture->SkipsFastClearFinalize())
			{
				ExtraReadState |= GetSkipFastClearEliminateStateFlags();
			}
		}

		// Special case for DSV read & write (Depth write allows depth read as well in D3D)
		if (D3D12AccessWithoutDiscard == ED3D12Access(ED3D12Access::DSVRead | ED3D12Access::DSVWrite))
		{
			return D3D12_RESOURCE_STATE_DEPTH_WRITE;
		}
#if D3D12_RHI_RAYTRACING
		else if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::BVHRead | ED3D12Access::BVHWrite))
		{
			return D3D12_RESOURCE_STATE_RAYTRACING_ACCELERATION_STRUCTURE;
		}
#endif
		else
		{
			// Should be combination from read only flags (write flags covered above)
			check(!(EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::WritableMask)));
			check(EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::ReadOnlyMask));

			D3D12_RESOURCE_STATES State = D3D12_RESOURCE_STATE_COMMON;

			// Translate the requested after state to a D3D state
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::SRVGraphics) && InQueueType == ED3D12QueueType::Direct)
			{
				State |= D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE | D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE | ExtraReadState;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::SRVCompute))
			{
				State |= D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE | ExtraReadState;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::VertexOrIndexBuffer) && InQueueType == ED3D12QueueType::Direct)
			{
				State |= D3D12_RESOURCE_STATE_VERTEX_AND_CONSTANT_BUFFER | D3D12_RESOURCE_STATE_INDEX_BUFFER;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::CopySrc))
			{
				State |= D3D12_RESOURCE_STATE_COPY_SOURCE;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::IndirectArgs))
			{
				State |= D3D12_RESOURCE_STATE_INDIRECT_ARGUMENT;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::ResolveSrc) && InQueueType == ED3D12QueueType::Direct)
			{
				State |= D3D12_RESOURCE_STATE_RESOLVE_SOURCE;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::DSVRead) && InQueueType == ED3D12QueueType::Direct)
			{
				State |= D3D12_RESOURCE_STATE_DEPTH_READ;
			}
			if (EnumHasAnyFlags(D3D12AccessWithoutDiscard, ED3D12Access::ShadingRateSource) && InQueueType == ED3D12QueueType::Direct)
			{
#if !UE_BUILD_SHIPPING
				if (GRHISupportsAttachmentVariableRateShading == false)
				{
					static bool bLogOnce = true;
					if (bLogOnce)
					{
						UE_LOGF(LogD3D12RHI, Warning, "(%ls) Resource state is D3D12_RESOURCE_STATE_SHADING_RATE_SOURCE but RHI does not support VRS.", InRHID3D12Texture == nullptr ? TEXT("Unknown") : *InRHID3D12Texture->GetName().GetPlainNameString());
					}
					bLogOnce = false;
				}
#endif

#if PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING
				State |= D3D12_RESOURCE_STATE_SHADING_RATE_SOURCE;
#endif
			}

			// Should have at least one valid state
			check(State != D3D12_RESOURCE_STATE_COMMON);

			return State;
		}
	}
	}
}

D3D12_RESOURCE_STATES FD3D12LegacyBarriersForAdapterImpl::GetInitialState(
	ED3D12Access InD3D12Access,
	const FD3D12ResourceDesc& InDesc)
{
	// This makes the assumption that all resources begin life on the gfx pipe
	return GetD3D12ResourceState(InD3D12Access, ED3D12QueueType::Direct, InDesc, nullptr);
}

void FD3D12LegacyBarriersForAdapterImpl::ConfigureDevice(
	ID3D12Device* Device,
	bool InWithD3DDebug)
{
	FD3D12DynamicRHI::SetFormatAliasedTexturesMustBeCreatedUsingCommonLayout(true);
	GRHIGlobals.NeedsTransientDiscardStateTracking = true;
	GRHIGlobals.NeedsTransientDiscardOnGraphicsWorkaround = true;
}

uint64 FD3D12LegacyBarriersForAdapterImpl::GetTransitionDataSizeBytes()
{
	return sizeof(FD3D12LegacyBarriersTransitionData);
}

uint64 FD3D12LegacyBarriersForAdapterImpl::GetTransitionDataAlignmentBytes()
{
	return alignof(FD3D12LegacyBarriersTransitionData);
}

void FD3D12LegacyBarriersForAdapterImpl::CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo)
{
	// Construct the data in-place on the transition instance
	FD3D12LegacyBarriersTransitionData* Data = 
		new (Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>())
			FD3D12LegacyBarriersTransitionData;

	Data->SrcPipelines = CreateInfo.SrcPipelines;
	Data->DstPipelines = CreateInfo.DstPipelines;
	Data->CreateFlags = CreateInfo.Flags;

	const bool bCrossPipeline = (CreateInfo.SrcPipelines != CreateInfo.DstPipelines) && (!EnumHasAnyFlags(Data->CreateFlags, ERHITransitionCreateFlags::NoFence));
	const bool bAsyncToAllPipelines = ((CreateInfo.SrcPipelines == ERHIPipeline::AsyncCompute) && (CreateInfo.DstPipelines == ERHIPipeline::All));

	Data->bCrossPipeline = bCrossPipeline;

	// In DX12 we cannot perform resource barrier with graphics state on the AsyncCompute pipe
	// This check is here to be able to force a crosspipe transition coming from AsyncCompute with graphics states to be split and processed in the both the Async and Graphics pipe
	// This case can be removed when using EB on DX12 
	if (bAsyncToAllPipelines)
	{
		for (const FRHITransitionInfo& TransitionInfo : CreateInfo.TransitionInfos)
		{
			if (EnumHasAnyFlags(TransitionInfo.AccessAfter, ERHIAccess::GraphicsReadableMask))
			{
				Data->bAsyncToAllPipelines = true;
				Data->bCrossPipeline = false;
				break;
			}
		}
	}

	if ((Data->bCrossPipeline) || (Data->bAsyncToAllPipelines))
	{
		// Create one sync point per device, per source pipe
		for (uint32 Index : FRHIGPUMask::All())
		{
			TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints.Emplace_GetRef();
			for (ERHIPipeline Pipeline : MakeFlagsRange(CreateInfo.SrcPipelines))
			{
				DeviceSyncPoints[Pipeline] = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUOnly, UE_D3D12_STATIC_FNAME("Transition"));
			}
		}
	}

	Data->TransitionInfos = CreateInfo.TransitionInfos;
	Data->AliasingInfos = CreateInfo.AliasingInfos;

	uint32 AliasingOverlapCount = 0;

	for (const FRHITransientAliasingInfo& AliasingInfo : Data->AliasingInfos)
	{
		AliasingOverlapCount += AliasingInfo.Overlaps.Num();
	}

	Data->AliasingOverlaps.Reserve(AliasingOverlapCount);

	for (FRHITransientAliasingInfo& AliasingInfo : Data->AliasingInfos)
	{
		const int32 OverlapCount = AliasingInfo.Overlaps.Num();

		if (OverlapCount > 0)
		{
			const int32 OverlapOffset = Data->AliasingOverlaps.Num();
			Data->AliasingOverlaps.Append(AliasingInfo.Overlaps.GetData(), OverlapCount);
			AliasingInfo.Overlaps = MakeArrayView(&Data->AliasingOverlaps[OverlapOffset], OverlapCount);
		}
	}
}

void FD3D12LegacyBarriersForAdapterImpl::ReleaseTransition(
		FRHITransition* Transition)
{
	// Destruct the transition data
	Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>()->~FD3D12LegacyBarriersTransitionData();
}

HRESULT FD3D12LegacyBarriersForAdapterImpl::CreateCommittedResource(
	FD3D12Adapter& Adapter,
	const D3D12_HEAP_PROPERTIES& InHeapProps,
	D3D12_HEAP_FLAGS InHeapFlags,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource)
{
	const D3D12_RESOURCE_STATES InitialState = GetInitialState(InInitialD3D12Access, InDesc);

	// @TODO - This Intel path won't work for alias formats
#if INTEL_EXTENSIONS
	if (InDesc.bRequires64BitAtomicSupport && IsRHIDeviceIntel() && GDX12INTCAtomicUInt64Emulation)
	{
		D3D12_RESOURCE_DESC LocalDesc = InDesc;
		INTC_D3D12_RESOURCE_DESC_0001 IntelLocalDesc{};
		IntelLocalDesc.pD3D12Desc = &LocalDesc;
		IntelLocalDesc.EmulatedTyped64bitAtomics = true;

		return 
			INTC_D3D12_CreateCommittedResource(
				FD3D12DynamicRHI::GetD3DRHI()->GetIntelExtensionContext(),
				&InHeapProps,
				InHeapFlags,
				&IntelLocalDesc,
				InitialState,
				InClearValue,
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
	else
#endif
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	if (InDesc.SupportsUncompressedUAV())
	{
		// Convert the desc to the version required by CreateCommittedResource3
		const CD3DX12_RESOURCE_DESC1 LocalDesc1(InDesc);

		// Common layout is the required starting state for any "legacy" transitions
		const D3D12_BARRIER_LAYOUT InitialLayout = D3D12_BARRIER_LAYOUT_COMMON;
		checkf(InitialState == D3D12_RESOURCE_STATE_COMMON, TEXT("RESOURCE_STATE_COMMON is required for castable resources (Given: %d)"), InitialState);

		ID3D12ProtectedResourceSession* ProtectedSession = nullptr;

		const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();

		return 
			Adapter.GetD3DDevice12()->CreateCommittedResource3(
				&InHeapProps,
				InHeapFlags,
				&LocalDesc1,
				InitialLayout,
				InClearValue,
				ProtectedSession,
				CastableFormats.Num(),
				CastableFormats.GetData(),
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
	else
#endif
	{
		
		return 
			Adapter.GetD3DDevice()->CreateCommittedResource(
				&InHeapProps,
				InHeapFlags,
				&InDesc,
				InitialState,
				InClearValue,
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
}

HRESULT FD3D12LegacyBarriersForAdapterImpl::CreateReservedResource(
	FD3D12Adapter& Adapter,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource)
{
	const D3D12_RESOURCE_STATES InitialState = GetInitialState(InInitialD3D12Access, InDesc);

#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	if (InDesc.SupportsUncompressedUAV())
	{
		checkf(InInitialD3D12Access == ED3D12Access::Common, TEXT("RESOURCE_STATE_COMMON is required for castable resources (Given: %d)"), EnumToUnderlyingType(InInitialD3D12Access));

		// Common layout is the require starting state for any "legacy" transitions
		const D3D12_BARRIER_LAYOUT InitialLayout = D3D12_BARRIER_LAYOUT_COMMON;

		ID3D12ProtectedResourceSession* ProtectedSession = nullptr;
		const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();

		return
			Adapter.GetD3DDevice12()->CreateReservedResource2(
				&InDesc,
				InitialLayout,
				InClearValue,
				ProtectedSession,
				CastableFormats.Num(),
				CastableFormats.GetData(),
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
	else
#endif
	{
		return
			Adapter.GetD3DDevice()->CreateReservedResource(
				&InDesc,
				InitialState,
				InClearValue,
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
}

HRESULT FD3D12LegacyBarriersForAdapterImpl::CreatePlacedResource(
	FD3D12Adapter& Adapter,
	ID3D12Heap* Heap,
	uint64 InHeapOffset,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource)
{
	const D3D12_RESOURCE_STATES InitialState = GetInitialState(InInitialD3D12Access, InDesc);

	// @TODO - This Intel path won't work for alias formats
#if INTEL_EXTENSIONS
	if (InDesc.bRequires64BitAtomicSupport && IsRHIDeviceIntel() && GDX12INTCAtomicUInt64Emulation)
	{
		FD3D12ResourceDesc LocalDesc = InDesc;
		INTC_D3D12_RESOURCE_DESC_0001 IntelLocalDesc{};
		IntelLocalDesc.pD3D12Desc = &LocalDesc;
		IntelLocalDesc.EmulatedTyped64bitAtomics = true;

		return
			INTC_D3D12_CreatePlacedResource(
				FD3D12DynamicRHI::GetD3DRHI()->GetIntelExtensionContext(),
				Heap,
				InHeapOffset,
				&IntelLocalDesc,
				InitialState,
				InClearValue, IID_PPV_ARGS(OutResource.GetInitReference()));
	}
	else
#endif
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	if (InDesc.SupportsUncompressedUAV())
	{
		checkf(InitialState == D3D12_RESOURCE_STATE_COMMON, TEXT("RESOURCE_STATE_COMMON is required for castable resources (Given: %d)"), InitialState);

		// Convert the desc to the version required by CreatePlacedResource2
		const CD3DX12_RESOURCE_DESC1 LocalDesc1(InDesc);

		// Common layout is the required starting state for any "legacy" transitions
		const D3D12_BARRIER_LAYOUT InitialLayout = D3D12_BARRIER_LAYOUT_COMMON;
		
		const TArray<DXGI_FORMAT, TInlineAllocator<4>> CastableFormats = InDesc.GetCastableFormats();

		return
			Adapter.GetD3DDevice10()->CreatePlacedResource2(
				Heap,
				InHeapOffset,
				&LocalDesc1,
				InitialLayout,
				InClearValue,
				CastableFormats.Num(),
				CastableFormats.GetData(),
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
	else
#endif
	{
		return
			Adapter.GetD3DDevice()->CreatePlacedResource(
				Heap,
				InHeapOffset,
				&InDesc,
				InitialState,
				InClearValue,
				IID_PPV_ARGS(OutResource.GetInitReference()));
	}
}

FD3D12LegacyBarriersForAdapter::~FD3D12LegacyBarriersForAdapter()
{}

void FD3D12LegacyBarriersForAdapter::ConfigureDevice(
	ID3D12Device* Device,
	bool InWithD3DDebug) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::ConfigureDevice(
			Device,
			InWithD3DDebug);
}

uint64 FD3D12LegacyBarriersForAdapter::GetTransitionDataSizeBytes() const
{
	return FD3D12LegacyBarriersForAdapterImpl::GetTransitionDataSizeBytes();
}

uint64 FD3D12LegacyBarriersForAdapter::GetTransitionDataAlignmentBytes() const
{
	return FD3D12LegacyBarriersForAdapterImpl::GetTransitionDataAlignmentBytes();
}

void FD3D12LegacyBarriersForAdapter::CreateTransition(
	FRHITransition* Transition,
	const FRHITransitionCreateInfo& CreateInfo) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::CreateTransition(
			Transition,
			CreateInfo);
}

void FD3D12LegacyBarriersForAdapter::ReleaseTransition(
	FRHITransition* Transition) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::ReleaseTransition(
			Transition);
}

HRESULT FD3D12LegacyBarriersForAdapter::CreateCommittedResource(
	FD3D12Adapter& Adapter,
	const D3D12_HEAP_PROPERTIES& InHeapProps,
	D3D12_HEAP_FLAGS InHeapFlags,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::CreateCommittedResource(
			Adapter,
			InHeapProps,
			InHeapFlags,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

HRESULT FD3D12LegacyBarriersForAdapter::CreateReservedResource(
	FD3D12Adapter& Adapter,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::CreateReservedResource(
			Adapter,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

HRESULT FD3D12LegacyBarriersForAdapter::CreatePlacedResource(
	FD3D12Adapter& Adapter,
	ID3D12Heap* Heap,
	uint64 InHeapOffset,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	TRefCountPtr<ID3D12Resource>& OutResource) const
{
	return
		FD3D12LegacyBarriersForAdapterImpl::CreatePlacedResource(
			Adapter,
			Heap,
			InHeapOffset,
			InDesc,
			InInitialD3D12Access,
			InClearValue,
			OutResource);
}

const TCHAR* FD3D12LegacyBarriersForAdapter::GetImplementationName() const
{
	return TEXT("D3D12LegacyBarriers");
}


/////////////////////////////////////////////////////////////////////
//	FD3D12 Legacy Barrier Batcher
/////////////////////////////////////////////////////////////////////

// Use the top bit of the flags enum to mark transitions as "idle" time (used to remove the swapchain wait time for back buffers).
static const D3D12_RESOURCE_BARRIER_FLAGS BarrierFlag_CountAsIdleTime = D3D12_RESOURCE_BARRIER_FLAGS(1ull << ((sizeof(D3D12_RESOURCE_BARRIER_FLAGS) * 8) - 1));

class FD3D12LegacyBarriersBatcher
{
	struct FD3D12ResourceBarrier : public D3D12_RESOURCE_BARRIER
	{
		FD3D12ResourceBarrier() = default;
		FD3D12ResourceBarrier(D3D12_RESOURCE_BARRIER&& Barrier) : D3D12_RESOURCE_BARRIER(MoveTemp(Barrier)) {}

		bool HasIdleFlag() const { return !!(Flags & BarrierFlag_CountAsIdleTime); }
		void ClearIdleFlag() { Flags &= ~BarrierFlag_CountAsIdleTime; }
	};
	static_assert(sizeof(FD3D12ResourceBarrier) == sizeof(D3D12_RESOURCE_BARRIER), "FD3D12ResourceBarrier is a wrapper to add helper functions. Do not add members.");

	static constexpr D3D12_RESOURCE_STATES BackBufferBarrierWriteTransitionTargets = 
		D3D12_RESOURCE_STATES(
			uint32(D3D12_RESOURCE_STATE_RENDER_TARGET) |
			uint32(D3D12_RESOURCE_STATE_UNORDERED_ACCESS) |
			uint32(D3D12_RESOURCE_STATE_STREAM_OUT) |
			uint32(D3D12_RESOURCE_STATE_COPY_DEST) |
			uint32(D3D12_RESOURCE_STATE_RESOLVE_DEST));

public:
	// Add a UAV barrier to the batch. Ignoring the actual resource for now.
	void AddUAV(
		FD3D12ContextCommon& Context);

	// Add a transition resource barrier to the batch. Returns the number of barriers added, which may be negative if an existing barrier was cancelled.
	int32 AddTransition(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		D3D12_RESOURCE_STATES Before,
		D3D12_RESOURCE_STATES After,
		uint32 Subresource);

	void AddAliasingBarrier(
		FD3D12ContextCommon& Context,
		ID3D12Resource* InResourceBefore,
		ID3D12Resource* InResourceAfter);

	void FlushIntoCommandList(
		FD3D12CommandList& CommandList,
		FD3D12QueryAllocator& TimestampAllocator);

	int32 Num() const { return Barriers.Num(); }

private:
	TArray<FD3D12ResourceBarrier> Barriers;
};

// Add a UAV barrier to the batch. Ignoring the actual resource for now.
void FD3D12LegacyBarriersBatcher::AddUAV(
	FD3D12ContextCommon& Context)
{
	FD3D12ResourceBarrier& Barrier = Barriers.Emplace_GetRef();
	Barrier.Type = D3D12_RESOURCE_BARRIER_TYPE_UAV;
	Barrier.Flags = D3D12_RESOURCE_BARRIER_FLAG_NONE;
	Barrier.UAV.pResource = nullptr;	// Ignore the resource ptr for now. HW doesn't do anything with it.

#if D3D12RHI_BARRIER_LOGGING
	if (GD3D12LogBarriers & D3D12LogBarriersWhenBatched)
	{
		LogResourceBarriers(
			MakeConstArrayView<D3D12_RESOURCE_BARRIER>({Barrier}),
			Context.GetCommandList().BaseCommandList().Get(),
			Context.GetCommandList().QueueType);
	}
#endif

	if (!GD3D12BatchResourceBarriers)
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}
}

// Add a transition resource barrier to the batch. Returns the number of barriers added, which may be negative if an existing barrier was cancelled.
int32 FD3D12LegacyBarriersBatcher::AddTransition(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* pResource,
	D3D12_RESOURCE_STATES Before,
	D3D12_RESOURCE_STATES After,
	uint32 Subresource)
{
	check(Before != After);

	if (Barriers.Num())
	{
		// Check if the new transition is for the same (sub)resource as the prior barrier, which also must be a transition.
		FD3D12ResourceBarrier& Last = Barriers.Last();
		if (Last.Type == D3D12_RESOURCE_BARRIER_TYPE_TRANSITION
			&& pResource->GetResource() == Last.Transition.pResource
			&& Subresource == Last.Transition.Subresource
			// This should always be true (assuming the above are also true) but we don't want to obscure bugs by assuming it
			&& Before == Last.Transition.StateAfter)
		{
			// Check if we are simply reverting the last transition. In that case, we can just remove both transitions.
			// This happens fairly frequently due to resource pooling since different RHI buffers can point to the same underlying D3D buffer.
			// Instead of ping-ponging that underlying resource between COPY_DEST and GENERIC_READ, several copies can happen without a ResourceBarrier() in between.
			// Doing this check also eliminates a D3D debug layer warning about multiple transitions of the same subresource.
			if (After == Last.Transition.StateBefore)
			{
				Barriers.RemoveAt(Barriers.Num() - 1);
				return -1;
			}
			// No work was done between this and the last transition so we can replace the existing transition's after state with
			// the new after state, effectively merging the two barriers. This commonly happens during copies since we "push" and 
			// then "pop" the copy states and often there's no additional work done between that pop (e.g. transition from the copy
			// state back to whatever state the resource was in before) and the next transition.
			else
			{
				Last.Transition.StateAfter = After;
				return 0;
			}
		}
	}

	check(IsValidD3D12ResourceState(Before) && IsValidD3D12ResourceState(After));

	FD3D12ResourceBarrier& Barrier = Barriers.Emplace_GetRef(CD3DX12_RESOURCE_BARRIER::Transition(pResource->GetResource(), Before, After, Subresource));
	if (pResource->IsBackBuffer() && EnumHasAnyFlags(After, BackBufferBarrierWriteTransitionTargets))
	{
		Barrier.Flags |= BarrierFlag_CountAsIdleTime;
	}

#if D3D12RHI_BARRIER_LOGGING
	if (GD3D12LogBarriers & D3D12LogBarriersWhenBatched)
	{
		LogResourceBarriers(
			MakeConstArrayView<D3D12_RESOURCE_BARRIER>({Barrier}),
			Context.GetCommandList().BaseCommandList().Get(),
			Context.GetCommandList().QueueType);
	}
#endif

	if (!GD3D12BatchResourceBarriers)
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}

	return 1;
}

void FD3D12LegacyBarriersBatcher::AddAliasingBarrier(
	FD3D12ContextCommon& Context,
	ID3D12Resource* InResourceBefore,
	ID3D12Resource* InResourceAfter)
{
	FD3D12ResourceBarrier& Barrier = Barriers.Emplace_GetRef();
	Barrier.Type = D3D12_RESOURCE_BARRIER_TYPE_ALIASING;
	Barrier.Flags = D3D12_RESOURCE_BARRIER_FLAG_NONE;
	Barrier.Aliasing.pResourceBefore = InResourceBefore;
	Barrier.Aliasing.pResourceAfter = InResourceAfter;

#if D3D12RHI_BARRIER_LOGGING
	if (GD3D12LogBarriers & D3D12LogBarriersWhenBatched)
	{
		LogResourceBarriers(
			MakeConstArrayView<D3D12_RESOURCE_BARRIER>({Barrier}),
			Context.GetCommandList().BaseCommandList().Get(),
			Context.GetCommandList().QueueType);
	}
#endif

	if (!GD3D12BatchResourceBarriers)
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}
}

void FD3D12LegacyBarriersBatcher::FlushIntoCommandList(
	FD3D12CommandList& CommandList,
	FD3D12QueryAllocator& TimestampAllocator)
{
	auto InsertTimestamp = [&](bool bBegin)
	{
		if (bBegin)
		{
			auto& Event = CommandList.EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
			CommandList.EndQuery(TimestampAllocator.Allocate(ED3D12QueryType::ProfilerTimestampBOP, &Event.GPUTimestampBOP));
		}
		else
		{
			// CPUTimestamp is filled in at submission time in FlushProfilerEvents
			auto& Event = CommandList.EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0);
			CommandList.EndQuery(TimestampAllocator.Allocate(ED3D12QueryType::ProfilerTimestampTOP, &Event.GPUTimestampTOP));
		}
	};

	for (int32 BatchStart = 0, BatchEnd = 0; BatchStart < Barriers.Num(); BatchStart = BatchEnd)
	{
		// Gather a range of barriers that all have the same idle flag
		bool const bIdle = Barriers[BatchEnd].HasIdleFlag();

		while (BatchEnd < Barriers.Num() 
			&& bIdle == Barriers[BatchEnd].HasIdleFlag())
		{
			// Clear the idle flag since its not a valid D3D bit.
			Barriers[BatchEnd++].ClearIdleFlag();
		}

		// Insert an idle begin/end timestamp around the barrier batch if required.
		if (bIdle)
		{
			InsertTimestamp(true);
		}

#if D3D12RHI_BARRIER_LOGGING
		if (GD3D12LogBarriers & D3D12LogBarriersWhenFlushed)
		{
			TArrayView<FD3D12ResourceBarrier> SubsetBarriers(Barriers.GetData() + BatchStart, BatchEnd - BatchStart);
			TConstArrayView<D3D12_RESOURCE_BARRIER> ConstArrayView(reinterpret_cast<const D3D12_RESOURCE_BARRIER*>(SubsetBarriers.GetData()), SubsetBarriers.Num());
			LogResourceBarriers(ConstArrayView, CommandList.BaseCommandList().Get(), CommandList.QueueType);
		}
#endif
		CommandList.GraphicsCommandList()->ResourceBarrier(BatchEnd - BatchStart, &Barriers[BatchStart]);
#if DEBUG_RESOURCE_STATES
		// Keep track of all the resource barriers that have been submitted to the current command list.
		for(int i = BatchStart; i < BatchEnd - BatchStart - 1; i++)
		{
			CommandList.State.ResourceBarriers.Emplace(Barriers[i]);
		}
#endif // #if DEBUG_RESOURCE_STATES

		if (bIdle)
		{
			InsertTimestamp(false);
		}
	}

	Barriers.Reset();
}

FD3D12LegacyBarriersForContext::FD3D12LegacyBarriersForContext()
: ID3D12BarriersForContext()
, Batcher(MakeUnique<FD3D12LegacyBarriersBatcher>())
{}

FD3D12LegacyBarriersForContext::~FD3D12LegacyBarriersForContext()
{}


template <typename FunctionType>
void EnumerateSubresources(FD3D12Resource* Resource, const FRHITransitionInfo& Info, FD3D12Texture* Texture, FunctionType Function)
{
	uint32 FirstMipSlice = 0;
	uint32 FirstArraySlice = 0;
	uint32 FirstPlaneSlice = 0;

	uint32 MipCount = Resource->GetMipLevels();
	uint32 ArraySize = Resource->GetArraySize();
	uint32 PlaneCount = Resource->GetPlaneCount();

	uint32 IterationMipCount = MipCount;
	uint32 IterationArraySize = ArraySize;
	uint32 IterationPlaneCount = PlaneCount;

	if (!Info.IsAllMips())
	{
		FirstMipSlice = Info.MipIndex;
		IterationMipCount = 1;
	}

	if (!Info.IsAllArraySlices())
	{
		FirstArraySlice = Info.ArraySlice;
		IterationArraySize = 1;
	}

	if (!Info.IsAllPlaneSlices())
	{
		FirstPlaneSlice = Info.PlaneSlice;
		IterationPlaneCount = 1;
	}

	for (uint32 PlaneSlice = FirstPlaneSlice; PlaneSlice < FirstPlaneSlice + IterationPlaneCount; ++PlaneSlice)
	{
		for (uint32 ArraySlice = FirstArraySlice; ArraySlice < FirstArraySlice + IterationArraySize; ++ArraySlice)
		{
			for (uint32 MipSlice = FirstMipSlice; MipSlice < FirstMipSlice + IterationMipCount; ++MipSlice)
			{
				const uint32 Subresource = D3D12CalcSubresource(MipSlice, ArraySlice, PlaneSlice, MipCount, ArraySize);
				const FD3D12RenderTargetView* RTV = nullptr;
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
				if (Texture)
				{
					RTV = Texture->GetRenderTargetView(MipSlice, ArraySlice);
				}
#endif // PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
				Function(Subresource, RTV);
			}
		}
	}
}

static
std::tuple<FD3D12Resource*,FD3D12Texture*> GetResourceAndTexture(FD3D12CommandContext& Context, const FRHITransitionInfo& Info)
{
	switch (Info.Type)
	{
	case FRHITransitionInfo::EType::UAV:
	{
		FD3D12UnorderedAccessView* UAV = Context.RetrieveObject<FD3D12UnorderedAccessView_RHI>(Info.UAV);
		check(UAV);
		if (UAV)
		{
			return { UAV->GetResource(), nullptr };
		}
		return { nullptr, nullptr };
	}
	case FRHITransitionInfo::EType::Buffer:
	{
		// Resource may be null if this is a multi-GPU resource not present on the current GPU
		FD3D12Buffer* Buffer = Context.RetrieveObject<FD3D12Buffer>(Info.Buffer);

		if (Buffer)
		{
			return { Buffer->GetResource(), nullptr };
		}
		return { nullptr, nullptr };
	}
	case FRHITransitionInfo::EType::Texture:
	{
		// Resource may be null if this is a multi-GPU resource not present on the current GPU
		FD3D12Texture* Texture = Context.RetrieveTexture(Info.Texture);

		if (Texture)
		{
			return { Texture->GetResource(), Texture };
		}
		return { nullptr, nullptr };
	}
	case FRHITransitionInfo::EType::BVH:
	{
		// Nothing special required for BVH transitions - handled inside d3d12 raytracing directly via UAV barriers and don't need explicit state changes
		return { nullptr, nullptr };
	}
	default:
		checkNoEntry();
		return { nullptr, nullptr };
		break;
	}
}

template <typename FunctionType>
void ProcessResource(FD3D12CommandContext& Context, const FRHITransitionInfo& Info, FunctionType Function)
{
	auto [Resource, Texture] = GetResourceAndTexture(Context, Info);
	FD3D12Texture* DiscardTextureOut = nullptr;

	if ((Info.Type == FRHITransitionInfo::EType::Texture) && Texture)
	{
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		if (Texture->GetRequiresTypelessResourceDiscardWorkaround())
		{
			DiscardTextureOut = Texture;
		}
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	}

	if (Resource)
	{
		Function(Info, Resource, Texture, DiscardTextureOut);
	}
}

// Pipe changes which are not ending with graphics or targeting all pipelines are handle during begin
static bool ProcessTransitionDuringBegin(
	const FD3D12LegacyBarriersTransitionData* Data)
{
	// Source pipelines aren't on all pipelines
	const bool bSrcPipelinesNotAll = !EnumHasAllFlags(Data->SrcPipelines, ERHIPipeline::All);

	// Source and destination pipelines are different
	const bool bSrcDstPipelinesDiffer = Data->SrcPipelines != Data->DstPipelines;

	// Destination pipeline is not only graphics
	const bool bDstPipelineNotGraphics = Data->DstPipelines != ERHIPipeline::Graphics;

	// Destination pipelines include all pipelines
	const bool bDstPipelinesIncludeAll = EnumHasAllFlags(Data->DstPipelines, ERHIPipeline::All);

	return bSrcPipelinesNotAll && ( (bSrcDstPipelinesDiffer && bDstPipelineNotGraphics) || bDstPipelinesIncludeAll );
}

static bool ShouldProcessTransition(
	const FD3D12LegacyBarriersTransitionData* Data,
	bool bIsBeginTransition,
	ERHIPipeline ExecutingPipeline)
{
	// Special DX12 case where crosspipe transitions from AsyncCompute with graphics state can only be processed on the ERHIPipeline::Graphics pipe
	if (Data->bAsyncToAllPipelines)
	{
		if ((bIsBeginTransition == false) && (ExecutingPipeline == ERHIPipeline::Graphics))
		{
			return true;
		}
		
		if ((bIsBeginTransition) && (ExecutingPipeline == ERHIPipeline::AsyncCompute))
		{
			return true;
		}

		return false;
	}

	return (bIsBeginTransition ? ProcessTransitionDuringBegin(Data) : !ProcessTransitionDuringBegin(Data));
}

struct FD3D12LegacyBarriersForContext::FD3D12DiscardResource
{
	FD3D12DiscardResource(
		FD3D12Resource* InResource,
		EResourceTransitionFlags InFlags,
		uint32 InSubresource = D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES,
		const FD3D12Texture* InTexture = nullptr,
		const FD3D12RenderTargetView* InRTV = nullptr)
		: Resource(InResource)
		, Flags(InFlags)
		, Subresource(InSubresource)
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		, Texture(InTexture)
		, RTV(InRTV)
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	{}

	FD3D12Resource* Resource;
	EResourceTransitionFlags Flags;
	uint32 Subresource;
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	const FD3D12Texture* Texture = nullptr;
	const FD3D12RenderTargetView* RTV = nullptr;
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
};

void FD3D12LegacyBarriersForContext::HandleReservedResourceCommits(
	FD3D12CommandContext& Context,
	const FD3D12LegacyBarriersTransitionData* TransitionData)
{
	for (const FRHITransitionInfo& Info : TransitionData->TransitionInfos)
	{
		if (const FRHICommitResourceInfo* CommitInfo = Info.CommitInfo.GetPtrOrNull())
		{
			if (Info.Type == FRHITransitionInfo::EType::Buffer)
			{
				FD3D12Buffer* Buffer = Context.RetrieveObject<FD3D12Buffer>(Info.Buffer);
				Context.SetReservedBufferCommitSize(Buffer, CommitInfo->SizeInBytes);
			}
			else
			{
				checkNoEntry();
			}
		}
	}
}

static bool IsImpossibleAsyncDiscardTransition(
	ERHIPipeline Pipeline,
	FRHITexture* Texture)
{
	return 
		Pipeline == ERHIPipeline::AsyncCompute 
		&& Texture 
		&& EnumHasAnyFlags(Texture->GetDesc().Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::DepthStencilTargetable);
}

void FD3D12LegacyBarriersForContext::HandleResourceDiscardTransitions(
	FD3D12CommandContext& Context,
	const FD3D12LegacyBarriersTransitionData* TransitionData,
	TArray<FD3D12DiscardResource>& ResourcesToDiscard)
{
	for (const FRHITransitionInfo& Info : TransitionData->TransitionInfos)
	{
		const UE::RHICore::FResourceState ResourceState(Context, TransitionData->SrcPipelines, TransitionData->DstPipelines, Info);

		if (!EnumHasAnyFlags(ResourceState.AccessBefore, ERHIAccess::Discard))
		{
			continue;
		}

		ProcessResource(Context, Info, [&](const FRHITransitionInfo& Info, FD3D12Resource* Resource, FD3D12Texture* Texture, FD3D12Texture* DiscardTexture)
		{
			const ED3D12QueueType QueueType = Context.GetCommandList().QueueType;
			D3D12_RESOURCE_STATES StateAfter  = GetDiscardedResourceState(Resource->GetDesc(), QueueType);
			D3D12_RESOURCE_STATES StateBefore = StateAfter;

			if (ResourceState.AccessBefore != ERHIAccess::Discard)
			{
				StateBefore = GetD3D12ResourceState(ConvertToD3D12Access(ResourceState.AccessBefore & ~ERHIAccess::Discard), QueueType, Resource->GetDesc(), Texture);
			}

			const bool bTransition = StateBefore != StateAfter;

			if (bTransition)
			{
				// Transitions here should only occur on the Direct queue and when the prior Discard operation failed due to being on async compute.
				ensure(IsImpossibleAsyncDiscardTransition(ResourceState.SrcPipelines, Texture) && QueueType == ED3D12QueueType::Direct);
			}

			if (Info.IsWholeResource() || Resource->GetSubresourceCount() == 1)
			{
				if (bTransition)
				{
					TransitionResource(Context, Resource, StateBefore, StateAfter, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
				}
				else
				{
					Context.UpdateResidency(Resource);
				}

				const FD3D12RenderTargetView* RTV = nullptr;
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
				if (DiscardTexture)
				{
					RTV = DiscardTexture->GetRenderTargetView(0, -1);
				}
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
				ResourcesToDiscard.Emplace(Resource, Info.Flags, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES, Texture, RTV);
			}
			else
			{
				EnumerateSubresources(Resource, Info, DiscardTexture, [&](uint32 Subresource, const FD3D12RenderTargetView* RTV)
				{
					if (bTransition)
					{
						TransitionResource(Context, Resource, StateBefore, StateAfter, Subresource);
					}
					else
					{
						Context.UpdateResidency(Resource);
					}

					ResourcesToDiscard.Emplace(Resource, Info.Flags, Subresource, DiscardTexture, RTV);
				});
			}
		});
	}
}

void FD3D12LegacyBarriersForContext::HandleDiscardResources(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> Transitions,
	bool bIsBeginTransition)
{
	TArray<FD3D12DiscardResource> ResourcesToDiscard;

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data = 
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		if (ProcessTransitionDuringBegin(Data) == bIsBeginTransition)
		{
			HandleResourceDiscardTransitions(Context, Data, ResourcesToDiscard);
		}
	}

	if (!GD3D12AllowDiscardResources)
	{
		return;
	}

	if (!ResourcesToDiscard.IsEmpty())
	{
		FlushIntoCommandList(Context.GetCommandList(), Context.GetTimestampQueries());
	}

	for (const FD3D12DiscardResource& DiscardResource : ResourcesToDiscard)
	{
#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		if (DiscardResource.Texture && DiscardResource.RTV && DiscardResource.Texture->GetRequiresTypelessResourceDiscardWorkaround())
		{
			FLinearColor ClearColor = DiscardResource.Texture->GetClearColor();
			Context.GetCommandList().GraphicsCommandList()->ClearRenderTargetView(DiscardResource.RTV->GetOfflineCpuHandle(), reinterpret_cast<float*>(&ClearColor), 0, nullptr);
			Context.UpdateResidency(DiscardResource.RTV->GetResource());
		}
		else
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		{
			if (GD3D12DisableDiscardOfDepthResources && DiscardResource.Resource->IsDepthStencilResource())
			{
				continue;
			}

			if (DiscardResource.Subresource == D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES)
			{
				Context.GetCommandList().GraphicsCommandList()->DiscardResource(DiscardResource.Resource->GetResource(), nullptr);
			}
			else
			{
				D3D12_DISCARD_REGION Region;
				Region.NumRects = 0;
				Region.pRects = nullptr;
				Region.FirstSubresource = DiscardResource.Subresource;
				Region.NumSubresources = 1;

				Context.GetCommandList().GraphicsCommandList()->DiscardResource(DiscardResource.Resource->GetResource(), &Region);
			}
		}
	}
}

void FD3D12LegacyBarriersForContext::HandleTransientAliasing(
	FD3D12CommandContext& Context,
	const FD3D12LegacyBarriersTransitionData* TransitionData)
{
	for (const FRHITransientAliasingInfo& Info : TransitionData->AliasingInfos)
	{
		FD3D12BaseShaderResource* BaseShaderResource = nullptr;
		switch (Info.Type)
		{
		case FRHITransientAliasingInfo::EType::Buffer:
		{
			// Resource may be null if this is a multi-GPU resource not present on the current GPU
			FD3D12Buffer* Buffer = Context.RetrieveObject<FD3D12Buffer>(Info.Buffer);
			check(Buffer || GNumExplicitGPUsForRendering > 1);
			BaseShaderResource = Buffer;
			break;
		}
		case FRHITransientAliasingInfo::EType::Texture:
		{
			// Resource may be null if this is a multi-GPU resource not present on the current GPU
			FD3D12Texture* Texture = Context.RetrieveTexture(Info.Texture);
			check(Texture || GNumExplicitGPUsForRendering > 1);
			BaseShaderResource = Texture;
			break;
		}
		default:
			checkNoEntry();
			break;
		}

		// Resource may be null if this is a multi-GPU resource not present on the current GPU
		if (!BaseShaderResource)
		{
			continue;
		}

		FD3D12Resource* Resource = BaseShaderResource->ResourceLocation.GetResource();
		if (Info.Action == FRHITransientAliasingInfo::EAction::Acquire)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::AcquireTransient);
			Batcher->AddAliasingBarrier(Context, nullptr, Resource->GetResource());
		}
	}
}

void FD3D12LegacyBarriersForContext::HandleResourceTransitions(
	FD3D12CommandContext& Context,
	const FD3D12LegacyBarriersTransitionData* TransitionData,
	bool& bUAVBarrier)
{
	for (const FRHITransitionInfo& Info : TransitionData->TransitionInfos)
	{
		if (!Info.Resource)
		{
			continue;
		}

		UE::RHICore::FResourceState ResourceState(Context, TransitionData->SrcPipelines, TransitionData->DstPipelines, Info);

		bUAVBarrier |=
			EnumHasAnyFlags(ResourceState.AccessBefore, ERHIAccess::UAVMask) &&
			EnumHasAnyFlags(ResourceState.AccessAfter,  ERHIAccess::UAVMask);

		// Skip duplicate transitions. This happens most frequently with implicit ones from NeedsExtraTransitions.
		if (ResourceState.AccessBefore == ResourceState.AccessAfter)
		{
			continue;
		}

		const ED3D12QueueType QueueType = Context.GetCommandList().QueueType;

		// Edge case to handle graphics state transitions on the async compute pipe that needs to be removed with EB
		// Transitions to read-only Gfx bits get handled on the Gfx pipe
		// a UAV -> SRVMask on the AsyncPipe get split in two: UAV->SRVCompute on Async and SRVCompute->SRVMask on Gfx
		// On the Async pipe is going to be : UAV->SRVMask(that is automatically converted in UAV->SRVCompute)
		// On the Direct(Gfx) pipe instead needs to be SRVCompute->SRVMask therefore the check there to change the Before state only on the Direct pipe.
		if (TransitionData->bAsyncToAllPipelines && EnumHasAllFlags(ResourceState.AccessAfter, ERHIAccess::SRVMask) && (QueueType == ED3D12QueueType::Direct))
		{
			ResourceState.AccessBefore = ERHIAccess::SRVCompute;
		}

		// Process transitions which are forced during begin because those contain transition from Graphics to Compute and should
		// help remove forced patch up command lists for async compute to run on the graphics queue
		ProcessResource(Context, Info, [&](const FRHITransitionInfo& Info, FD3D12Resource* Resource, FD3D12Texture* Texture, FD3D12Texture* DiscardTexture)
		{
			if (!Resource->RequiresResourceStateTracking())
			{
				return;
			}

			if (ResourceState.AccessAfter == ERHIAccess::Discard && IsImpossibleAsyncDiscardTransition(ResourceState.DstPipelines, Texture))
			{
				return;
			}

			D3D12_RESOURCE_STATES StateBefore;

			if (EnumHasAnyFlags(ResourceState.AccessBefore, ERHIAccess::Discard))
			{
				StateBefore = GetDiscardedResourceState(Resource->GetDesc(), QueueType);
			}
			else
			{
				StateBefore = GetD3D12ResourceState(ConvertToD3D12Access(ResourceState.AccessBefore), QueueType, Resource->GetDesc(), Texture);
			}

			if (ResourceState.AccessBefore != ERHIAccess::Present)
			{
				check(StateBefore != D3D12_RESOURCE_STATE_COMMON);
			}

			D3D12_RESOURCE_STATES StateAfter = GetD3D12ResourceState(ConvertToD3D12Access(ResourceState.AccessAfter), QueueType, Resource->GetDesc(), Texture);

			// enqueue the correct transitions
			if (Info.IsWholeResource() || Resource->GetSubresourceCount() == 1)
			{
				TransitionResource(Context, Resource, StateBefore, StateAfter, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
			}
			else
			{
				// high level rendering is controlling transition ranges, at this level this is an index not a range
				check(Info.MipIndex != FRHISubresourceRange::kAllSubresources);
				check(Info.ArraySlice != FRHISubresourceRange::kAllSubresources);
				check(Info.PlaneSlice != FRHISubresourceRange::kAllSubresources);
				const uint32 Subresource = D3D12CalcSubresource(Info.MipIndex, Info.ArraySlice, Info.PlaneSlice, Resource->GetMipLevels(), Resource->GetArraySize());
				check(Subresource < Resource->GetSubresourceCount());
				TransitionResource(Context, Resource, StateBefore, StateAfter, Subresource);
			}
		});
	}
}

static bool IsTransitionNeeded(
	D3D12_RESOURCE_STATES Before,
	D3D12_RESOURCE_STATES After,
	const FD3D12Resource* InResource = nullptr)
{
	check(Before != D3D12_RESOURCE_STATE_CORRUPT && After != D3D12_RESOURCE_STATE_CORRUPT);
	check(Before != D3D12_RESOURCE_STATE_TBD && After != D3D12_RESOURCE_STATE_TBD);

	// COMMON is an oddball state that doesn't follow the RESOURE_STATE pattern of 
	// having exactly one bit set so we need to special case these
	if (After == D3D12_RESOURCE_STATE_COMMON)
	{
		// Before state should not have the common state otherwise it's invalid transition
		check(Before != D3D12_RESOURCE_STATE_COMMON);
		return true;
	}

	return Before != After;
}

void FD3D12LegacyBarriersForContext::TransitionResource(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* InResource,
	D3D12_RESOURCE_STATES InBeforeState,
	D3D12_RESOURCE_STATES InAfterState,
	uint32 InSubresourceIndex)
{
	check(InResource);
	//check(InResource->RequiresResourceStateTracking());
	check(!((InAfterState & (D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE | D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE)) && (InResource->GetDesc().Flags & D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE)));
	check(InBeforeState != D3D12_RESOURCE_STATE_TBD);
	check(InAfterState != D3D12_RESOURCE_STATE_TBD);

#ifdef PLATFORM_SUPPORTS_RESOURCE_COMPRESSION
	After |= Resource->GetCompressedState();
#endif

#if ENABLE_RHI_VALIDATION
	FString IncompatibilityReason;
	if (CheckResourceStateCompatibility(InAfterState, InResource->GetDesc().Flags, IncompatibilityReason) == false)
	{
		UE_LOGF(LogRHI, Error, "Incompatible Transition State for Resource %ls - %ls", *InResource->GetName().ToString(), *IncompatibilityReason);
	}
#endif

	Context.UpdateResidency(InResource);

#if D3D12_RHI_RAYTRACING
	// Special case for raytracing because the API doesn't allow expressing
	// read<->write state transitions for acceleration structures.
	// @TODO - This could be made better if we were to make the decision based on the ED3D12Access bits
	// which could discern if this is a transition from read<->read which we could actually skip
	if (InBeforeState == D3D12_RESOURCE_STATE_RAYTRACING_ACCELERATION_STRUCTURE
		&& InAfterState == D3D12_RESOURCE_STATE_RAYTRACING_ACCELERATION_STRUCTURE)
	{
		Batcher->AddUAV(Context);
	}
	else
#endif
	if (IsTransitionNeeded(InBeforeState, InAfterState, InResource))
	{
		Batcher->AddTransition(Context, InResource, InBeforeState, InAfterState, InSubresourceIndex);
	}
}

void FD3D12LegacyBarriersForContext::BeginTransitions(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> Transitions)
{
	const ERHIPipeline CurrentPipeline = Context.GetPipeline();
	const bool bIsBeginTransition = true;

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data = 
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		if (ShouldProcessTransition (Data, bIsBeginTransition, CurrentPipeline))
		{
			HandleTransientAliasing(Context, Data);
		}
	}

	HandleDiscardResources(Context, Transitions, bIsBeginTransition);

	bool bUAVBarrier = false;

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data = 
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		// Handle transition during BeginTransitions?
		if (ShouldProcessTransition(Data, bIsBeginTransition, CurrentPipeline))
		{
			HandleResourceTransitions(Context, Data, bUAVBarrier);
		}
	}

	if (bUAVBarrier)
	{
		Context.StateCache.FlushComputeShaderCache(true);
	}

	// Signal fences
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();
		if (Data->bCrossPipeline)
		{
			const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];

			if (DeviceSyncPoints[CurrentPipeline])
			{
				Context.SignalSyncPoint(DeviceSyncPoints[CurrentPipeline]);
			}
		}
	}
}

void FD3D12LegacyBarriersForContext::EndTransitions(
	FD3D12CommandContext& Context,
	TArrayView<const FRHITransition*> Transitions)
{
	const ERHIPipeline CurrentPipeline = Context.GetPipeline();
	const bool bIsBeginTransition = false;

	// Wait for fences
	{
		for (const FRHITransition* Transition : Transitions)
		{
			const FD3D12LegacyBarriersTransitionData* Data =
				Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

			if (Data->bAsyncToAllPipelines)
			{
				const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];
				if (CurrentPipeline == ERHIPipeline::Graphics)
				{
					Context.WaitSyncPoint(DeviceSyncPoints[ERHIPipeline::AsyncCompute]);
				}
			}
			else if (Data->bCrossPipeline)
			{
				const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];

				for (ERHIPipeline SrcPipeline : MakeFlagsRange(Data->SrcPipelines))
				{
					if (SrcPipeline != CurrentPipeline && DeviceSyncPoints[SrcPipeline])
					{
						Context.WaitSyncPoint(DeviceSyncPoints[SrcPipeline]);
					}
				}
			}
		}
	}

	// Update reserved resource memory mapping
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		HandleReservedResourceCommits(Context, Data);
	}

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		if (ShouldProcessTransition(Data, bIsBeginTransition, CurrentPipeline))
		{
			HandleTransientAliasing(Context, Data);
		}
	}

	HandleDiscardResources(Context, Transitions, false /** bIsBeginTransitions */);

	bool bUAVBarrier = false;

	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		// Handle transition during EndTransitions?
		if (ShouldProcessTransition(Data, bIsBeginTransition, CurrentPipeline))
		{
			HandleResourceTransitions(Context, Data, bUAVBarrier);
		}
	}

	if (bUAVBarrier)
	{
		Context.StateCache.FlushComputeShaderCache(true);
	}

	// Signal fences
	for (const FRHITransition* Transition : Transitions)
	{
		const FD3D12LegacyBarriersTransitionData* Data =
			Transition->GetPrivateData<FD3D12LegacyBarriersTransitionData>();

		if ((Data->bAsyncToAllPipelines) && (CurrentPipeline == ERHIPipeline::AsyncCompute))
		{
			const TRHIPipelineArray<FD3D12SyncPointRef>& DeviceSyncPoints = Data->SyncPoints[Context.GetGPUIndex()];

			if (DeviceSyncPoints[CurrentPipeline])
			{
				Context.SignalSyncPoint(DeviceSyncPoints[CurrentPipeline]);
			}
		}
	}
}

void FD3D12LegacyBarriersForContext::AddGlobalBarrier(
	FD3D12ContextCommon& Context,
	ED3D12Access D3D12AccessBefore,
	ED3D12Access D3D12AccessAfter)
{
	if (EnumOnlyContainsFlags(D3D12AccessBefore, ED3D12Access::UAVMask | ED3D12Access::BVHRead | ED3D12Access::BVHWrite)
		&& EnumOnlyContainsFlags(D3D12AccessAfter, ED3D12Access::UAVMask | ED3D12Access::BVHRead | ED3D12Access::BVHWrite))
	{
		Batcher->AddUAV(Context);
	}
	else
	{
		Batcher->AddAliasingBarrier(Context, nullptr, nullptr);
	}
}

void FD3D12LegacyBarriersForContext::AddBarrier(
	FD3D12ContextCommon& Context,
	const FD3D12Resource* pResource,
	ED3D12Access D3D12AccessBefore,
	ED3D12Access D3D12AccessAfter,
	uint32 Subresource)
{
	check(pResource);

	const ED3D12QueueType QueueType =
		Context.GetCommandList().QueueType;

	const D3D12_RESOURCE_STATES StateBefore =
		GetD3D12ResourceState(
			D3D12AccessBefore,
			QueueType,
			pResource->GetDesc(),
			nullptr);

	const D3D12_RESOURCE_STATES StateAfter =
		GetD3D12ResourceState(
			D3D12AccessAfter,
			QueueType,
			pResource->GetDesc(),
			nullptr);

	TransitionResource(
		Context,
		pResource,
		StateBefore,
		StateAfter,
		Subresource);
}

void FD3D12LegacyBarriersForContext::FlushIntoCommandList(
	class FD3D12CommandList& CommandList,
	class FD3D12QueryAllocator& TimestampAllocator)
{
	Batcher->FlushIntoCommandList(CommandList, TimestampAllocator);
}

int32 FD3D12LegacyBarriersForContext::GetNumPendingBarriers() const
{
	return Batcher->Num();
}

#endif // D3D12RHI_SUPPORTS_LEGACY_BARRIERS