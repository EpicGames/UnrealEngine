// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIDefinitions.h"

#if D3D12RHI_SUPPORTS_LEGACY_BARRIERS

#include "ID3D12Barriers.h"

struct FD3D12LegacyBarriersTransitionData;
class FD3D12LegacyBarriersBatcher;

//
// Separate the implementation details from the fulfillment of
// the ID3D12BarriersForAdapter interface so that other platforms
// which need to further specialize both the interface and the 
// implementation don't run into the diamond inheritance problem
// with the interface or are forced to have multiple v-tables
//

namespace FD3D12LegacyBarriersForAdapterImpl
{
	D3D12_RESOURCE_STATES GetInitialState(
		ED3D12Access InD3D12Access,
		const FD3D12ResourceDesc& InDesc);

	void ConfigureDevice(
		ID3D12Device* Device,
		bool InWithD3DDebug);

	uint64 GetTransitionDataSizeBytes();
	uint64 GetTransitionDataAlignmentBytes();

	void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo);

	void ReleaseTransition(
		FRHITransition* Transition);

	HRESULT CreateCommittedResource(
		FD3D12Adapter& Adapter,
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);
	
	HRESULT CreateReservedResource(
		FD3D12Adapter& Adapter,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);

	HRESULT CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 InHeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource);

} // namespace FD3D12LegacyBarriersForAdapterImpl

class FD3D12LegacyBarriersForAdapter final
	: public ID3D12BarriersForAdapter
{
public:
	virtual ~FD3D12LegacyBarriersForAdapter();

	virtual void ConfigureDevice(
		ID3D12Device* Device,
		bool InWithD3DDebug) const override final;

	virtual uint64 GetTransitionDataSizeBytes() const override final;
	virtual uint64 GetTransitionDataAlignmentBytes() const override final;

	virtual void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo) const override final;

	virtual void ReleaseTransition(
		FRHITransition* Transition) const override final;

	virtual HRESULT CreateCommittedResource(
		FD3D12Adapter& Adapter,
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual HRESULT CreateReservedResource(
		FD3D12Adapter& Adapter,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual HRESULT CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 InHeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const override final;

	virtual const TCHAR* GetImplementationName() const override final;
};

class FD3D12LegacyBarriersForContext final
	: public ID3D12BarriersForContext
{
public:
	FD3D12LegacyBarriersForContext();
	virtual ~FD3D12LegacyBarriersForContext();

	virtual void BeginTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) override final;

	virtual void EndTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) override final;

	virtual void AddGlobalBarrier(
		FD3D12ContextCommon& Context,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter) override final;

	virtual void AddBarrier(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter,
		uint32 Subresource) override final;

	virtual void FlushIntoCommandList(
		class FD3D12CommandList& CommandList,
		class FD3D12QueryAllocator& TimestampAllocator) override final;

	virtual int32 GetNumPendingBarriers() const override final;

private:
	struct FD3D12DiscardResource;

	static void HandleReservedResourceCommits(
		FD3D12CommandContext& Context,
		const FD3D12LegacyBarriersTransitionData* TransitionData);

	void HandleResourceDiscardTransitions(
		FD3D12CommandContext& Context,
		const FD3D12LegacyBarriersTransitionData* TransitionData,
		TArray<FD3D12DiscardResource>& ResourcesToDiscard);

	void HandleDiscardResources(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions,
		bool bIsBeginTransition);

	void HandleTransientAliasing(
		FD3D12CommandContext& Context,
		const FD3D12LegacyBarriersTransitionData* TransitionData);

	void HandleResourceTransitions(
		FD3D12CommandContext& Context,
		const FD3D12LegacyBarriersTransitionData* TransitionData,
		bool& bUAVBarrier);

	void TransitionResource(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* InResource,
		D3D12_RESOURCE_STATES InBeforeState,
		D3D12_RESOURCE_STATES InAfterState,
		uint32 InSubresourceIndex);

	const TUniquePtr<FD3D12LegacyBarriersBatcher> Batcher;
};

#endif // D3D12RHI_SUPPORTS_LEGACY_BARRIERS