// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12NvidiaExtensions.h"
#include "D3D12PipelineState.h"
#include "D3D12RayTracing.h"
#include "D3D12WorkGraph.h"
#include "D3D12RHICommon.h"
#include "PipelineStateCache.h"
#include "Async/ParallelFor.h"

#if NV_AFTERMATH

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
	#include "GFSDK_Aftermath.h"
	#include "GFSDK_Aftermath_GpuCrashdump.h"
	#include "GFSDK_Aftermath_GpuCrashDumpDecoding.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace UE::RHICore::Nvidia::Aftermath::D3D12
{
	void InitializeDevice(ID3D12Device* RootDevice)
	{
		UE::RHICore::Nvidia::Aftermath::InitializeDevice([&](uint32 Flags)
		{
			return GFSDK_Aftermath_DX12_Initialize(GFSDK_Aftermath_Version_API, Flags, RootDevice);
		});
	}

	template<typename T>
	void RegisterShader(T* Value)
	{
		RegisterShaderBinary(Value->Code.GetData(), Value->Code.Num(), Value->HasShaderName() ? Value->GetShaderName() : nullptr);
	}
	
	template<typename T>
	void ConditionallyAddShader(T* Value, TArray<FRHIShader*>& Shaders, TSet<FRHIShader*>& ShaderSet)
	{
		if (Value && !ShaderSet.Contains(Value))
		{
			Shaders.Add(Value);
			ShaderSet.Add(Value);
		}
	}

	bool IsPipelineUnused(FD3D12Adapter* Adapter, const D3D12ResourceFrameCounter& Counter, uint32_t Threshold)
	{
		const uint32 AdapterFrameIndex = Adapter->GetFrameFence().GetNextFenceToSignal();
		const uint32 PipelineFrameIndex = Counter.Get();

		// Check in case that it's ahead of the adapter, just in case it entered a strange state
		return (PipelineFrameIndex <= AdapterFrameIndex) && (AdapterFrameIndex - PipelineFrameIndex) > Threshold;
	}

	void CreateShaderAssociations(float TimeLimitSeconds, uint32 FrameLimit)
	{
		uint64 CycleStart = FPlatformTime::Cycles64();

		UE_LOGF(LogD3D12RHI, Log, "Starting late shader associations...");

		TArray<FRHIShader*> Shaders;
		TSet<FRHIShader*>   ShaderSet;

		uint32_t IgnoredPipelines = 0;

		// Get active pipelines, allow one second for consolidation to finish
		TArray<TRefCountPtr<FRHIResource>> PipelineResources;
		PipelineStateCache::GetPipelineStates(PipelineResources, true, FTimeout(FTimespan::FromSeconds(1)));

		// Deduplicate shaders, Aftermath hashes are not local to the parent pipeline
		for (FRHIResource* Resource : PipelineResources)
		{
			if (!Resource)
			{
				continue;
			}
			
			switch (Resource->GetType())
			{
				default:
					checkNoEntry();
				case RRT_GraphicsPipelineState:
				{
					auto* Pipeline = static_cast<FD3D12GraphicsPipelineState*>(Resource);
						
					if (IsPipelineUnused(Pipeline->PipelineState->GetParentAdapter(), Pipeline->FrameCounter, FrameLimit))
					{
						IgnoredPipelines++;
						continue;
					}
						
					ConditionallyAddShader(static_cast<FD3D12VertexShader*>(Pipeline->PipelineStateInitializer.BoundShaderState.GetVertexShader()), Shaders, ShaderSet);
					ConditionallyAddShader(static_cast<FD3D12GeometryShader*>(Pipeline->PipelineStateInitializer.BoundShaderState.GetGeometryShader()), Shaders, ShaderSet);
					ConditionallyAddShader(static_cast<FD3D12AmplificationShader*>(Pipeline->PipelineStateInitializer.BoundShaderState.GetAmplificationShader()), Shaders, ShaderSet);
					ConditionallyAddShader(static_cast<FD3D12MeshShader*>(Pipeline->PipelineStateInitializer.BoundShaderState.GetMeshShader()), Shaders, ShaderSet);
					ConditionallyAddShader(static_cast<FD3D12PixelShader*>(Pipeline->PipelineStateInitializer.BoundShaderState.GetPixelShader()), Shaders, ShaderSet);
					break;
				}
				case RRT_ComputePipelineState:
				{
					auto* Pipeline = static_cast<FD3D12ComputePipelineState*>(Resource);
					
					if (IsPipelineUnused(Pipeline->PipelineState->GetParentAdapter(), Pipeline->FrameCounter, FrameLimit))
					{
						IgnoredPipelines++;
						continue;
					}
						
					ConditionallyAddShader(static_cast<FD3D12ComputeShader*>(Pipeline->GetComputeShader()), Shaders, ShaderSet);
					break;
				}
#if D3D12_RHI_RAYTRACING
				case RRT_RayTracingPipelineState:
				{
					auto* Pipeline = static_cast<FD3D12RayTracingPipelineState*>(Resource);
						
					if (IsPipelineUnused(Pipeline->Device->GetParentAdapter(), Pipeline->FrameCounter, FrameLimit))
					{
						IgnoredPipelines++;
						continue;
					}
						
					for (const TRefCountPtr<FD3D12RayTracingShader>& Shader : Pipeline->RayGenShaders.Shaders)
					{
						ConditionallyAddShader(Shader.GetReference(), Shaders, ShaderSet);
					}
					for (const TRefCountPtr<FD3D12RayTracingShader>& Shader : Pipeline->CallableShaders.Shaders)
					{
						ConditionallyAddShader(Shader.GetReference(), Shaders, ShaderSet);
					}
					for (const TRefCountPtr<FD3D12RayTracingShader>& Shader : Pipeline->HitGroupShaders.Shaders)
					{
						ConditionallyAddShader(Shader.GetReference(), Shaders, ShaderSet);
					}
					for (const TRefCountPtr<FD3D12RayTracingShader>& Shader : Pipeline->MissShaders.Shaders)
					{
						ConditionallyAddShader(Shader.GetReference(), Shaders, ShaderSet);
					}
					break;
				}
#endif // D3D12_RHI_RAYTRACING
#if D3D12_RHI_WORKGRAPHS
				case RRT_WorkGraphPipelineState:
				{
					auto* Pipeline = static_cast<FD3D12WorkGraphPipelineState*>(Resource);
						
					if (IsPipelineUnused(Pipeline->Device->GetParentAdapter(), Pipeline->FrameCounter, FrameLimit))
					{
						IgnoredPipelines++;
						continue;
					}
						
					for (FRHIShader* Shader : Pipeline->Shaders)
					{
						auto* ShaderTyped = static_cast<FD3D12WorkGraphShader*>(Shader);
						ConditionallyAddShader(ShaderTyped, Shaders, ShaderSet);
					}
					break;
				}
#endif // D3D12_RHI_WORKGRAPHS
			}
		}

		UE_LOGF(LogD3D12RHI, Log, "Late shader associations ignored %u pipelines based on frame fences", IgnoredPipelines);

		// Parallelize as much as possible to avoid timeouts
		ParallelFor(Shaders.Num(), [CycleStart, TimeLimitSeconds, &Shaders](int32 Index)
		{
			// Aftermath handling is time constrained, if we hit the limit just stop
			float Elapsed = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CycleStart);
			if (Elapsed >= TimeLimitSeconds)
			{
				UE_CALL_ONCE([Elapsed]
				{
					UE_LOGF(LogD3D12RHI, Warning, "Late shader associations timed out at %.0fs", Elapsed);
				});
				return;
			}

			FRHIShader* Shader = Shaders[Index];
			switch (Shader->GetFrequency())
			{
				default:
					checkNoEntry();
					break;
				case SF_Vertex:
					RegisterShader(static_cast<FD3D12VertexShader*>(Shader));
					break;
				case SF_Amplification:
					RegisterShader(static_cast<FD3D12AmplificationShader*>(Shader));
					break;
				case SF_Mesh:
					RegisterShader(static_cast<FD3D12MeshShader*>(Shader));
					break;
				case SF_Geometry:
					RegisterShader(static_cast<FD3D12GeometryShader*>(Shader));
					break;
				case SF_Pixel:
					RegisterShader(static_cast<FD3D12PixelShader*>(Shader));
					break;
				case SF_Compute:
					RegisterShader(static_cast<FD3D12ComputeShader*>(Shader));
					break;
#if D3D12_RHI_RAYTRACING
				case SF_RayGen:
				case SF_RayCallable:
				case SF_RayHitGroup:
				case SF_RayMiss:
					RegisterShader(static_cast<FD3D12RayTracingShader*>(Shader));
					break;
#endif // D3D12_RHI_RAYTRACING
#if D3D12_RHI_WORKGRAPHS
				case SF_WorkGraphRoot:
				case SF_WorkGraphComputeNode:
					RegisterShader(static_cast<FD3D12WorkGraphShader*>(Shader));
					break;
#endif // D3D12_RHI_WORKGRAPHS
			}
		});
		
		double TimeMS = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - CycleStart);
		UE_LOGF(LogD3D12RHI, Log, "Created late shader associations, took %.0fms", TimeMS);
	}

	FCommandList RegisterCommandList(ID3D12CommandList* D3DCommandList)
	{
		FCommandList Handle{};
		if (IsEnabled())
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_DX12_CreateContextHandle(D3DCommandList, &Handle);
			if (Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_DX12_CreateContextHandle failed: 0x%08x", Result);
				Handle = {};
			}
		}

		return Handle;
	}

	void UnregisterCommandList(FCommandList CommandList)
	{
		if (IsEnabled() && CommandList)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_ReleaseContextHandle(CommandList);
			UE_CLOGF(Result != GFSDK_Aftermath_Result_Success, LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_ReleaseContextHandle failed: 0x%08x", Result);
		}
	}

	FResource RegisterResource(ID3D12Resource* D3DResource)
	{
		FResource Handle{};
		if (IsResourceTrackingEnabled())
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_DX12_RegisterResource(D3DResource, &Handle);
			if (Result != GFSDK_Aftermath_Result_Success)
			{
				UE_LOGF(LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_DX12_RegisterResource failed: 0x%08x", Result);
				Handle = {};
			}
		}
		return Handle;
	}

	void UnregisterResource(FResource Resource)
	{
		if (IsResourceTrackingEnabled() && Resource)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_DX12_UnregisterResource(Resource);
			UE_CLOGF(Result != GFSDK_Aftermath_Result_Success, LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_DX12_UnregisterResource failed: 0x%08x", Result);
		}
	}

#if WITH_RHI_BREADCRUMBS
	void BeginBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		FMarker Marker(Breadcrumb);
		if (Marker)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_SetEventMarker(CommandList, Marker.GetPtr(), Marker.GetSize());
			UE_CLOGF(Result != GFSDK_Aftermath_Result_Success, LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_SetEventMarker failed in BeginBreadcrumb: 0x%08x", Result);
		}
	}

	void EndBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb)
	{
		FMarker Marker(Breadcrumb->GetParent());
		if (Marker)
		{
			GFSDK_Aftermath_Result Result = GFSDK_Aftermath_SetEventMarker(CommandList, Marker.GetPtr(), Marker.GetSize());
			UE_CLOGF(Result != GFSDK_Aftermath_Result_Success, LogD3D12RHI, VeryVerbose, "GFSDK_Aftermath_SetEventMarker failed in EndBreadcrumb: 0x%08x", Result);
		}
	}
#endif
}

#endif // NV_AFTERMATH
