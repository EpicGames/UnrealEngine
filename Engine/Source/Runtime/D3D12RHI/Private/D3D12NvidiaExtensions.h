// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12ThirdParty.h"
#include "RHICoreNvidiaAftermath.h"

#if WITH_NVAPI
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START
		#include <nvapi.h>
		#include <nvShaderExtnEnums.h>
	THIRD_PARTY_INCLUDES_END
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif // WITH_NVAPI

#if NV_AFTERMATH

	struct GFSDK_Aftermath_ContextHandle__;
	struct GFSDK_Aftermath_ResourceHandle__;

	namespace UE::RHICore::Nvidia::Aftermath::D3D12
	{
		typedef struct GFSDK_Aftermath_ContextHandle__* FCommandList;
		typedef struct GFSDK_Aftermath_ResourceHandle__* FResource;

		void InitializeDevice(ID3D12Device* RootDevice);
		void CreateShaderAssociations(float TimeLimitSeconds, uint32 FrameLimit);

		FCommandList RegisterCommandList(ID3D12CommandList* D3DCommandList);
		void UnregisterCommandList(FCommandList CommandList);

		FResource RegisterResource(ID3D12Resource* D3DResource);
		void UnregisterResource(FResource Resource);

	#if WITH_RHI_BREADCRUMBS
		void BeginBreadcrumb(FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb);
		void EndBreadcrumb  (FCommandList CommandList, FRHIBreadcrumbNode* Breadcrumb);
	#endif
	}

#endif // NV_AFTERMATH
