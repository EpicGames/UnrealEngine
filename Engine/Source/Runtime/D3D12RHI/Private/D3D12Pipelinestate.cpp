// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of D3D12 Pipelinestate related functions

//-----------------------------------------------------------------------------
//	Include Files
//-----------------------------------------------------------------------------
#include "D3D12PipelineState.h"
#include "D3D12RHIPrivate.h"
#include "Hash/CityHash.h"
#include "PipelineStateCache.h"

static TAutoConsoleVariable<float> CVarPSOStallWarningThresholdInMs(
	TEXT("D3D12.PSO.StallWarningThresholdInMs"),
	100.0f,
	TEXT("Sets a threshold of when to logs messages about stalls due to PSO creation.\n")
	TEXT("Value is in milliseconds. (100 is the default)\n"),
	ECVF_ReadOnly);

bool GLowLevelCacheKeepPrecachedPSOs = false;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepLowLevel(
	TEXT("D3D12.PSOPrecache.KeepLowLevel"),
	GLowLevelCacheKeepPrecachedPSOs,
	TEXT("Keep in memory the d3d12 PSO blob for precached PSOs. Consumes more memory but reduces stalls.\n"),
	ECVF_ReadOnly
);

bool GLowLevelCacheKeepUsedPSOs = false;
static FAutoConsoleVariableRef CVarLowLevelCacheKeepUsedPSOs(
	TEXT("D3D12.PSO.KeepUsedPSOsInLowLevelCache"),
	GLowLevelCacheKeepUsedPSOs,
	TEXT("Keep in memory the d3d12 PSO blob for used PSOs. Consumes more memory but reduces potential stalls.\n"),
	ECVF_ReadOnly
);

/// @cond DOXYGEN_WARNINGS

static void TranslateRenderTargetFormats(
	const FGraphicsPipelineStateInitializer& PsoInit,
	D3D12_RT_FORMAT_ARRAY& RTFormatArray,
	DXGI_FORMAT& DSVFormat)
{
	RTFormatArray.NumRenderTargets = PsoInit.ComputeNumValidRenderTargets();

	for (uint32 RTIdx = 0; RTIdx < PsoInit.RenderTargetsEnabled; ++RTIdx)
	{
		checkSlow(PsoInit.RenderTargetFormats[RTIdx] == PF_Unknown || GPixelFormats[PsoInit.RenderTargetFormats[RTIdx]].Supported);

		DXGI_FORMAT PlatformFormat = (DXGI_FORMAT)GPixelFormats[PsoInit.RenderTargetFormats[RTIdx]].PlatformFormat;
		ETextureCreateFlags Flags = PsoInit.RenderTargetFlags[RTIdx];

		RTFormatArray.RTFormats[RTIdx] = UE::DXGIUtilities::FindShaderResourceFormat(UE::DXGIUtilities::GetPlatformTextureResourceFormat(PlatformFormat, Flags), EnumHasAnyFlags(Flags, ETextureCreateFlags::SRGB));
	}

	checkSlow(PsoInit.DepthStencilTargetFormat == PF_Unknown || GPixelFormats[PsoInit.DepthStencilTargetFormat].Supported);

	DXGI_FORMAT PlatformFormat = (DXGI_FORMAT)GPixelFormats[PsoInit.DepthStencilTargetFormat].PlatformFormat;

	DSVFormat = UE::DXGIUtilities::FindDepthStencilFormat(UE::DXGIUtilities::GetPlatformTextureResourceFormat(PlatformFormat, PsoInit.DepthStencilTargetFlag));
}

static FD3D12LowLevelGraphicsPipelineStateDesc GetLowLevelGraphicsPipelineStateDesc(const FGraphicsPipelineStateInitializer& Initializer, const FD3D12RootSignature* RootSignature)
{
	FD3D12LowLevelGraphicsPipelineStateDesc Desc{};

	Desc.pRootSignature = RootSignature;
	Desc.Desc.pRootSignature = RootSignature->GetRootSignature();

#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
	Desc.Desc.BlendState = Initializer.BlendState ? FD3D12DynamicRHI::ResourceCast(Initializer.BlendState)->Desc : CD3DX12_BLEND_DESC(D3D12_DEFAULT);
#endif // #if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
#if !D3D12_USE_DERIVED_PSO
	Desc.Desc.SampleMask = 0xFFFFFFFF;
	Desc.Desc.RasterizerState = Initializer.RasterizerState ? FD3D12DynamicRHI::ResourceCast(Initializer.RasterizerState)->Desc : CD3DX12_RASTERIZER_DESC(D3D12_DEFAULT);
	Desc.Desc.DepthStencilState = Initializer.DepthStencilState ? CD3DX12_DEPTH_STENCIL_DESC1(FD3D12DynamicRHI::ResourceCast(Initializer.DepthStencilState)->Desc) : CD3DX12_DEPTH_STENCIL_DESC1(D3D12_DEFAULT);
#endif // !D3D12_USE_DERIVED_PSO

	Desc.Desc.PrimitiveTopologyType = D3D12PrimitiveTypeToTopologyType(TranslatePrimitiveType(Initializer.PrimitiveType));

	TranslateRenderTargetFormats(Initializer, Desc.Desc.RTFormatArray, Desc.Desc.DSVFormat);

	Desc.Desc.SampleDesc.Count = Initializer.NumSamples;
	Desc.Desc.SampleDesc.Quality = GetMaxMSAAQuality(Initializer.NumSamples);

	if (FD3D12VertexDeclaration* InputLayout = (FD3D12VertexDeclaration*) Initializer.BoundShaderState.VertexDeclarationRHI)
	{
		Desc.Desc.InputLayout.NumElements        = InputLayout->VertexElements.Num();
		Desc.Desc.InputLayout.pInputElementDescs = InputLayout->VertexElements.GetData();
		Desc.InputLayoutHash = InputLayout->HashNoStrides; // Vertex stream stride does not affect the D3D12 PSO
	}

#define COPY_SHADER(Initial, Name) \
	if (FD3D12##Name##Shader* Shader = (FD3D12##Name##Shader*) Initializer.BoundShaderState.Get##Name##Shader()) \
	{ \
		Desc.Desc.Initial##S = Shader->GetShaderBytecode(); \
		Desc.Initial##SHash = Shader->GetBytecodeHash(); \
	}
	COPY_SHADER(V, Vertex);
	COPY_SHADER(M, Mesh);
	COPY_SHADER(A, Amplification);
	COPY_SHADER(P, Pixel);
	COPY_SHADER(G, Geometry);
#undef COPY_SHADER

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
#define EXT_SHADER(Initial, Name) \
	if (FD3D12##Name##Shader* Shader = (FD3D12##Name##Shader*) Initializer.BoundShaderState.Get##Name##Shader()) \
	{ \
		if (Shader->VendorExtensions.Num() > 0) \
		{ \
			Desc.Initial##SExtensions = &Shader->VendorExtensions; \
		} \
	}
	EXT_SHADER(V, Vertex);
	EXT_SHADER(M, Mesh);
	EXT_SHADER(A, Amplification);
	EXT_SHADER(P, Pixel);
	EXT_SHADER(G, Geometry);
#undef EXT_SHADER
#endif

#if !D3D12_USE_DERIVED_PSO
	// TODO: [PSO API] For now, keep DBT enabled, if available, until it is added as part of a member to the Initializer's DepthStencilState
	Desc.Desc.DepthStencilState.DepthBoundsTestEnable = GSupportsDepthBoundsTest && Initializer.bDepthBounds;
#endif

	// The blend state is liable to have non default values for RenderTarget indices that exceed
	// NumRenderTargets, this will in turn cause them to have a different `D3D12_PIPELINE_STATE_STREAM_DESC`
	// as far as the driver is concerned.
	// Which will then cause a hash collision when using driver level pipeline caching, due to the hash
	// only being derived from the active render target blend states.
#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
	for (UINT RenderTarget = Desc.Desc.RTFormatArray.NumRenderTargets; RenderTarget < 8; ++RenderTarget)
	{
		Desc.Desc.BlendState.RenderTarget[RenderTarget] = D3D12_RENDER_TARGET_BLEND_DESC{};
	}
#endif // !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS

	Desc.bFromPSOFileCache = Initializer.bFromPSOFileCache;

	return Desc;
}

static FD3D12ComputePipelineStateDesc GetComputePipelineStateDesc(const FD3D12ComputeShader* ComputeShader, const FD3D12RootSignature* RootSignature)
{
	FD3D12ComputePipelineStateDesc Desc{};

	Desc.pRootSignature = RootSignature;
	Desc.Desc.pRootSignature = RootSignature->GetRootSignature();
	Desc.Desc.CS = ComputeShader->GetShaderBytecode();
	Desc.CSHash = ComputeShader->GetBytecodeHash();
#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	if (ComputeShader->VendorExtensions.Num() > 0)
	{
		Desc.Extensions = &ComputeShader->VendorExtensions;
	}
#endif

	return Desc;
}

FD3D12PipelineStateWorker::FD3D12PipelineStateWorker(FD3D12Adapter* Adapter, const ComputePipelineCreationArgs& InArgs)
	: FD3D12AdapterChild(Adapter)
	, bIsGraphics(false)
{
	CreationArgs.ComputeArgs = new ComputePipelineCreationArgs_POD();
	CreationArgs.ComputeArgs->Init(InArgs.Args);
};

FD3D12PipelineStateWorker::FD3D12PipelineStateWorker(FD3D12Adapter* Adapter, const GraphicsPipelineCreationArgs& InArgs)
	: FD3D12AdapterChild(Adapter)
	, bIsGraphics(true)
{
	CreationArgs.GraphicsArgs = new GraphicsPipelineCreationArgs_POD();
	CreationArgs.GraphicsArgs->Init(InArgs.Args);
};

/// @endcond

uint64 FD3D12PipelineStateCacheBase::HashData(const void* Data, int32 NumBytes)
{
	return CityHash64((const char*) Data, NumBytes);
}

uint64 FD3D12PipelineStateCacheBase::HashPSODesc(const FD3D12LowLevelGraphicsPipelineStateDesc& Desc)
{
	struct GraphicsPSOData
	{
		ShaderBytecodeHash VSHash;
		ShaderBytecodeHash MSHash;
		ShaderBytecodeHash ASHash;
		ShaderBytecodeHash GSHash;
		ShaderBytecodeHash PSHash;
		uint32 InputLayoutHash;

#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
		uint8 AlphaToCoverageEnable;
		uint8 IndependentBlendEnable;
#endif // #if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
#if !D3D12_USE_DERIVED_PSO
		uint32 SampleMask;
		D3D12_RASTERIZER_DESC RasterizerState;
		D3D12_DEPTH_STENCIL_DESC1 DepthStencilState;
#endif // #if !D3D12_USE_DERIVED_PSO

		D3D12_INDEX_BUFFER_STRIP_CUT_VALUE IBStripCutValue;
		D3D12_PRIMITIVE_TOPOLOGY_TYPE PrimitiveTopologyType;
		DXGI_FORMAT DSVFormat;
		DXGI_SAMPLE_DESC SampleDesc;
		uint32 NodeMask;

		D3D12_PIPELINE_STATE_FLAGS Flags;
	};

	struct RenderTargetData
	{
		DXGI_FORMAT Format;
#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
		D3D12_RENDER_TARGET_BLEND_DESC BlendDesc;
#endif // #if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
	};


	const int32 NumRenderTargets = Desc.Desc.RTFormatArray.NumRenderTargets;

	const size_t GraphicsPSODataSize = sizeof(GraphicsPSOData);
	const size_t RenderTargetDataSize = NumRenderTargets * sizeof(RenderTargetData);
	const size_t TotalDataSize = GraphicsPSODataSize + RenderTargetDataSize;

	char Data[GraphicsPSODataSize + 8 * sizeof(RenderTargetData)];
	FMemory::Memzero(Data, TotalDataSize);

	GraphicsPSOData* PSOData = (GraphicsPSOData*) Data;
	RenderTargetData* RTData = (RenderTargetData*) (Data + GraphicsPSODataSize);

	PSOData->VSHash          = Desc.VSHash;
	PSOData->MSHash          = Desc.MSHash;
	PSOData->ASHash          = Desc.ASHash;
	PSOData->GSHash          = Desc.GSHash;
	PSOData->PSHash          = Desc.PSHash;
	PSOData->InputLayoutHash = Desc.InputLayoutHash;

#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
	PSOData->AlphaToCoverageEnable  = Desc.Desc.BlendState.AlphaToCoverageEnable;
	PSOData->IndependentBlendEnable = Desc.Desc.BlendState.IndependentBlendEnable;
#endif // #if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
#if !D3D12_USE_DERIVED_PSO
	PSOData->SampleMask             = Desc.Desc.SampleMask;
	PSOData->RasterizerState        = Desc.Desc.RasterizerState;
	PSOData->DepthStencilState      = Desc.Desc.DepthStencilState;
#endif // #if !D3D12_USE_DERIVED_PSO
	PSOData->IBStripCutValue        = Desc.Desc.IBStripCutValue;
	PSOData->PrimitiveTopologyType  = Desc.Desc.PrimitiveTopologyType;
	PSOData->DSVFormat              = Desc.Desc.DSVFormat;
	PSOData->SampleDesc             = Desc.Desc.SampleDesc;
	PSOData->NodeMask               = Desc.Desc.NodeMask;
	PSOData->Flags                  = Desc.Desc.Flags;

	for (int32 RT = 0; RT < NumRenderTargets; RT++)
	{
		RTData[RT].Format    = Desc.Desc.RTFormatArray.RTFormats[RT];
#if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
		RTData[RT].BlendDesc = Desc.Desc.BlendState.RenderTarget[RT];
#endif // #if !D3D12_USE_DERIVED_PSO || D3D12_USE_DERIVED_PSO_SHADER_EXPORTS
	}

	return HashData(Data, TotalDataSize);
}

uint64 FD3D12PipelineStateCacheBase::HashPSODesc(const FD3D12ComputePipelineStateDesc& Desc)
{
	struct ComputePSOData
	{
		ShaderBytecodeHash CSHash;
		UINT NodeMask;
		D3D12_PIPELINE_STATE_FLAGS Flags;
	};
	ComputePSOData Data;
	FMemory::Memzero(Data);
	Data.CSHash   = Desc.CSHash;
	Data.NodeMask = Desc.Desc.NodeMask;
	Data.Flags    = Desc.Desc.Flags;

	return HashData(&Data, sizeof(Data));
}

FD3D12PipelineStateCacheBase::FD3D12PipelineStateCacheBase(FD3D12Adapter* InParent)
	: FD3D12AdapterChild(InParent)
{
}

FD3D12PipelineStateCacheBase::~FD3D12PipelineStateCacheBase()
{
	CleanupPipelineStateCaches();
}


FD3D12PipelineState::FD3D12PipelineState(FD3D12Adapter* Parent)
	: FD3D12AdapterChild(Parent)
	, FD3D12MultiNodeGPUObject(FRHIGPUMask::All(), FRHIGPUMask::All()) //Create on all, visible on all
	, Worker(nullptr)
	, InitState(PSOInitState::Uninitialized)
{
	INC_DWORD_STAT(STAT_D3D12NumPSOs);
}

FD3D12PipelineState::~FD3D12PipelineState()
{
	check(!UsePSORefCounting() || GetRefCount() == 0);
	if (Worker)
	{
		Worker->EnsureCompletion(true);
		delete Worker;
		Worker = nullptr;
	}

	if (UsePSORefCounting())
	{
		PipelineState->AddRef(); // AddRef for DeferredDelete to take over the actual deletion
		FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(PipelineState);
	}

	DEC_DWORD_STAT(STAT_D3D12NumPSOs);
}

bool FD3D12PipelineState::UsePSORefCounting()
{
#if D3D12_USE_DERIVED_PSO
	return false;
#else
	bool bShouldRefCountForPSOPrecaching = PipelineStateCache::IsPSOPrecachingEnabled() && !GLowLevelCacheKeepPrecachedPSOs;
	bool bShouldRefCountForUsedPSOs = !GLowLevelCacheKeepUsedPSOs;
	return bShouldRefCountForPSOPrecaching || bShouldRefCountForUsedPSOs;
#endif
}

ID3D12PipelineState* FD3D12PipelineState::InternalGetPipelineState()
{
	// Only one thread should finalize any tasks and set the cached pipeline state.
	// Other threads can wait efficiently behind the lock. Using Sleep() can wake threads 1ms too late.
	FRWScopeLock Lock(GetPipelineStateMutex, SLT_Write);

	if (Worker)
	{
		check(InitState == PSOInitState::Uninitialized);

		Worker->EnsureCompletion(true);
		check(Worker->IsWorkDone());

		PipelineState = Worker->GetTask().PSO.GetReference();

		// Cleanup the worker.
		delete Worker;
		Worker = nullptr;

		InitState = (PipelineState.GetReference() != nullptr)? PSOInitState::Initialized : PSOInitState::CreationFailed;
	}
	else
	{
		// Busy-wait for the PSO. This avoids giving up our time slice.
		if (InitState == PSOInitState::Uninitialized)
		{
			double StartTime = FPlatformTime::Seconds();
			double BusyWaitWarningTime = CVarPSOStallWarningThresholdInMs.GetValueOnAnyThread() * 0.001;
			while (InitState == PSOInitState::Uninitialized)
			{
				const double Time = FPlatformTime::Seconds();

				if (Time - StartTime > BusyWaitWarningTime)
				{
					UE_LOGF(LogD3D12RHI, Log, "Waited for PSO creation for %fms", BusyWaitWarningTime * 1000.0);
					BusyWaitWarningTime *= 2.0;
				}
			}
		}
	}

	return PipelineState.GetReference();
}

FD3D12PipelineStateCommonData::FD3D12PipelineStateCommonData(const FD3D12RootSignature* InRootSignature, FD3D12PipelineState* InPipelineState)
	: RootSignature(InRootSignature)
	, PipelineState(InPipelineState)
{
}

FD3D12GraphicsPipelineState::FD3D12GraphicsPipelineState(
	const FGraphicsPipelineStateInitializer& Initializer,
	const FD3D12RootSignature* InRootSignature,
	FD3D12PipelineState* InPipelineState)
	: FD3D12PipelineStateCommonData(InRootSignature, InPipelineState)
	, PipelineStateInitializer(Initializer)
	, StreamStrides(InPlace, 0)
{
	// hold on to bound RHI resources
	PipelineStateInitializer.BoundShaderState.AddRefResources();
	if (PipelineStateInitializer.BlendState)
	{
		PipelineStateInitializer.BlendState->AddRef();
	}
	if (PipelineStateInitializer.RasterizerState)
	{
		PipelineStateInitializer.RasterizerState->AddRef();
	}
	if (PipelineStateInitializer.DepthStencilState)
	{
		PipelineStateInitializer.DepthStencilState->AddRef();
	}

	if (Initializer.BoundShaderState.VertexDeclarationRHI)
	{
		StreamStrides = static_cast<FD3D12VertexDeclaration*>(Initializer.BoundShaderState.VertexDeclarationRHI)->StreamStrides;
	}

	bShaderNeedsGlobalConstantBuffer[SF_Vertex] = GetVertexShader() && GetVertexShader()->UsesGlobalUniformBuffer();
	bShaderNeedsGlobalConstantBuffer[SF_Mesh] = GetMeshShader() && GetMeshShader()->UsesGlobalUniformBuffer();
	bShaderNeedsGlobalConstantBuffer[SF_Amplification] = GetAmplificationShader() && GetAmplificationShader()->UsesGlobalUniformBuffer();
	bShaderNeedsGlobalConstantBuffer[SF_Pixel] = GetPixelShader() && GetPixelShader()->UsesGlobalUniformBuffer();
	bShaderNeedsGlobalConstantBuffer[SF_Geometry] = GetGeometryShader() && GetGeometryShader()->UsesGlobalUniformBuffer();

	// GRHISupportsPipelineStateSortKey
	SetSortKey(InPipelineState->GetContextSortKey());
}

FD3D12GraphicsPipelineState::~FD3D12GraphicsPipelineState()
{
	// At this point the object is not safe to use in the PSO cache.
	// Currently, the PSO cache manages the lifetime but we could potentially
	// stop doing an AddRef() and remove the PipelineState from any caches at this point.

#if D3D12_USE_DERIVED_PSO
	delete PipelineState;
	PipelineState = nullptr;
#else
	if (FD3D12PipelineState::UsePSORefCounting() && PipelineState != nullptr)
	{
		uint32 RefCount = PipelineState->Release();
		check(RefCount > 0);
        // precache PSO are here to avoid hitches at runtime when we want to create one that is actually used. We don't need to keep them
		// around as this can add up to a lot of system memory
		bool bRemovePrecachedPSO = PipelineStateInitializer.bPSOPrecache && !GLowLevelCacheKeepPrecachedPSOs;
		bool bRemoveUsedPSO = !PipelineStateInitializer.bPSOPrecache && !GLowLevelCacheKeepUsedPSOs;
		if (RefCount == 1 && (bRemoveUsedPSO || bRemovePrecachedPSO))
		{
			FD3D12DynamicRHI* D3D12RHI = FD3D12DynamicRHI::GetD3DRHI();
			FD3D12PipelineStateCache& PSOCache = D3D12RHI->GetAdapter().GetPSOCache();
			// NB: it's possible that this remove will not do anything because another thread might have requested the same PSO since we entered the if
			PSOCache.RemoveFromLowLevelCache(PipelineState, PipelineStateInitializer, RootSignature);
		}
	}
#endif

	// release bound RHI resources
	PipelineStateInitializer.BoundShaderState.ReleaseResources();
	if (PipelineStateInitializer.BlendState)
	{
		PipelineStateInitializer.BlendState->Release();
	}
	if (PipelineStateInitializer.RasterizerState)
	{
		PipelineStateInitializer.RasterizerState->Release();
	}
	if (PipelineStateInitializer.DepthStencilState)
	{
		PipelineStateInitializer.DepthStencilState->Release();
	}
}

FD3D12ComputePipelineState::FD3D12ComputePipelineState(const FComputePipelineStateInitializer& InInitializer, const FD3D12RootSignature* InRootSignature, FD3D12PipelineState* InPipelineState)
	: FRHIComputePipelineState(InInitializer.ComputeShader)
	, FD3D12PipelineStateCommonData(InRootSignature, InPipelineState)
{
	FD3D12ComputeShader* D3D12ComputeShader = static_cast<FD3D12ComputeShader*>(InInitializer.ComputeShader);
	bShaderNeedsGlobalConstantBuffer = D3D12ComputeShader && D3D12ComputeShader->UsesGlobalUniformBuffer();
	bPSOPrecache = InInitializer.bPSOPrecache;
}

FD3D12ComputePipelineState::~FD3D12ComputePipelineState()
{
	// At this point the object is not safe to use in the PSO cache.
	// Currently, the PSO cache manages the lifetime but we could potentially
	// stop doing an AddRef() and remove the PipelineState from any caches at this point.
	if (FD3D12PipelineState::UsePSORefCounting() && PipelineState != nullptr)
	{
		uint32 RefCount = PipelineState->Release();
		check(RefCount > 0);
		bool bRemovePrecachedPSO = bPSOPrecache && !GLowLevelCacheKeepPrecachedPSOs;
		bool bRemoveUsedPSO = !bPSOPrecache && !GLowLevelCacheKeepUsedPSOs;
		if (RefCount == 1 && (bRemoveUsedPSO || bRemovePrecachedPSO))
		{
			FD3D12DynamicRHI* D3D12RHI = FD3D12DynamicRHI::GetD3DRHI();
			FD3D12PipelineStateCache& PSOCache = D3D12RHI->GetAdapter().GetPSOCache();
			// NB: it's possible that this remove will not do anything because another thread might have requested the same PSO since we entered the if
			PSOCache.RemoveFromLowLevelCache(PipelineState, static_cast<FD3D12ComputeShader*>(GetComputeShader()), RootSignature);
		}
	}
}

void FD3D12PipelineStateCacheBase::CleanupPipelineStateCaches()
{
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	// The runtime caches manage the lifetime of their FD3D12GraphicsPipelineState and FD3D12ComputePipelineState.
	// We need to release them.
	{
		FRWScopeLock Lock(InitializerToGraphicsPipelineMapMutex, FRWScopeLockType::SLT_Write);
		for (auto Pair : InitializerToGraphicsPipelineMap)
		{
			FD3D12GraphicsPipelineState* GraphicsPipelineState = Pair.Value;
			ensure(GIsRHIInitialized || (!GIsRHIInitialized && GraphicsPipelineState->GetRefCount() == 1));
			GraphicsPipelineState->Release();
		}
		InitializerToGraphicsPipelineMap.Reset();
	}

	{
		FRWScopeLock Lock(ComputeShaderToComputePipelineMapMutex, FRWScopeLockType::SLT_Write);
		for (auto Pair : ComputeShaderToComputePipelineMap)
		{
			FD3D12ComputePipelineState* ComputePipelineState = Pair.Value;
			ensure(GIsRHIInitialized || (!GIsRHIInitialized && ComputePipelineState->GetRefCount() == 1));
			ComputePipelineState->Release();
		}
		ComputeShaderToComputePipelineMap.Reset();
	}
#endif
	{
		FRWScopeLock Lock(LowLevelGraphicsPipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
		for (auto Iter = LowLevelGraphicsPipelineStateCache.CreateConstIterator(); Iter; ++Iter)
		{
			const FD3D12PipelineState* const PipelineState = Iter.Value();
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				if (PipelineState)
				{
					PipelineState->Release();
				}
			}
			else
			{
				delete PipelineState;
			}
		}
		LowLevelGraphicsPipelineStateCache.Empty();
	}

	{
		FRWScopeLock Lock(ComputePipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
		for (auto Iter = ComputePipelineStateCache.CreateConstIterator(); Iter; ++Iter)
		{
			const FD3D12PipelineState* const PipelineState = Iter.Value();
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				if (PipelineState)
				{
					PipelineState->Release();
				}
			}
			else
			{
				delete PipelineState;
			}
		}
		ComputePipelineStateCache.Empty();
	}
}

#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
FD3D12GraphicsPipelineState* FD3D12PipelineStateCacheBase::AddToRuntimeCache(const FGraphicsPipelineStateInitializer& Initializer, uint32 InitializerHash, const FD3D12RootSignature* RootSignature, FD3D12PipelineState* PipelineState)
{
	// Lifetime managed by the runtime cache. AddRef() so the upper level doesn't delete the FD3D12GraphicsPipelineState objects while they're still in the runtime cache.
	// One alternative is to remove the object from the runtime cache in the FD3D12GraphicsPipelineState destructor.
	FD3D12GraphicsPipelineState* const GraphicsPipelineState = new FD3D12GraphicsPipelineState(Initializer, RootSignature, PipelineState);
	GraphicsPipelineState->AddRef();

	check(GraphicsPipelineState && InitializerHash != 0);
	check(GraphicsPipelineState->PipelineState != nullptr);

	{
		FRWScopeLock Lock(InitializerToGraphicsPipelineMapMutex, FRWScopeLockType::SLT_Write);
		InitializerToGraphicsPipelineMap.Add(FInitializerToGPSOMapKey(&GraphicsPipelineState->PipelineStateInitializer, InitializerHash), GraphicsPipelineState);
	}

	INC_DWORD_STAT(STAT_PSOGraphicsNumHighlevelCacheEntries);
	return GraphicsPipelineState;
}
#endif

FD3D12PipelineState* FD3D12PipelineStateCacheBase::FindInLowLevelCache(const FD3D12LowLevelGraphicsPipelineStateDesc& Desc)
{
	check(Desc.CombinedHash != 0);

	{
		FRWScopeLock Lock(LowLevelGraphicsPipelineStateCacheMutex, FRWScopeLockType::SLT_ReadOnly);
		FD3D12PipelineState** Found = LowLevelGraphicsPipelineStateCache.Find(Desc);
		if (Found)
		{
			INC_DWORD_STAT(STAT_PSOGraphicsLowlevelCacheHit);
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				const uint32 Refs = (*Found)->AddRef();
				checkSlow(Refs >= 2);
			}
			return *Found;
		}
	}

	INC_DWORD_STAT(STAT_PSOGraphicsLowlevelCacheMiss);
	return nullptr;
}

FD3D12PipelineState* FD3D12PipelineStateCacheBase::CreateAndAddToLowLevelCache(const FD3D12LowLevelGraphicsPipelineStateDesc& Desc)
{
	// Add PSO to low level cache.
	FD3D12PipelineState* PipelineState = nullptr;
	AddToLowLevelCache(Desc, &PipelineState, [this](FD3D12PipelineState** PipelineState, const FD3D12LowLevelGraphicsPipelineStateDesc& Desc)
	{ 
		OnPSOCreated(*PipelineState, Desc);
	});

	return PipelineState;
}

void FD3D12PipelineStateCacheBase::RemoveFromLowLevelCache(FD3D12PipelineState* PipelineState, const FGraphicsPipelineStateInitializer& PipelineStateInitializer, const FD3D12RootSignature* RootSignature)
{
    if (!FD3D12PipelineState::UsePSORefCounting())
    {
        checkNoEntry();
        return;
    }

	FRWScopeLock Lock(LowLevelGraphicsPipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
	// By the time we reach this function, it's possible another thread made a request for the same PSO
	if (PipelineState->Release() == 0)
	{
		FD3D12LowLevelGraphicsPipelineStateDesc LowLevelDesc = GetLowLevelGraphicsPipelineStateDesc(PipelineStateInitializer, RootSignature);
		LowLevelDesc.Desc.NodeMask = FRHIGPUMask::All().GetNative();
		LowLevelDesc.CombinedHash = FD3D12PipelineStateCacheBase::HashPSODesc(LowLevelDesc);
		int32 ElementsRemoved = LowLevelGraphicsPipelineStateCache.Remove(LowLevelDesc);
		ensure(ElementsRemoved == 1);
	}
	else
	{
		// If another thread requested the pipeline state, we need to restore the refcount we just decremented
		const uint32 Refs = PipelineState->AddRef();
		checkSlow(Refs >= 2);
	}
}

void FD3D12PipelineStateCacheBase::RemoveFromLowLevelCache(FD3D12PipelineState* PipelineState, const FD3D12ComputeShader* ComputeShader, const FD3D12RootSignature* RootSignature)
{
	if (!FD3D12PipelineState::UsePSORefCounting())
	{
		checkNoEntry();
		return;
	}

	FRWScopeLock Lock(ComputePipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
	// By the time we reach this function, it's possible another thread made a request for the same PSO
	if (PipelineState->Release() == 0)
	{
		FD3D12ComputePipelineStateDesc LowLevelDesc = GetComputePipelineStateDesc(ComputeShader, RootSignature);
		LowLevelDesc.Desc.NodeMask = FRHIGPUMask::All().GetNative();
		LowLevelDesc.CombinedHash = FD3D12PipelineStateCacheBase::HashPSODesc(LowLevelDesc);
		int32 ElementsRemoved = ComputePipelineStateCache.Remove(LowLevelDesc);
		ensure(ElementsRemoved == 1);
	}
	else
	{
		// If another thread requested the pipeline state, we need to restore the refcount we just decremented
		const uint32 Refs = PipelineState->AddRef();
		checkSlow(Refs >= 2);
	}
}

void FD3D12PipelineStateCacheBase::AddToLowLevelCache(const FD3D12LowLevelGraphicsPipelineStateDesc& Desc, FD3D12PipelineState** OutPipelineState, const FPostCreateGraphicCallback& PostCreateCallback)
{
	check(Desc.CombinedHash != 0);

	// Double check the desc doesn't already exist while the lock is taken.
	// This avoids having multiple threads try to create the same PSO.
	{
		FRWScopeLock Lock(LowLevelGraphicsPipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
		FD3D12PipelineState** const PipelineState = LowLevelGraphicsPipelineStateCache.Find(Desc);
		if (PipelineState)
		{
			// This desc already exists.
			*OutPipelineState = *PipelineState;
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				(*OutPipelineState)->AddRef();
			}
			return;
		}

		// Add the FD3D12PipelineState object to the cache, but don't actually create the underlying PSO yet while the lock is taken.
		// This allows multiple threads to create different PSOs at the same time.
		INC_DWORD_STAT(STAT_PSOGraphicsNumLowlevelCacheEntries);
		FD3D12PipelineState* NewPipelineState = new FD3D12PipelineState(GetParentAdapter());
		LowLevelGraphicsPipelineStateCache.Add(Desc, NewPipelineState);

		// This AddRef is for the low level cache
		if (FD3D12PipelineState::UsePSORefCounting())
		{
			NewPipelineState->AddRef();
		}

		*OutPipelineState = NewPipelineState;

		// This AddRef is for the FD3D12GraphicsPipelineState requesting it
		if (FD3D12PipelineState::UsePSORefCounting())
		{
			(*OutPipelineState)->AddRef();
		}

	}

	// Create the underlying PSO and then perform any other additional tasks like cleaning up/adding to caches, etc.
	PostCreateCallback(OutPipelineState, Desc);
}

#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
FD3D12ComputePipelineState* FD3D12PipelineStateCacheBase::AddToRuntimeCache(FD3D12ComputeShader* ComputeShader, FD3D12PipelineState* PipelineState)
{
	// Lifetime managed by the runtime cache. AddRef() so the upper level doesn't delete the FD3D12ComputePipelineState objects while they're still in the runtime cache.
	// One alternative is to remove the object from the runtime cache in the FD3D12ComputePipelineState destructor.
	FD3D12ComputePipelineState* const ComputePipelineState = new FD3D12ComputePipelineState(ComputeShader, PipelineState);
	ComputePipelineState->AddRef();

	check(ComputePipelineState && ComputeShader != nullptr);
	check(ComputePipelineState->PipelineState != nullptr);

	{
		FRWScopeLock Lock(ComputeShaderToComputePipelineMapMutex, FRWScopeLockType::SLT_Write);
		ComputeShaderToComputePipelineMap.Add(ComputeShader, ComputePipelineState);
	}

	INC_DWORD_STAT(STAT_PSOComputeNumHighlevelCacheEntries);
	return ComputePipelineState;
}
#endif

FD3D12PipelineState* FD3D12PipelineStateCacheBase::FindInLowLevelCache(const FD3D12ComputePipelineStateDesc& Desc)
{
	check(Desc.CombinedHash != 0);

	{
		FRWScopeLock Lock(ComputePipelineStateCacheMutex, FRWScopeLockType::SLT_ReadOnly);
		FD3D12PipelineState** Found = ComputePipelineStateCache.Find(Desc);
		if (Found)
		{
			INC_DWORD_STAT(STAT_PSOComputeLowlevelCacheHit);
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				const uint32 Refs = (*Found)->AddRef();
				checkSlow(Refs >= 2);
			}
			return *Found;
		}
	}

	INC_DWORD_STAT(STAT_PSOComputeLowlevelCacheMiss);
	return nullptr;
}

FD3D12PipelineState* FD3D12PipelineStateCacheBase::CreateAndAddToLowLevelCache(const FD3D12ComputePipelineStateDesc& Desc)
{
	// Add PSO to low level cache.
	FD3D12PipelineState* PipelineState = nullptr;

	AddToLowLevelCache(Desc, &PipelineState, [&](FD3D12PipelineState* PipelineState, const FD3D12ComputePipelineStateDesc& Desc)
	{ 
		OnPSOCreated(PipelineState, Desc); 
	});

	return PipelineState;
}

void FD3D12PipelineStateCacheBase::AddToLowLevelCache(const FD3D12ComputePipelineStateDesc& Desc, FD3D12PipelineState** OutPipelineState, const FPostCreateComputeCallback& PostCreateCallback)
{
	check(Desc.CombinedHash != 0);

	// Double check the desc doesn't already exist while the lock is taken.
	// This avoids having multiple threads try to create the same PSO.
	{
		FRWScopeLock Lock(ComputePipelineStateCacheMutex, FRWScopeLockType::SLT_Write);
		FD3D12PipelineState** const PipelineState = ComputePipelineStateCache.Find(Desc);
		if (PipelineState)
		{
			// This desc already exists.
			*OutPipelineState = *PipelineState;
			if (FD3D12PipelineState::UsePSORefCounting())
			{
				(*OutPipelineState)->AddRef();
			}
			return;
		}

		// Add the FD3D12PipelineState object to the cache, but don't actually create the underlying PSO yet while the lock is taken.
		// This allows multiple threads to create different PSOs at the same time.
		INC_DWORD_STAT(STAT_PSOComputeNumLowlevelCacheEntries);
		FD3D12PipelineState* NewPipelineState = new FD3D12PipelineState(GetParentAdapter());
		ComputePipelineStateCache.Add(Desc, NewPipelineState);

		// This AddRef is for the low level cache.
		if (FD3D12PipelineState::UsePSORefCounting())
		{
			NewPipelineState->AddRef();
		}

		*OutPipelineState = NewPipelineState;

		// This AddRef is for the FD3D12ComputePipelineState requesting it.
		if (FD3D12PipelineState::UsePSORefCounting())
		{
			(*OutPipelineState)->AddRef();
		}
	}

	// Create the underlying PSO and then perform any other additional tasks like cleaning up/adding to caches, etc.
	PostCreateCallback(*OutPipelineState, Desc);
}

#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
FD3D12GraphicsPipelineState* FD3D12PipelineStateCacheBase::FindInRuntimeCache(const FGraphicsPipelineStateInitializer& Initializer, uint32& OutHash)
{
	OutHash = HashData(&Initializer, sizeof(Initializer));

	{
		FRWScopeLock Lock(InitializerToGraphicsPipelineMapMutex, FRWScopeLockType::SLT_ReadOnly);
		FD3D12GraphicsPipelineState** GraphicsPipelineState = InitializerToGraphicsPipelineMap.Find(FInitializerToGPSOMapKey(&Initializer, OutHash));
		if (GraphicsPipelineState)
		{
			INC_DWORD_STAT(STAT_PSOGraphicsHighlevelCacheHit);
			return *GraphicsPipelineState;
		}
	}

	INC_DWORD_STAT(STAT_PSOGraphicsHighlevelCacheMiss);
	return nullptr;
}
#endif

FD3D12GraphicsPipelineState* FD3D12PipelineStateCacheBase::FindInLoadedCache(
	const FGraphicsPipelineStateInitializer& Initializer,
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	uint32 InitializerHash,
#endif
	const FD3D12RootSignature* RootSignature,
	FD3D12LowLevelGraphicsPipelineStateDesc& OutLowLevelDesc)
{
	// TODO: For now PSOs will be created on every node of the LDA chain.
	OutLowLevelDesc = GetLowLevelGraphicsPipelineStateDesc(Initializer, RootSignature);
	OutLowLevelDesc.Desc.NodeMask = FRHIGPUMask::All().GetNative();

	OutLowLevelDesc.CombinedHash = FD3D12PipelineStateCacheBase::HashPSODesc(OutLowLevelDesc);

	// First try to find the PSO in the low level cache that can be populated from disk.
	FD3D12PipelineState* PipelineState = FindInLowLevelCache(OutLowLevelDesc);
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	if (PipelineState)
	{
		// Add the PSO to the runtime cache for better performance next time.
		return AddToRuntimeCache(Initializer, InitializerHash, RootSignature, PipelineState);
	}

	// TODO: Try to load from a PipelineLibrary now instead of at Create time.

	return nullptr;
#else
	if (PipelineState && PipelineState->IsValid())
	{
		return new FD3D12GraphicsPipelineState(Initializer, RootSignature, PipelineState);
	}

	return nullptr;
#endif
}

FD3D12GraphicsPipelineState* FD3D12PipelineStateCacheBase::CreateAndAdd(
	const FGraphicsPipelineStateInitializer& Initializer,
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	uint32 InitializerHash,
#endif
	const FD3D12RootSignature* RootSignature,
	const FD3D12LowLevelGraphicsPipelineStateDesc& LowLevelDesc)
{
	FD3D12PipelineState* const PipelineState = CreateAndAddToLowLevelCache(LowLevelDesc);
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	if (PipelineState == nullptr)
	{
		return nullptr;
	}

	// Add the PSO to the runtime cache for better performance next time.
	return AddToRuntimeCache(Initializer, InitializerHash, RootSignature, PipelineState);
#else
	if (PipelineState && PipelineState->IsValid())
	{
		return new FD3D12GraphicsPipelineState(Initializer, RootSignature, PipelineState);
	}

	return nullptr;
#endif
}

#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
FD3D12ComputePipelineState* FD3D12PipelineStateCacheBase::FindInRuntimeCache(const FD3D12ComputeShader* ComputeShader)
{
	{
		FRWScopeLock Lock(ComputeShaderToComputePipelineMapMutex, FRWScopeLockType::SLT_ReadOnly);
		FD3D12ComputePipelineState** ComputePipelineState = ComputeShaderToComputePipelineMap.Find(ComputeShader);
		if (ComputePipelineState)
		{
			INC_DWORD_STAT(STAT_PSOComputeHighlevelCacheHit);
			return *ComputePipelineState;
		}
	}

	INC_DWORD_STAT(STAT_PSOComputeHighlevelCacheMiss);
	return nullptr;
}
#endif

FD3D12ComputePipelineState* FD3D12PipelineStateCacheBase::FindInLoadedCache(const FComputePipelineStateInitializer& Initializer, const FD3D12RootSignature* RootSignature, FD3D12ComputePipelineStateDesc& OutLowLevelDesc)
{
	// TODO: For now PSOs will be created on every node of the LDA chain.
	OutLowLevelDesc = GetComputePipelineStateDesc(static_cast<FD3D12ComputeShader*>(Initializer.ComputeShader), RootSignature);
	OutLowLevelDesc.Desc.NodeMask = FRHIGPUMask::All().GetNative();
	OutLowLevelDesc.CombinedHash = FD3D12PipelineStateCacheBase::HashPSODesc(OutLowLevelDesc);

	// First try to find the PSO in the low level cache that can be populated from disk.
	FD3D12PipelineState* PipelineState = FindInLowLevelCache(OutLowLevelDesc);
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	if (PipelineState)
	{
		// Add the PSO to the runtime cache for better performance next time.
		return AddToRuntimeCache(ComputeShader, PipelineState);
	}

	// TODO: Try to load from a PipelineLibrary now instead of at Create time.

	return nullptr;
#else
	if (PipelineState && PipelineState->IsValid())
	{
		return new FD3D12ComputePipelineState(Initializer, RootSignature, PipelineState);
	}

	return nullptr;
#endif
}

FD3D12ComputePipelineState* FD3D12PipelineStateCacheBase::CreateAndAdd(const FComputePipelineStateInitializer& Initializer, const FD3D12RootSignature* RootSignature, const FD3D12ComputePipelineStateDesc& LowLevelDesc)
{
	FD3D12PipelineState* const PipelineState = CreateAndAddToLowLevelCache(LowLevelDesc);
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	// Add the PSO to the runtime cache for better performance next time.
	return AddToRuntimeCache(ComputeShader, PipelineState);
#else
	if (PipelineState && PipelineState->IsValid())
	{
		return new FD3D12ComputePipelineState(Initializer, RootSignature, PipelineState);
	}

	return nullptr;
#endif
}
