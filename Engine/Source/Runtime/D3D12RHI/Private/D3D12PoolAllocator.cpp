// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12PoolAllocator.h"
#include "D3D12RHIPrivate.h"
#include "HAL/LowLevelMemTracker.h"

LLM_DECLARE_TAG(D3D12AllocatorUnused);

//-----------------------------------------------------------------------------
//	FD3D12MemoryPool
//-----------------------------------------------------------------------------

FD3D12MemoryPool::FD3D12MemoryPool(
		  FD3D12Device* ParentDevice
		, FRHIGPUMask VisibleNodes
		, const FD3D12ResourceInitConfig& InInitConfig
		, const FString& InName
		, EResourceAllocationStrategy InAllocationStrategy
		, int16 InPoolIndex
		, uint64 InPoolSize
		, uint32 InPoolAlignment
		, ERHIPoolResourceTypes InSupportedResourceTypes
		, EFreeListOrder InFreeListOrder
		, HeapId InTraceParentHeapId
	)
	: FRHIMemoryPool(InPoolIndex, InPoolSize, InPoolAlignment, InSupportedResourceTypes, InFreeListOrder)
	, FD3D12DeviceChild(ParentDevice)
	, FD3D12MultiNodeGPUObject(ParentDevice->GetGPUMask(), VisibleNodes)
	, InitConfig(InInitConfig)
	, Name(InName)
	, AllocationStrategy(InAllocationStrategy)
	, LastUsedFrameFence(0)
{
#if UE_MEMORY_TRACE_ENABLED
	TraceHeapId = MemoryTrace_HeapSpec(InTraceParentHeapId, AllocationStrategy == EResourceAllocationStrategy::kPlacedResource ? TEXT("D3D12MemoryPool (PlacedResource)") : TEXT("D3D12MemoryPool (ManualSubAllocation)"));
#endif
}


FD3D12MemoryPool::~FD3D12MemoryPool()
{
}


void FD3D12MemoryPool::Init()
{
	if (PoolSize == 0)
	{
		return;
	}

	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	if (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource)
	{
		// Alignment should be either 4K or 64K for places resources
		check(PoolAlignment == D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT || PoolAlignment == D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT);

		D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InitConfig.HeapType);
		HeapProps.CreationNodeMask = GetGPUMask().GetNative();
		HeapProps.VisibleNodeMask = GetVisibilityMask().GetNative();

		D3D12_HEAP_DESC Desc = {};
		Desc.SizeInBytes = PoolSize;
		Desc.Properties = HeapProps;
		Desc.Alignment = 0;
		Desc.Flags = InitConfig.HeapFlags;

		// All all resources types on heap when tracking allocations (need to be able to create a buffer on it to extract the base gpu address)
		// (Tier 2 support already checked when this flag is enabled)
		if (Adapter->IsTrackingAllAllocations())
		{
			Desc.Flags = D3D12_HEAP_FLAG_ALLOW_ALL_BUFFERS_AND_TEXTURES;
		}

		if (Adapter->IsHeapNotZeroedSupported())
		{
			Desc.Flags |= FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED;
		}

		CA_SUPPRESS(6286);
		const bool bShouldTrackResource = (GNumExplicitGPUsForRendering == 1) || IsGPUOnly(HeapProps.Type);

#if ENABLE_RESIDENCY_MANAGEMENT
		if (!GD3D12StartResourceResident && bShouldTrackResource)
		{
			Desc.Flags |= D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT;
		}
#endif

		ID3D12Heap* Heap = nullptr;
		{
			LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

#if PLATFORM_WINDOWS
			// we are tracking allocations ourselves
			LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
#else
			LLM_SCOPE_BYTAG(D3D12AllocatorUnused);
#endif
			VERIFYD3D12RESULT(Adapter->GetD3DDevice()->CreateHeap(&Desc, IID_PPV_ARGS(&Heap)));
		}

		BackingHeap = new FD3D12Heap(GetParentDevice(), GetVisibilityMask(), TraceHeapId);
		BackingHeap->SetHeap(Heap, TEXT("PoolAllocator Heap"));

		if (bShouldTrackResource)
		{
			BackingHeap->BeginTrackingResidency(Desc.SizeInBytes);
		}
	}
	else
	{
		{
#if PLATFORM_WINDOWS
			LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
#else
			LLM_SCOPE_BYTAG(D3D12AllocatorUnused);
#endif
			const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InitConfig.HeapType, GetGPUMask().GetNative(), GetVisibilityMask().GetNative());
			VERIFYD3D12RESULT(Adapter->CreateBuffer(
				HeapProps,
				GetGPUMask(),
				InitConfig.InitialD3D12Access,
				ED3D12ResourceStateMode::SingleState,
				InitConfig.InitialD3D12Access,
				PoolSize,
				BackingResource.GetInitReference(),
				TEXT("Resource PoolAllocator Underlying Buffer"),
				InitConfig.ResourceFlags));

#if UE_MEMORY_TRACE_ENABLED
			MemoryTrace_MarkAllocAsHeap(BackingResource->GetGPUVirtualAddress(), TraceHeapId);
#endif
		}

		if (IsCPUAccessible(InitConfig.HeapType))
		{
			BackingResource->Map();
		}
	}	

#if D3D12_RHI_RAYTRACING
	// Elevates the raytracing acceleration structure heap priority, which may help performance / stability in low memory conditions.
	if (IsGPUOnly(InitConfig.HeapType)
		&& EnumHasAnyFlags(InitConfig.InitialD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite))
	{
		for (uint32 GPUIndex : GetGPUMask())
		{
			ID3D12Pageable* HeapResource = AllocationStrategy == EResourceAllocationStrategy::kPlacedResource
				? BackingHeap->GetHeap()
				: BackingResource->GetPageable();
			const D3D12_RESIDENCY_PRIORITY HeapPriority = D3D12_RESIDENCY_PRIORITY_HIGH;
			Adapter->SetResidencyPriority(HeapResource, HeapPriority, GPUIndex, PoolSize);
		}
	}
#endif // D3D12_RHI_RAYTRACING

#if PLATFORM_WINDOWS
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, int64(PoolSize), ELLMTracker::Platform, ELLMAllocType::System);
#endif
	FRHIMemoryPool::Init();
}


void FD3D12MemoryPool::Destroy()
{
#if PLATFORM_WINDOWS
	LLM_SCOPED_PAUSE_TRACKING_FOR_TRACKER(ELLMTracker::Default, ELLMAllocType::System);
#endif

	FRHIMemoryPool::Destroy();

	if (BackingResource)
	{
#if UE_MEMORY_TRACE_ENABLED
		MemoryTrace_UnmarkAllocAsHeap(BackingResource->GetGPUVirtualAddress(), TraceHeapId);
		MemoryTrace_Free(BackingResource->GetGPUVirtualAddress(), EMemoryTraceRootHeap::VideoMemory);
#endif
		ensure(BackingResource->GetRefCount() == 1 || GNumExplicitGPUsForRendering > 1);
		BackingResource = nullptr;
	}
}


//-----------------------------------------------------------------------------
//	FD3D12PoolAllocator
//-----------------------------------------------------------------------------


FD3D12ResourceInitConfig FD3D12PoolAllocator::GetResourceAllocatorInitConfig(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage)
{
	FD3D12ResourceInitConfig InitConfig;
	InitConfig.HeapType = InHeapType;
	InitConfig.ResourceFlags = InResourceFlags;

#if D3D12_RHI_RAYTRACING
	// Setup initial resource state depending on the requested buffer flags
	if (EnumHasAnyFlags(InBufferUsage, BUF_AccelerationStructure))
	{
		// should only have this flag and no other flags
		check(InBufferUsage == BUF_AccelerationStructure);
		InitConfig.InitialD3D12Access = ED3D12Access::BVHRead | ED3D12Access::BVHWrite;
	}
	else
#endif // D3D12_RHI_RAYTRACING
	if (InitConfig.HeapType == D3D12_HEAP_TYPE_READBACK)
	{
		InitConfig.InitialD3D12Access = ED3D12Access::CopyDest;
	}
	else if (EnumHasAnyFlags(InBufferUsage, BUF_UnorderedAccess))
	{
		check(InResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
		InitConfig.InitialD3D12Access = ED3D12Access::UAVMask;
	}
	else
	{
		InitConfig.InitialD3D12Access = ED3D12Access::GenericRead;
	}

	InitConfig.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS;
	if (EnumHasAnyFlags(InBufferUsage, BUF_DrawIndirect))
	{
		InitConfig.HeapFlags |= D3D12RHI_HEAP_FLAG_ALLOW_INDIRECT_BUFFERS;
	}

	return InitConfig;
}

EResourceAllocationStrategy FD3D12PoolAllocator::GetResourceAllocationStrategy(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment)
{
	if (Alignment > kD3D12ManualSubAllocationAlignment)
	{
		return EResourceAllocationStrategy::kPlacedResource;
	}

	// Does the resource need state tracking and transitions
	ED3D12ResourceStateMode ResourceStateMode = InResourceStateMode;
	if (ResourceStateMode == ED3D12ResourceStateMode::Default)
	{
		ResourceStateMode = (InResourceFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) ? ED3D12ResourceStateMode::MultiState : ED3D12ResourceStateMode::SingleState;
	}

	// multi state resource need to placed because each allocation can be in a different state
	return (ResourceStateMode == ED3D12ResourceStateMode::MultiState) ? EResourceAllocationStrategy::kPlacedResource : EResourceAllocationStrategy::kManualSubAllocation;
}


FD3D12PoolAllocator::FD3D12PoolAllocator(FD3D12Device* ParentDevice, FRHIGPUMask VisibleNodes, const FD3D12ResourceInitConfig& InInitConfig, const FString& InName,
	EResourceAllocationStrategy InAllocationStrategy, uint64 InDefaultPoolSize, uint32 InPoolAlignment, uint32 InMaxAllocationSize, FRHIMemoryPool::EFreeListOrder InFreeListOrder, bool bInDefragEnabled, HeapId InTraceParentHeapId, ERHIPoolAllocationStrategy InPoolAllocationStrategy, uint16 InOverFlowPoolsCount) :
	FRHIPoolAllocator(InDefaultPoolSize, InPoolAlignment, InMaxAllocationSize, InFreeListOrder, bInDefragEnabled, InPoolAllocationStrategy, InOverFlowPoolsCount),
	FD3D12DeviceChild(ParentDevice), 
	FD3D12MultiNodeGPUObject(ParentDevice->GetGPUMask(), VisibleNodes), 
	InitConfig(InInitConfig), 
	Name(InName), 
	AllocationStrategy(InAllocationStrategy)
{
#if UE_MEMORY_TRACE_ENABLED
	TraceHeapId = MemoryTrace_HeapSpec(InTraceParentHeapId, *Name);
#endif
}


FD3D12PoolAllocator::~FD3D12PoolAllocator()
{
	Destroy();
}


bool FD3D12PoolAllocator::SupportsAllocation(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment) const
{
#if WITH_MGPU
	// NNE resources must be in heaps visible on GPU0 only.  Required by DirectML.
	if (EnumHasAnyFlags(InBufferUsage, EBufferUsageFlags::NNE) && VisibilityMask != FRHIGPUMask::GPU0())
	{
		return false;
	}
#endif

	FD3D12ResourceInitConfig InInitConfig = GetResourceAllocatorInitConfig(InHeapType, InResourceFlags, InBufferUsage);
	EResourceAllocationStrategy InAllocationStrategy = GetResourceAllocationStrategy(InResourceFlags, InResourceStateMode, Alignment);

	// Don't check the resource flags & states for places resource but only the heap flags
	if (InAllocationStrategy == EResourceAllocationStrategy::kPlacedResource && AllocationStrategy == InAllocationStrategy)
	{
		return (InitConfig.HeapType == InInitConfig.HeapType && InitConfig.HeapFlags == InInitConfig.HeapFlags);
	}
	else
	{
		return (InitConfig == InInitConfig && AllocationStrategy == InAllocationStrategy);
	}
}


void FD3D12PoolAllocator::AllocDefaultResource(
	D3D12_HEAP_TYPE InHeapType,
	const FD3D12ResourceDesc& InDesc,
	EBufferUsageFlags InBufferUsage,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InCreateD3D12Access,
	uint32 InAllocationAlignment,
	const TCHAR* InName,
	FD3D12ResourceLocation& ResourceLocation)
{
#if DO_CHECK
	// Validate the create state
	if (InHeapType == D3D12_HEAP_TYPE_READBACK)
	{
		check(InCreateD3D12Access == ED3D12Access::CopyDest);
	}
	else if (InHeapType == D3D12_HEAP_TYPE_UPLOAD)
	{
		check(InCreateD3D12Access == ED3D12Access::GenericRead);
	}
	else if (InBufferUsage == BUF_UnorderedAccess && InResourceStateMode == ED3D12ResourceStateMode::SingleState)
	{
		check(InCreateD3D12Access == ED3D12Access::UAVMask);
	}
#if D3D12_RHI_RAYTRACING
	else if (EnumHasAnyFlags(InBufferUsage, BUF_AccelerationStructure))
	{
		// RayTracing acceleration structures must be created in a particular state and may never transition out of it.
		check(InResourceStateMode == ED3D12ResourceStateMode::SingleState);
		check(EnumHasAnyFlags(InCreateD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite));
	}
#endif // D3D12_RHI_RAYTRACING
#endif  // DO_CHECK

	AllocateResource(
		GetParentDevice()->GetGPUIndex(),
		InHeapType,
		InDesc,
		InDesc.Width,
		InAllocationAlignment,
		InCreateD3D12Access,
		InResourceStateMode,
		InCreateD3D12Access,
		nullptr,
		InName,
		ResourceLocation);
}


void FD3D12PoolAllocator::AllocateResource(
	uint32 GPUIndex,
	D3D12_HEAP_TYPE InHeapType,
	const FD3D12ResourceDesc& InDesc,
	uint64 InSize,
	uint32 InAllocationAlignment,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	const TCHAR* InName,
	FD3D12ResourceLocation& ResourceLocation)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::AllocatePoolResource);

	// If the resource location owns a block, this will deallocate it.
	ResourceLocation.Clear();
	if (InSize == 0)
	{
		return;
	}

	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	check(GPUIndex == Device->GetGPUIndex());

	const bool bSharedResource = EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS);
	
	bool bPoolResource = InSize <= MaxAllocationSize && !bSharedResource;
	bool bPlacedResource = bPoolResource && (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource);
	if (bPlacedResource)
	{
		// Make sure that we can satisfy the alignment requirements. If not, we have to fall back to a committed resource. This can happen for MSAA targets,
		// because the pools are aligned to D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, but MSAA needs more.
		if (InAllocationAlignment > PoolAlignment || InDesc.Alignment > PoolAlignment)
		{
			bPoolResource = false;
			bPlacedResource = false;
		}
	}
	   
	if (bPoolResource)
	{
		uint32 AllocationAlignment = InAllocationAlignment;

		// Ensure we're allocating from the correct pool
		if (bPlacedResource)
		{
			// If it's a placed resource then base offset will always be 0 from the actual d3d resource so ignore the allocation alignment - no extra offset required
			// for creating the views!
			AllocationAlignment = PoolAlignment;
		}
		else
		{
			// Read-only resources get suballocated from big resources, thus share ID3D12Resource* and resource state with other resources. Ensure it's suballocated from a resource with identical flags.
			check(InDesc.Flags == InitConfig.ResourceFlags);
		}

		// Find the correct allocation resource type
		ERHIPoolResourceTypes AllocationResourceType;
		if (InDesc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
		{
			AllocationResourceType = ERHIPoolResourceTypes::Buffers;
		}
		else if (EnumHasAnyFlags(InDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL))
		{
			AllocationResourceType = ERHIPoolResourceTypes::RTDSTextures;
		}
		else
		{
			AllocationResourceType = ERHIPoolResourceTypes::NonRTDSTextures;
		}

		// Try to allocate in one of the pools
		FRHIPoolAllocationData& AllocationData = ResourceLocation.GetPoolAllocatorPrivateData().PoolData;
		verify(TryAllocateInternal(InSize, AllocationAlignment, AllocationResourceType, AllocationData));

		// Setup the resource location
		ResourceLocation.SetType(FD3D12ResourceLocation::ResourceLocationType::eSubAllocation);
		ResourceLocation.SetPoolAllocator(this);
		ResourceLocation.SetSize(InSize);

		AllocationData.SetOwner(&ResourceLocation);

		if (AllocationStrategy == EResourceAllocationStrategy::kManualSubAllocation)
		{
			FD3D12Resource* BackingResource = GetBackingResource(ResourceLocation);

			ResourceLocation.SetOffsetFromBaseOfResource(AllocationData.GetOffset());
			ResourceLocation.SetResource(BackingResource);
			ResourceLocation.SetGPUVirtualAddress(BackingResource->GetGPUVirtualAddress() + AllocationData.GetOffset());

			if (IsCPUAccessible(InitConfig.HeapType))
			{
				ResourceLocation.SetMappedBaseAddress((uint8*)BackingResource->GetResourceBaseAddress() + AllocationData.GetOffset());
			}
		}
		else
		{
			check(ResourceLocation.GetResource() == nullptr);
			
			FD3D12ResourceDesc Desc = InDesc;
			Desc.Alignment = AllocationAlignment;

			FD3D12Resource* NewResource =
				CreatePlacedResource(
					AllocationData,
					Desc,
					InInitialD3D12Access,
					InResourceStateMode,
					InDefaultD3D12Access,
					InClearValue,
					InName);

			ResourceLocation.SetResource(NewResource);
			ResourceLocation.SetGPUVirtualAddress(NewResource->GetGPUVirtualAddress());
		}

		UpdateAllocationTracking(ResourceLocation, EAllocationType::Allocate);
	}
	else
	{
		FD3D12Resource* NewResource = nullptr;
		const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(InHeapType, GetGPUMask().GetNative(), GetVisibilityMask().GetNative());

		// If we are tracking all allocation data and allocating a standalone texture, then first create a heap so we can retrieve the GPU virtual address as well
		if (Adapter->IsTrackingAllAllocations() && (InDesc.Dimension != D3D12_RESOURCE_DIMENSION_BUFFER) && !bSharedResource)
		{
			D3D12_HEAP_DESC HeapDesc = {};
			HeapDesc.SizeInBytes = FMath::Max((uint64)D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, InSize);
			HeapDesc.Properties = HeapProps;
			HeapDesc.Alignment = InDesc.SampleDesc.Count > 1 ? D3D12_DEFAULT_MSAA_RESOURCE_PLACEMENT_ALIGNMENT : 0;
			HeapDesc.Flags = D3D12_HEAP_FLAG_ALLOW_ALL_BUFFERS_AND_TEXTURES;
			if (Adapter->IsHeapNotZeroedSupported())
			{
				HeapDesc.Flags |= FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED;
			}

			if (InDesc.Flags & D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS)
			{
				HeapDesc.Flags |= D3D12_HEAP_FLAG_SHARED;
			}

			CA_SUPPRESS(6286);
			const bool bShouldTrackResource = (GNumExplicitGPUsForRendering == 1) || IsGPUOnly(HeapProps.Type);

#if ENABLE_RESIDENCY_MANAGEMENT
			if (!GD3D12StartResourceResident && bShouldTrackResource)
			{
				HeapDesc.Flags |= D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT;
			}
#endif

			ID3D12Heap* Heap = nullptr;
			VERIFYD3D12RESULT(Adapter->GetD3DDevice()->CreateHeap(&HeapDesc, IID_PPV_ARGS(&Heap)));
			TRefCountPtr<FD3D12Heap> BackingHeap = new FD3D12Heap(GetParentDevice(), GetVisibilityMask(), TraceHeapId);
			bool bTrack = false;
			BackingHeap->SetHeap(Heap, InName, bTrack);

			if (bShouldTrackResource)
			{
				BackingHeap->BeginTrackingResidency(HeapDesc.SizeInBytes);
			}

			VERIFYD3D12RESULT(
				Adapter->CreatePlacedResource(
					InDesc,
					BackingHeap,
					0,
					InInitialD3D12Access,
					InResourceStateMode,
					InDefaultD3D12Access,
					InClearValue,
					&NewResource,
					InName));
		}
		else
		{
			// Allocate Standalone - move to owner of resource because this allocator should only manage pooled allocations (needed for now to do the same as FD3D12DefaultBufferPool)
			const ED3D12Access DefaultAccess =
				(InResourceStateMode == ED3D12ResourceStateMode::Default) ?
					ED3D12Access::Unknown :
					InDefaultD3D12Access;

			// Use default alignment for buffers because mights still be set to sub allocate alignment sizes which isn't valid
			FD3D12ResourceDesc Desc = InDesc;
			if (InDesc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
			{				
				Desc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
			}

			VERIFYD3D12RESULT(
				Adapter->CreateCommittedResource(
					Desc,
					GetGPUMask(),
					HeapProps,
					InInitialD3D12Access,
					InResourceStateMode,
					DefaultAccess,
					InClearValue,
					&NewResource,
					InName,
					false));
		}

		ResourceLocation.AsStandAlone(NewResource, InSize);
	}

#if D3D12_RHI_RAYTRACING
	// Elevates the raytracing acceleration structure heap priority, which may help performance / stability in low memory conditions.
	// When allocation is pooled, the entire pool priority is boosted on creation.
	if (EnumHasAnyFlags(InInitialD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite)
		&& ResourceLocation.GetAllocatorType() != FD3D12ResourceLocation::AT_Pool)
	{
		ID3D12Pageable* HeapResource = ResourceLocation.GetResource()->GetPageable();
		Adapter->SetResidencyPriority(HeapResource, D3D12_RESIDENCY_PRIORITY_HIGH, GPUIndex, InSize);
	}
#endif // D3D12_RHI_RAYTRACING
}


FD3D12Resource* FD3D12PoolAllocator::CreatePlacedResource(
	const FRHIPoolAllocationData& InAllocationData,
	const FD3D12ResourceDesc& InDesc,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	const TCHAR* InName)
{
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12HeapAndOffset HeapAndOffset = GetBackingHeapAndAllocationOffsetInBytes(InAllocationData);

	FD3D12Resource* NewResource = nullptr;
	VERIFYD3D12RESULT(
		Adapter->CreatePlacedResource(
			InDesc,
			HeapAndOffset.Heap,
			HeapAndOffset.Offset,
			InInitialD3D12Access,
			InResourceStateMode,
			InDefaultD3D12Access,
			InClearValue,
			&NewResource,
			InName));
	return NewResource;
}


void FD3D12PoolAllocator::DeallocateResource(FD3D12ResourceLocation& ResourceLocation, bool bDefragFree)
{
	check(IsOwner(ResourceLocation));

	UpdateAllocationTracking(ResourceLocation, bDefragFree ? EAllocationType::DefragFree : EAllocationType::Free);
	
	FScopeLock Lock(&CS);

	// Mark allocation data as free
	FRHIPoolAllocationData& AllocationData = ResourceLocation.GetPoolAllocatorPrivateData().PoolData;
	check(AllocationData.IsAllocated());

	// If locked then assume still initial setup or in defragmentation unlock request
	// Mark as nop because block will be deleted anyway
	// TODO: optimize to not perform the full iteration here!
	if (AllocationData.IsLocked())
	{
		for (FrameFencedAllocationData& Operation : FrameFencedOperations)
		{
			if (Operation.AllocationData == &AllocationData)
			{
				check(Operation.Operation == FrameFencedAllocationData::EOperation::Unlock);
				Operation.Operation = FrameFencedAllocationData::EOperation::Nop;
				Operation.AllocationData = nullptr;
				break;
			}
		}
		
		// If still pending copy then clear the copy operation data
		for (FD3D12VRAMCopyOperation& CopyOperation : PendingCopyOps)
		{
			if (CopyOperation.SrcResource == ResourceLocation.GetResource() ||
				CopyOperation.DstResource == ResourceLocation.GetResource())
			{
				CopyOperation.SrcResource = nullptr;
				CopyOperation.DstResource = nullptr;
				break;
			}
		}
	}

	int16 PoolIndex = AllocationData.GetPoolIndex();
	FRHIPoolAllocationData* ReleasedAllocationData = (AllocationDataPool.Num() > 0) ? AllocationDataPool.Pop(EAllowShrinking::No) : new FRHIPoolAllocationData();
	bool bLocked = true;
	ReleasedAllocationData->MoveFrom(AllocationData, bLocked);

	// Clear the allocator data
	ResourceLocation.ClearAllocator();

	// Store fence when last used so we know when to unlock the free data
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	FrameFencedAllocationData& DeleteRequest = FrameFencedOperations.Emplace_GetRef();
	DeleteRequest.Operation = FrameFencedAllocationData::EOperation::Deallocate;
	DeleteRequest.FrameFence = FrameFence.GetNextFenceToSignal();
	DeleteRequest.AllocationData = ReleasedAllocationData;

	PendingDeleteRequestSize += DeleteRequest.AllocationData->GetSize();

	// Update the last used frame fence (used during garbage collection)
	((FD3D12MemoryPool*)Pools[PoolIndex])->UpdateLastUsedFrameFence(DeleteRequest.FrameFence);

	// Also store the placed resource so it can be correctly freed when fence is done
	if (ResourceLocation.GetResource()->IsPlacedResource())
	{
		DeleteRequest.PlacedResource = ResourceLocation.GetResource();
	}
	else
	{
		DeleteRequest.PlacedResource = nullptr;
	}
}


FRHIMemoryPool* FD3D12PoolAllocator::CreateNewPool(int16 InPoolIndex, uint32 InMinimumAllocationSize, ERHIPoolResourceTypes InAllocationResourceType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12PoolAllocator::CreateNewPool);

	// Find out the pool size - use default, but if allocation doesn't fit then round up to next power of 2
	// so it 'always' fits the pool allocator
	uint32 PoolSize = DefaultPoolSize;
	if (InMinimumAllocationSize > PoolSize)
	{
		check(InMinimumAllocationSize <= MaxAllocationSize);
		PoolSize = FMath::Min(FMath::RoundUpToPowerOfTwo(InMinimumAllocationSize), (uint32)MaxAllocationSize);
	}

	FD3D12MemoryPool* NewPool = new FD3D12MemoryPool(GetParentDevice(),	GetVisibilityMask(), InitConfig,
		Name, AllocationStrategy, InPoolIndex, PoolSize, PoolAlignment, InAllocationResourceType, FreeListOrder, TraceHeapId);
	NewPool->Init();
	return NewPool;
}


bool FD3D12PoolAllocator::HandleDefragRequest(FRHIContextArray const& Contexts, FRHIPoolAllocationData* InSourceBlock, FRHIPoolAllocationData& InTmpTargetBlock)
{
	// Cache source copy data
	FD3D12ResourceLocation* Owner = (FD3D12ResourceLocation*)InSourceBlock->GetOwner();
	uint64 CurrentOffset = Owner->GetOffsetFromBaseOfResource();
	FD3D12Resource* CurrentResource = Owner->GetResource();

	ED3D12Access SrcD3D12Access = ED3D12Access::Unknown;
	if (CurrentResource->RequiresResourceStateTracking())
	{
		// State tracking is happening at the RHI high level 
		FD3D12Texture* CurrentResourceRHITexture = nullptr;
		if (Owner->Owner)
		{
			if (CurrentResource->GetDesc().Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
			{
				FD3D12Buffer* CurrentResourceRHIBuffer = (FD3D12Buffer*)Owner->Owner;
				SrcD3D12Access = ConvertToD3D12Access(CurrentResourceRHIBuffer->GetTrackedAccess_Unsafe());
			}
			else
			{
				CurrentResourceRHITexture = (FD3D12Texture*)Owner->Owner;
				SrcD3D12Access = ConvertToD3D12Access(CurrentResourceRHITexture->GetTrackedAccess_Unsafe());
			}
		}

		check(SrcD3D12Access != ED3D12Access::Unknown);
	}
	else
	{
		//If the resource is a single state type and doesn't need state tracking set the access as Unknown so the scoped transition in FlushPendingCopyOps can revert back to the default state
		check(SrcD3D12Access == ED3D12Access::Unknown);
	}

	// Release the current allocation (will only be freed on the next frame fence)
	bool bDefragFree = true;
	DeallocateResource(*Owner, bDefragFree);

	// Move temp allocation data to allocation data of the owner (part of different allocator now)		
	bool bLocked = true;
	InSourceBlock->MoveFrom(InTmpTargetBlock, bLocked);
	InSourceBlock->SetOwner(Owner);
	Owner->SetPoolAllocator(this);

	// Notify owner of moved allocation data (recreated resources and SRVs if needed)
	ED3D12Access DstD3D12Access = ED3D12Access::Unknown;
	Owner->OnAllocationMoved(Contexts, InSourceBlock, DstD3D12Access);

	check(IsValidAccess(DstD3D12Access));
	if (!Owner->GetResource()->RequiresResourceStateTracking())
	{
		//If the resource is a single state type and doesn't need state tracking set the state as TBD so the scoped transition in FlushPendingCopyOps can revert back to the default state
		DstD3D12Access = ED3D12Access::Unknown;
	}

	// Add request to unlock the source block on the next fence value (copy operation should have been done by then)
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	FrameFencedAllocationData& UnlockRequest = FrameFencedOperations.Emplace_GetRef();
	UnlockRequest.Operation = FrameFencedAllocationData::EOperation::Unlock;
	UnlockRequest.FrameFence = FrameFence.GetNextFenceToSignal();
	UnlockRequest.AllocationData = InSourceBlock;

	// Schedule a copy operation of the actual data
	FD3D12VRAMCopyOperation CopyOp = {
		.SrcResource          = CurrentResource,
		.SrcResourcED3D12Access = SrcD3D12Access,
		.SrcOffset            = static_cast<uint32>(CurrentOffset),
		.DstResource          = Owner->GetResource(),
		.DstResourcED3D12Access = DstD3D12Access,
		.DstOffset            = static_cast<uint32>(Owner->GetOffsetFromBaseOfResource()),
		.Size		          = InSourceBlock->GetSize(),
		.CopyType	          =
			(AllocationStrategy == EResourceAllocationStrategy::kManualSubAllocation) ?
							FD3D12VRAMCopyOperation::ECopyType::BufferRegion :
							FD3D12VRAMCopyOperation::ECopyType::Resource,
	};
	check(CopyOp.SrcResource != nullptr);
	check(CopyOp.DstResource != nullptr);
	PendingCopyOps.Add(MoveTemp(CopyOp));

	UpdateAllocationTracking(*Owner, EAllocationType::Allocate);

	// TODO: Using aliasing buffer on whole heap for copies to reduce flushes and resource transitions

	return true;
}


void FD3D12PoolAllocator::CleanUpAllocations(uint64 InFrameLag, bool bForceFree)
{
	FScopeLock Lock(&CS);

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	uint32 PopCount = 0;
	for (int32 i = 0; i < FrameFencedOperations.Num(); i++)
	{
		FrameFencedAllocationData& Operation = FrameFencedOperations[i];
		if (bForceFree || FrameFence.IsFenceComplete(Operation.FrameFence, /* bUpdateCachedFenceValue */ false))
		{
			switch (Operation.Operation)
			{
			case FrameFencedAllocationData::EOperation::Deallocate:
			{
				// Not pending anymore
				check(PendingDeleteRequestSize >= Operation.AllocationData->GetSize());
				PendingDeleteRequestSize -= Operation.AllocationData->GetSize();
				// Deallocate the locked block (actually free now)
				DeallocateInternal(*Operation.AllocationData);
				Operation.AllocationData->Reset();
				AllocationDataPool.Add(Operation.AllocationData);

				// Free placed resource if created
				if (AllocationStrategy == EResourceAllocationStrategy::kPlacedResource)
				{
					// Release the resource
					check(Operation.PlacedResource != nullptr);
					Operation.PlacedResource->Release();
					Operation.PlacedResource = nullptr;
				}
				else
				{
					check(Operation.PlacedResource == nullptr);
				}
				break;
			}
			case FrameFencedAllocationData::EOperation::Unlock:
			{
				Operation.AllocationData->Unlock();
				break;
			}
			case FrameFencedAllocationData::EOperation::Nop:
			{
				break;
			}
			default: check(false);
			}

			PopCount = i + 1;
		}
		else
		{
			break;
		}
	}

	if (PopCount)
	{
		// clear out all of the released blocks, don't allow the array to shrink
		FrameFencedOperations.RemoveAt(0, PopCount, EAllowShrinking::No);
	}

	// Trim empty allocators if not used in last n frames
	const uint64 CompletedFence = FrameFence.GetCompletedFenceValue(/* bUpdateCachedFenceValue */ true);
	for (int32 PoolIndex = 0; PoolIndex < Pools.Num(); ++PoolIndex)
	{
		FD3D12MemoryPool* MemoryPool = (FD3D12MemoryPool*) Pools[PoolIndex];
		if (MemoryPool != nullptr && MemoryPool->IsEmpty() && (bForceFree || (MemoryPool->GetLastUsedFrameFence() + InFrameLag <= CompletedFence)))
		{
#if PLATFORM_WINDOWS
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - int64(MemoryPool->GetPoolSize()), ELLMTracker::Platform, ELLMAllocType::System);
#endif
			MemoryPool->Destroy();
			delete(MemoryPool);
			Pools[PoolIndex] = nullptr;
		}
	}

	// Sort the pools
	SortPools();
}


void FD3D12PoolAllocator::TransferOwnership(FD3D12ResourceLocation& InSource, FD3D12ResourceLocation& InDest)
{
	FScopeLock Lock(&CS);

	check(IsOwner(InSource));

	// Note: mirrors FD3D12ResourceLocation::TransferOwnership but is inside our CS.
	FMemory::Memmove(&InDest, &InSource, sizeof(FD3D12ResourceLocation));

	FRHIPoolAllocationData& SourcePoolData = InSource.GetPoolAllocatorPrivateData().PoolData;
	FRHIPoolAllocationData& DestinationPoolData = InDest.GetPoolAllocatorPrivateData().PoolData;

	// If locked then assume the block is currently being defragged and then the frame fenced operation
	// also needs to be updated
	if (SourcePoolData.IsLocked())
	{
		bool bFound = false;
		for (FrameFencedAllocationData& Operation : FrameFencedOperations)
		{
			if (Operation.AllocationData == &SourcePoolData)
			{
				check(Operation.Operation == FrameFencedAllocationData::EOperation::Unlock);
				Operation.AllocationData = &DestinationPoolData;
				bFound = true;
				break;
			}
		}
		check(bFound);
	}

	// Take over lock as well - can be defragging
	bool bLocked = SourcePoolData.IsLocked();
	DestinationPoolData.MoveFrom(InSource.GetPoolAllocatorPrivateData().PoolData, bLocked);
	DestinationPoolData.SetOwner(&InDest);
}

FD3D12Resource* FD3D12PoolAllocator::GetBackingResource(FD3D12ResourceLocation& InResourceLocation) const
{
	check(IsOwner(InResourceLocation));
	FRHIPoolAllocationData& AllocationData = InResourceLocation.GetPoolAllocatorPrivateData().PoolData;
	return ((FD3D12MemoryPool*)Pools[AllocationData.GetPoolIndex()])->GetBackingResource();
}


FD3D12HeapAndOffset FD3D12PoolAllocator::GetBackingHeapAndAllocationOffsetInBytes(FD3D12ResourceLocation& InResourceLocation) const
{
	check(IsOwner(InResourceLocation));

	return GetBackingHeapAndAllocationOffsetInBytes(InResourceLocation.GetPoolAllocatorPrivateData().PoolData);
}


FD3D12HeapAndOffset FD3D12PoolAllocator::GetBackingHeapAndAllocationOffsetInBytes(const FRHIPoolAllocationData& InAllocationData) const
{
	FD3D12HeapAndOffset HeapAndOffset;
	HeapAndOffset.Heap = ((FD3D12MemoryPool*)Pools[InAllocationData.GetPoolIndex()])->GetBackingHeap();
	HeapAndOffset.Offset = uint64(AlignDown(InAllocationData.GetOffset(), PoolAlignment));
	return HeapAndOffset;
}


void FD3D12PoolAllocator::UpdateAllocationTracking(FD3D12ResourceLocation& InAllocation, EAllocationType InAllocationType)
{
#if TRACK_RESOURCE_ALLOCATIONS
	check(IsOwner(InAllocation));
	FRHIPoolAllocationData& AllocationData = InAllocation.GetPoolAllocatorPrivateData().PoolData;
	check(AllocationData.IsAllocated());
	uint64 AllocationSize = AllocationData.GetSize();

	// Track only when tracking all resources is set
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	if (InitConfig.HeapType == D3D12_HEAP_TYPE_DEFAULT && AllocationStrategy == EResourceAllocationStrategy::kPlacedResource && Adapter->IsTrackingAllAllocations())
	{
		if (InAllocationType == EAllocationType::Allocate)
		{
			bool bCollectCallstack = false;
			Adapter->TrackAllocationData(&InAllocation, AllocationSize, bCollectCallstack);
		}
		else
		{
			bool bDefragFree = InAllocationType == EAllocationType::DefragFree;
			Adapter->ReleaseTrackedAllocationData(&InAllocation, bDefragFree);
		}
	}
#endif // TRACK_RESOURCE_ALLOCATIONS

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	{
		int64 AllocSize = int64(InAllocation.GetPoolAllocatorPrivateData().PoolData.GetSize());
		
		if (InAllocationType == EAllocationType::Allocate)
		{
#if PLATFORM_WINDOWS
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - AllocSize, ELLMTracker::Platform, ELLMAllocType::System);
#else
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, InAllocation.GetAddressForLLMTracking(), AllocSize));
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, 0 - AllocSize, ELLMTracker::Default, ELLMAllocType::System);
#endif
		}
		else
		{
#if PLATFORM_WINDOWS
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, AllocSize, ELLMTracker::Platform, ELLMAllocType::System);
#else
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, InAllocation.GetAddressForLLMTracking()));
			LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT_BYTAG(D3D12AllocatorUnused, AllocSize, ELLMTracker::Default, ELLMAllocType::System);
#endif
		}
	}
#endif
}


void FD3D12PoolAllocator::FlushPendingCopyOps(FD3D12CommandContext& InCommandContext)
{
	FScopeLock Lock(&CS);

	// TODO: sort the copy ops to reduce amount of transitions!!

	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	FD3D12ManualFence& FrameFence = Adapter->GetFrameFence();

	for (FD3D12VRAMCopyOperation& CopyOperation : PendingCopyOps)
	{
		// Cleared copy op?
		if (CopyOperation.SrcResource == nullptr || CopyOperation.DstResource == nullptr)
		{
			continue;
		}

#if D3D12_RHI_RAYTRACING
		if (!CopyOperation.SrcResource->RequiresResourceStateTracking() && 
			EnumHasAnyFlags(CopyOperation.SrcResource->GetDesc().Flags, D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE))
		{
			// can't make state changes to RT resources
			check(EnumHasAnyFlags(CopyOperation.DstResource->GetDesc().Flags, D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE));

			D3D12_GPU_VIRTUAL_ADDRESS SrcAddress = CopyOperation.SrcResource->GetResource()->GetGPUVirtualAddress() + CopyOperation.SrcOffset;
			D3D12_GPU_VIRTUAL_ADDRESS DestAddress = CopyOperation.DstResource->GetResource()->GetGPUVirtualAddress() + CopyOperation.DstOffset;
			InCommandContext.RayTracingCommandList()->CopyRaytracingAccelerationStructure(DestAddress, SrcAddress, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_COPY_MODE_CLONE);
		}
		else
#endif // D3D12_RHI_RAYTRACING
		{

			{
				FScopedResourceBarrier ScopedResourceBarrierSrc(
					InCommandContext,
					CopyOperation.SrcResource,
					CopyOperation.SrcResourcED3D12Access,
					ED3D12Access::CopySrc,
					D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
				FScopedResourceBarrier ScopedResourceBarrierDst(
					InCommandContext,
					CopyOperation.DstResource,
					CopyOperation.DstResourcED3D12Access,
					ED3D12Access::CopyDest,
					D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);

				InCommandContext.FlushResourceBarriers();

				switch (CopyOperation.CopyType)
				{
				case FD3D12VRAMCopyOperation::BufferRegion:
				{
					InCommandContext.CopyBufferRegionChecked(
						CopyOperation.DstResource->GetResource(), CopyOperation.DstResource->GetName(),
						CopyOperation.DstOffset,
						CopyOperation.SrcResource->GetResource(), CopyOperation.SrcResource->GetName(),
						CopyOperation.SrcOffset,
						CopyOperation.Size
					);
					break;
				}
				case FD3D12VRAMCopyOperation::Resource:
				{
					InCommandContext.GraphicsCommandList()->CopyResource(
						CopyOperation.DstResource->GetResource(),
						CopyOperation.SrcResource->GetResource()
					);
					break;
				}
				}
			}

			// if the resource need tracking we are setting the state of the destination equal to the original source state
			if (CopyOperation.SrcResource->RequiresResourceStateTracking())
			{
				check(CopyOperation.DstResource->RequiresResourceStateTracking());
				check(IsValidAccess(CopyOperation.SrcResourcED3D12Access));

				if (CopyOperation.SrcResourcED3D12Access != CopyOperation.DstResourcED3D12Access)
				{
					InCommandContext.AddBarrier(
						CopyOperation.DstResource,
						CopyOperation.DstResourcED3D12Access,
						CopyOperation.SrcResourcED3D12Access,
						D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
				}
			}
		}

		InCommandContext.UpdateResidency(CopyOperation.SrcResource);
		InCommandContext.UpdateResidency(CopyOperation.DstResource);
	}

	InCommandContext.ConditionalSplitCommandList();

	PendingCopyOps.Empty(PendingCopyOps.Num());
}

