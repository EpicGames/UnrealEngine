// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHIPoolAllocator.h"
#include "D3D12Resources.h"

enum class EResourceAllocationStrategy
{
	// This strategy uses Placed Resources to sub-allocate a buffer out of an underlying ID3D12Heap.
	// The benefit of this is that each buffer can have it's own resource state and can be treated
	// as any other buffer. The downside of this strategy is the API limitation which enforces
	// the minimum buffer size to 64k leading to large internal fragmentation in the allocator
	kPlacedResource,
	// The alternative is to manually sub-allocate out of a single large buffer which allows block
	// allocation granularity down to 1 byte. However, this strategy is only really valid for buffers which
	// will be treated as read-only after their creation (i.e. most Index and Vertex buffers). This 
	// is because the underlying resource can only have one state at a time.
	kManualSubAllocation
};


// All data required to create a pool
struct FD3D12ResourceInitConfig
{
	static FD3D12ResourceInitConfig CreateUpload()
	{
		FD3D12ResourceInitConfig Config;
		Config.HeapType = D3D12_HEAP_TYPE_UPLOAD;
		Config.HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS;
		Config.ResourceFlags = D3D12_RESOURCE_FLAG_NONE;
		Config.InitialD3D12Access = ED3D12Access::GenericRead;
		return Config;
	}

	bool operator==(const FD3D12ResourceInitConfig& InOther) const
	{
		return HeapType == InOther.HeapType &&
			HeapFlags == InOther.HeapFlags &&
			ResourceFlags == InOther.ResourceFlags &&
			InitialD3D12Access   == InOther.InitialD3D12Access;
	}

	D3D12_HEAP_TYPE HeapType;
	D3D12_HEAP_FLAGS HeapFlags;
	D3D12_RESOURCE_FLAGS ResourceFlags;
	ED3D12Access InitialD3D12Access;
};


/**
@brief Stores all required data for a VRAM copy operation - doesn't own the resources
**/
struct FD3D12VRAMCopyOperation
{
	enum ECopyType
	{
		BufferRegion,
		Resource,
	};

	FD3D12Resource* SrcResource;
	ED3D12Access SrcResourcED3D12Access;
	uint32 SrcOffset;
	FD3D12Resource* DstResource;
	ED3D12Access DstResourcED3D12Access;
	uint32 DstOffset;
	uint32 Size;
	ECopyType CopyType;
};

// Heap and offset combined to specific location of a resource
struct FD3D12HeapAndOffset
{
	FD3D12Heap* Heap;
	uint64 Offset;
};


//////////////////////////////////////////////////////////////////////////
// FD3D12Pool
//////////////////////////////////////////////////////////////////////////

/**
@brief D3D12 specific implementation of the FRHIMemoryPool
**/
class FD3D12MemoryPool : public FRHIMemoryPool, public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:

	// Constructor
	FD3D12MemoryPool(FD3D12Device* ParentDevice, FRHIGPUMask VisibleNodes, const FD3D12ResourceInitConfig& InInitConfig, const FString& Name,
		EResourceAllocationStrategy InAllocationStrategy, int16 InPoolIndex, uint64 InPoolSize, uint32 InPoolAlignment, ERHIPoolResourceTypes InSupportedResourceTypes, EFreeListOrder InFreeListOrder, HeapId InTraceParentHeapId);
	virtual ~FD3D12MemoryPool();

	// Setup/Shutdown
	virtual void Init() override;
	virtual void Destroy() override;

	FD3D12Resource* GetBackingResource() { check(AllocationStrategy == EResourceAllocationStrategy::kManualSubAllocation); return BackingResource.GetReference(); }
	FD3D12Heap* GetBackingHeap() { check(AllocationStrategy == EResourceAllocationStrategy::kPlacedResource); return BackingHeap.GetReference(); }

	uint64 GetLastUsedFrameFence() const { return LastUsedFrameFence; }
	void UpdateLastUsedFrameFence(uint64 InFrameFence) { LastUsedFrameFence = FMath::Max(LastUsedFrameFence, InFrameFence); }

protected:

	const FD3D12ResourceInitConfig InitConfig;
	const FString Name;
	EResourceAllocationStrategy AllocationStrategy;

	// Actual D3D resource - either resource or heap (see EResourceAllocationStrategy)
	TRefCountPtr<FD3D12Resource> BackingResource;
	TRefCountPtr<FD3D12Heap> BackingHeap;

	uint64 LastUsedFrameFence;

private:
	HeapId TraceHeapId;
};


//////////////////////////////////////////////////////////////////////////
// FD3D12PoolAllocator
//////////////////////////////////////////////////////////////////////////

/**
@brief D3D12 specific implementation of the FRHIPoolAllocator
**/
class FD3D12PoolAllocator : public FRHIPoolAllocator, public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject, public ID3D12ResourceAllocator
{
public:

	// Constructor
	FD3D12PoolAllocator(FD3D12Device* ParentDevice, FRHIGPUMask VisibleNodes, const FD3D12ResourceInitConfig& InInitConfig, const FString& InName,
		EResourceAllocationStrategy InAllocationStrategy, uint64 InDefaultPoolSize, uint32 InPoolAlignment, uint32 InMaxAllocationSize, FRHIMemoryPool::EFreeListOrder InFreeListOrder, bool bInDefragEnabled, HeapId InTraceParentHeapId, ERHIPoolAllocationStrategy InPoolAllocationStrategy, uint16 InOverFlowPoolsCount);
	~FD3D12PoolAllocator();
	
	// Function names currently have to match with FD3D12DefaultBufferPool until we can make full replacement of the allocator
	bool SupportsAllocation(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment) const;
	void AllocDefaultResource(
		D3D12_HEAP_TYPE InHeapType,
		const FD3D12ResourceDesc& InDesc,
		EBufferUsageFlags InBufferUsage,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InCreateD3D12Access,
		uint32 InAlignment,
		const TCHAR* InName,
		FD3D12ResourceLocation& ResourceLocation);
	virtual void DeallocateResource(FD3D12ResourceLocation& ResourceLocation, bool bDefragFree = false);

	// Implementation of ID3D12ResourceAllocator
	virtual void AllocateResource(
		uint32 GPUIndex,
		D3D12_HEAP_TYPE InHeapType,
		const FD3D12ResourceDesc& InDesc,
		uint64 InSize,
		uint32 InAllocationAlignment,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		const TCHAR* InName,
		FD3D12ResourceLocation& ResourceLocation) override;

	void CleanUpAllocations(uint64 InFrameLag, bool bForceFree = false);
	void FlushPendingCopyOps(FD3D12CommandContext& InCommandContext);

	void TransferOwnership(FD3D12ResourceLocation& InSource, FD3D12ResourceLocation& InDest);
	bool IsOwner(FD3D12ResourceLocation& ResourceLocation) const { return ResourceLocation.GetPoolAllocator() == this; }

	EResourceAllocationStrategy GetAllocationStrategy() const { return AllocationStrategy; }
	FD3D12Resource* GetBackingResource(FD3D12ResourceLocation& InResourceLocation) const;
	FD3D12HeapAndOffset GetBackingHeapAndAllocationOffsetInBytes(FD3D12ResourceLocation& InResourceLocation) const;
	FD3D12HeapAndOffset GetBackingHeapAndAllocationOffsetInBytes(const FRHIPoolAllocationData& InAllocationData) const;
	uint64 GetPendingDeleteRequestSize() const { return PendingDeleteRequestSize; }

	static FD3D12ResourceInitConfig GetResourceAllocatorInitConfig(D3D12_HEAP_TYPE InHeapType, D3D12_RESOURCE_FLAGS InResourceFlags, EBufferUsageFlags InBufferUsage);
	static EResourceAllocationStrategy GetResourceAllocationStrategy(D3D12_RESOURCE_FLAGS InResourceFlags, ED3D12ResourceStateMode InResourceStateMode, uint32 Alignment);
	
protected:

	// Implementation of FRHIPoolAllocator pure virtuals
	virtual FRHIMemoryPool* CreateNewPool(int16 InPoolIndex, uint32 InMinimumAllocationSize, ERHIPoolResourceTypes InAllocationResourceType) override;
	virtual bool HandleDefragRequest(FRHIContextArray const& Contexts, FRHIPoolAllocationData* InSourceBlock, FRHIPoolAllocationData& InTmpTargetBlock) override;
	
	// Placed resource allocation helper function which can be overriden
	virtual FD3D12Resource* CreatePlacedResource(
		const FRHIPoolAllocationData& InAllocationData,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		const TCHAR* InName);
	
	// Track the allocation
	enum class EAllocationType
	{
		Allocate,		// Update tracking data on allocation
		DefragFree,		// Update tracking data on free during defrag op
		Free			// Update tracking data when resource is actually freed
	};
	virtual void UpdateAllocationTracking(FD3D12ResourceLocation& InAllocation, EAllocationType InAllocationType);

	// Locked allocation data which needs to do something specific at certain frame fence value	
	struct FrameFencedAllocationData
	{
		enum class EOperation
		{
			Invalid,
			Deallocate,
			Unlock,
			Nop,
		};

		EOperation Operation = EOperation::Invalid;
		FD3D12Resource* PlacedResource = nullptr;
		uint64 FrameFence = 0;
		FRHIPoolAllocationData* AllocationData = nullptr;
	};

	const FD3D12ResourceInitConfig InitConfig;
	const FString Name;
	EResourceAllocationStrategy AllocationStrategy;

	// All operations which need to happen on specific frame fences
	TArray<FrameFencedAllocationData> FrameFencedOperations;

	// Size of all pending delete requests in the frame fence operation array
	uint64 PendingDeleteRequestSize = 0;

	// Pending copy ops which need to be flush this frame
	TArray<FD3D12VRAMCopyOperation> PendingCopyOps;

	// Allocation data pool used to reduce allocation overhead
	TArray<FRHIPoolAllocationData*> AllocationDataPool;

private:
	HeapId TraceHeapId;
};

