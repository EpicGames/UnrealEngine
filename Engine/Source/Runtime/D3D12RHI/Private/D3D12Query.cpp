// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Query.cpp: D3D query RHI implementation.
=============================================================================*/

#include "D3D12Query.h"
#include "D3D12RHIPrivate.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHIStrings.h"

namespace D3D12RHI
{
	/**
	* RHI console variables used by queries.
	*/
	namespace RHIConsoleVariables
	{
		int32 GInsertOuterOcclusionQuery = 0;
		static FAutoConsoleVariableRef CVarInsertOuterOcclusionQuery(
			TEXT("D3D12.InsertOuterOcclusionQuery"),
			GInsertOuterOcclusionQuery,
			TEXT("If true, enable a dummy outer occlusion query around occlusion query batches. Can help performance on some GPU architectures"),
			ECVF_Default
		);
	}
}
using namespace D3D12RHI;

FD3D12RenderQuery::FD3D12RenderQuery(FD3D12Device* Parent, ERenderQueryType InQueryType)
	: FD3D12DeviceChild(Parent)
	, Type(InQueryType)
	, Result((uint64*)FMemory::Malloc(sizeof(uint64), alignof(uint64)))
{}

FD3D12RenderQuery::~FD3D12RenderQuery()
{
	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(Result, FD3D12DeferredDeleteObject::EType::CPUAllocation);
	Result = nullptr;
}

FD3D12QueryHeap::FD3D12QueryHeap(FD3D12Device* Device, D3D12_QUERY_TYPE QueryType, D3D12_QUERY_HEAP_TYPE HeapType)
	: FD3D12SingleNodeGPUObject(Device->GetGPUMask())
	, Device(Device)
	, QueryType(QueryType)
	, HeapType(HeapType)
	, NumQueries(MaxHeapSize / GetResultSize())
{
	INC_DWORD_STAT(STAT_D3D12NumQueryHeaps);

	TCHAR const *QueryHeapName, *ResultBufferName;

	switch (HeapType)
	{
	default: checkNoEntry(); [[fallthrough]];
	case D3D12_QUERY_HEAP_TYPE_OCCLUSION:
		QueryHeapName    = TEXT("Occlusion Query Heap");
		ResultBufferName = TEXT("Occlusion Query Heap Result Buffer");
		break;

	case D3D12_QUERY_HEAP_TYPE_TIMESTAMP:
		QueryHeapName    = TEXT("Timestamp Query Heap");
		ResultBufferName = TEXT("Timestamp Query Heap Result Buffer");
		break;

	case D3D12_QUERY_HEAP_TYPE_COPY_QUEUE_TIMESTAMP:
		QueryHeapName    = TEXT("Timestamp Query Heap (Copy)");
		ResultBufferName = TEXT("Timestamp Query Heap Result Buffer (Copy)");
		break;

	case D3D12_QUERY_HEAP_TYPE_PIPELINE_STATISTICS:
		QueryHeapName    = TEXT("Pipeline Statistics Query Heap (Copy)");
		ResultBufferName = TEXT("Pipeline Statistics Query Heap Result Buffer (Copy)");
		break;
	}

	const static FLazyName D3D12QueryHeapName(TEXT("FD3D12QueryHeap"));
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(FName(ResultBufferName), D3D12QueryHeapName, NAME_None);

	// Create the query heap
#if D3D12RHI_PLATFORM_USES_TIMESTAMP_QUERIES == 0
	if (HeapType != D3D12_QUERY_HEAP_TYPE_TIMESTAMP)
#endif
	{
		D3D12_QUERY_HEAP_DESC QueryHeapDesc = {};
		QueryHeapDesc.Type = HeapType;
		QueryHeapDesc.Count = NumQueries;
		QueryHeapDesc.NodeMask = GetGPUMask().GetNative();

		VERIFYD3D12RESULT(Device->GetDevice()->CreateQueryHeap(&QueryHeapDesc, IID_PPV_ARGS(D3DQueryHeap.GetInitReference())));
		SetD3D12ObjectName(D3DQueryHeap, QueryHeapName);

#if ENABLE_RESIDENCY_MANAGEMENT && 0 // Temporary workaround for missing resource usage tracking for query heap
		D3DX12Residency::Initialize(ResidencyHandle, D3DQueryHeap, GetResultSize() * NumQueries, this);
		D3DX12Residency::BeginTrackingObject(Device->GetResidencyManager(), ResidencyHandle);
#endif
	}

	// Create the readback heap to hold the resolved results
	{
		D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(
			D3D12_HEAP_TYPE_READBACK,
			GetGPUMask().GetNative(),
			GetVisibilityMask().GetNative());
			
		D3D12_RESOURCE_DESC BufferDesc = CD3DX12_RESOURCE_DESC::Buffer(GetResultSize() * NumQueries);

		// Create the readback heap
		VERIFYD3D12RESULT(
			Device->GetParentAdapter()->CreateCommittedResource(
				BufferDesc,
				GetGPUMask(),
				HeapProps,
				ED3D12Access::CopyDest,
				nullptr,
				ResultBuffer.GetInitReference(),
				ResultBufferName
		));
		SetD3D12ResourceName(ResultBuffer, ResultBufferName);
	}

	// Map the readback buffer. Resources in a readback heap are allowed
	// to be persistently mapped, so we only need to do this once.
	ResultPtr = static_cast<uint8 const*>(ResultBuffer->Map());
}

FD3D12QueryHeap::~FD3D12QueryHeap()
{
	if (ResultPtr)
	{
		ResultBuffer->Unmap();
		ResultPtr = nullptr;
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	if (D3DX12Residency::IsInitialized(ResidencyHandle))
	{
		D3DX12Residency::EndTrackingObject(Device->GetResidencyManager(), ResidencyHandle);
	}
#endif

	DEC_DWORD_STAT(STAT_D3D12NumQueryHeaps);
}

uint32 FD3D12QueryHeap::Release()
{
	uint32 Refs = uint32(NumRefs.Decrement());
	if (Refs == 0)
	{
		Device->ReleaseQueryHeap(this);
	}
	return Refs;
}

FRenderQueryRHIRef FD3D12DynamicRHI::RHICreateRenderQuery(ERenderQueryType QueryType)
{
	check(QueryType == RQT_Occlusion || QueryType == RQT_AbsoluteTime);
	return GetAdapter().CreateLinkedObject<FD3D12RenderQuery>(FRHIGPUMask::All(), [QueryType](FD3D12Device* Device, FD3D12RenderQuery* FirstLinkedObject)
	{
		return new FD3D12RenderQuery(Device, QueryType);
	});
}

void FD3D12DynamicRHI::RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	// Each query batch uses a single sync point to signal when the results are ready (one per active GPU).
	for (uint32 GPUIndex : RHICmdList.GetGPUMask())
	{
		auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
		checkf(QueryBatchData[GPUIndex] == nullptr, TEXT("A query batch for this type has already begun on this command list."));

		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, GetRenderQueryTypeName(QueryType));

		// Keep a reference on the RHI command list, so we can retrieve it later in BeginQuery/EndQuery/EndBatch.
		QueryBatchData[GPUIndex] = SyncPoint.GetReference();
		SyncPoint->AddRef();
	}

	if (QueryType == RQT_Occlusion && RHIConsoleVariables::GInsertOuterOcclusionQuery)
	{
		// Insert an outer query that encloses the whole batch
		RHICmdList.EnqueueLambda([](FRHICommandListBase& ExecutingCmdList)
		{
			for (uint32 GPUIndex : ExecutingCmdList.GetGPUMask())
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);

				if (!Context.OuterOcclusionQuery.IsValid())
					Context.OuterOcclusionQuery = GDynamicRHI->RHICreateRenderQuery(RQT_Occlusion);

				Context.RHIBeginRenderQuery(Context.OuterOcclusionQuery);
				Context.bOuterOcclusionQuerySubmitted = true;
			}
		});
	}
}

void FD3D12CommandContext::RHIBeginRenderQuery(FRHIRenderQuery* QueryRHI)
{
	FD3D12RenderQuery* Query = RetrieveObject<FD3D12RenderQuery>(QueryRHI);
	checkf(Query->Type == RQT_Occlusion, TEXT("Only occlusion queries support RHIBeginRenderQuery()."));

	Query->ActiveLocation = AllocateQuery(ED3D12QueryType::Occlusion, Query->Result);
	BeginQuery(Query->ActiveLocation);

	ActiveQueries++;
}

void FD3D12DynamicRHI::RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	for (uint32 GPUIndex : RHICmdList.GetGPUMask())
	{
		FD3D12RenderQuery* Query = ResourceCast(RenderQuery, GPUIndex);
		auto& QueryBatchData = RHICmdList.GetQueryBatchData(Query->Type);

		if (QueryBatchData[GPUIndex])
		{
			// This query belongs to a batch. Use the sync point we created earlier
			Query->SyncPoint = static_cast<FD3D12SyncPoint*>(QueryBatchData[GPUIndex]);
		}
		else
		{
			// Queries issued outside of a batch use one sync point per query.
			Query->SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, GetRenderQueryTypeName(Query->Type));

			RHICmdList.EnqueueLambda([SyncPoint = Query->SyncPoint, GPUIndex](FRHICommandListBase& ExecutingCmdList) mutable
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
				Context.BatchedSyncPoints.ToSignal.Emplace(MoveTemp(SyncPoint));
			});
		}
	}

	// Enqueue the RHI command to record the EndQuery() call on the context.
	FDynamicRHI::RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FD3D12CommandContext::RHIEndRenderQuery(FRHIRenderQuery* QueryRHI)
{
	FD3D12RenderQuery* Query = RetrieveObject<FD3D12RenderQuery>(QueryRHI);
	switch (Query->Type)
	{
	default:
		checkNoEntry();
		return;

	case RQT_Occlusion:
		check(ActiveQueries > 0);
		ActiveQueries--;

		EndQuery(Query->ActiveLocation);
		Query->ActiveLocation = {};
		break;

	case RQT_AbsoluteTime:
		InsertTimestamp(ED3D12Units::Microseconds, Query->Result);
		break;
	}
}

void FD3D12DynamicRHI::RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	for (uint32 GPUIndex : RHICmdList.GetGPUMask())
	{
		auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
		checkf(QueryBatchData[GPUIndex], TEXT("A query batch for this type is not open on this command list."));

		FD3D12SyncPointRef SyncPoint = static_cast<FD3D12SyncPoint*>(QueryBatchData[GPUIndex]);

		// Clear the sync point reference on the RHI command list
		SyncPoint->Release();
		QueryBatchData[GPUIndex] = nullptr;

		RHICmdList.EnqueueLambda([GPUIndex, SyncPoint = MoveTemp(SyncPoint), QueryType](FRHICommandListBase& ExecutingCmdList)
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
			Context.BatchedSyncPoints.ToSignal.Add(SyncPoint);

			if (QueryType == RQT_Occlusion)
			{
				// End the outer query
				if (Context.bOuterOcclusionQuerySubmitted)
				{
					Context.RHIEndRenderQuery(Context.OuterOcclusionQuery);
					Context.bOuterOcclusionQuerySubmitted = false;
				}
			}
		});
	}
}

bool FD3D12DynamicRHI::RHIGetRenderQueryResult(FRHIRenderQuery* QueryRHI, uint64& OutResult, bool bWait, uint32 QueryGPUIndex)
{
	FD3D12RenderQuery* Query;

	// This will be the common case, as most users aren't running MGPU, so check this first (also becomes a constant compare if WITH_MGPU disabled)
	if (GNumExplicitGPUsForRendering <= 1)
	{
		Query = FD3D12DynamicRHI::ResourceCast(QueryRHI, 0);
	}
	else if (QueryGPUIndex != INDEX_NONE)
	{
		Query = FD3D12DynamicRHI::ResourceCast(QueryRHI, QueryGPUIndex);
	}
	else
	{
		// Pick the first query that has a valid sync point.  If none have a valid sync point, the function will return failure, so it
		// doesn't matter which we pick.  We need to set Query outisde the loop to avoid an uninitialized variable compile error, so
		// we check the first item before starting the loop.
		FD3D12RenderQuery::FLinkedObjectIterator CurrentQuery((FD3D12RenderQuery*)QueryRHI);
		Query = CurrentQuery.Get();

		if (!Query->SyncPoint.IsValid())
		{
			for (++CurrentQuery; CurrentQuery; ++CurrentQuery)
			{
				Query = CurrentQuery.Get();
				if (Query->SyncPoint.IsValid())
				{
					break;
				}
			}
		}
	}

	if (!ensureMsgf(Query->SyncPoint, TEXT("Attempt to get result data for an FRHIRenderQuery that was never used in a command list.")))
	{
		OutResult = 0;
		return false;
	}

	if (!Query->SyncPoint->IsComplete())
	{
		if (bWait)
		{
			Query->SyncPoint->Wait();
		}
		else
		{
			return false;
		}
	}

	OutResult = *Query->Result;
	return true;
}
