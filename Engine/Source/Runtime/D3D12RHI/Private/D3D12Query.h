// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#include "D3D12Residency.h"
#include "D3D12Resources.h"
#include "RHIResources.h"

class FD3D12SyncPoint;
using FD3D12SyncPointRef = TRefCountPtr<FD3D12SyncPoint>;
struct FD3D12QueryLocation;

enum class ED3D12QueryType
{
	None,
	PipelineStats,
	TimestampMicroseconds,
	TimestampRaw,

	// Top-of-pipe GPU timestamp, converted to the CPU clock domain (i.e. relative to FPlatformTime::Cycles64())
	ProfilerTimestampTOP,

	// Bottom-of-pipe GPU timestamp, converted to the CPU clock domain (i.e. relative to FPlatformTime::Cycles64())
	ProfilerTimestampBOP,

	Occlusion
};

enum class ED3D12QueryPosition
{
	// Query result should be written before any future command list work is started.
	TopOfPipe,

	// Query result should be written after all prior command list work has completed.
	BottomOfPipe
};

// Wraps an ID3D12QueryHeap and its readback buffer. Used by command contexts to create timestamp and occlusion queries.
// Ref-counting is used to recycle the heaps on parent device when all refering command lists have completed on the GPU.
class FD3D12QueryHeap final : public FD3D12SingleNodeGPUObject
{
	// All query heaps are allocated to fill a single 64KB page
	static constexpr uint32 MaxHeapSize = 65536;

private:
	friend FD3D12Device;
	friend FD3D12QueryLocation;

	FD3D12QueryHeap(FD3D12QueryHeap const&) = delete;
	FD3D12QueryHeap(FD3D12QueryHeap&&) = delete;

	FD3D12QueryHeap(FD3D12Device* Device, D3D12_QUERY_TYPE QueryType, D3D12_QUERY_HEAP_TYPE HeapType);

public:
	~FD3D12QueryHeap();

	FD3D12Device* const Device;
	D3D12_QUERY_TYPE const QueryType;
	D3D12_QUERY_HEAP_TYPE const HeapType;
	uint32 const NumQueries;

	// The byte size of a result for a single query
	uint32 GetResultSize() const
	{
		switch (QueryType)
		{
		default: checkNoEntry(); [[fallthrough]];
		case D3D12_QUERY_TYPE_TIMESTAMP:
		case D3D12_QUERY_TYPE_OCCLUSION:
			return sizeof(uint64);

		case D3D12_QUERY_TYPE_PIPELINE_STATISTICS:
			return sizeof(D3D12_QUERY_DATA_PIPELINE_STATISTICS);
		}
	}

	ID3D12QueryHeap* GetD3DQueryHeap() const { return D3DQueryHeap; }
	FD3D12Resource* GetResultBuffer() const { return ResultBuffer; }

	FD3D12ResidencyHandle& GetHeapResidencyHandle() { return ResidencyHandle; }

	// Ref-counting used for object pool recycling.
	uint32 AddRef() { return uint32(NumRefs.Increment()); }
	uint32 Release();

private:
	TRefCountPtr<FD3D12Resource> ResultBuffer;
	uint8 const* ResultPtr = nullptr;
	TRefCountPtr<ID3D12QueryHeap> D3DQueryHeap;
	FD3D12ResidencyHandle ResidencyHandle;

	FThreadSafeCounter NumRefs;
};

struct FD3D12QueryRange
{
	uint32 Start = 0, End = 0;

	FD3D12QueryRange() = default;
	FD3D12QueryRange(uint32 Start, uint32 End)
		: Start(Start)
		, End(End)
	{}

	inline bool IsFull(FD3D12QueryHeap* Heap) const;

	bool operator == (FD3D12QueryRange const& RHS) const
	{
		return Start == RHS.Start
			&& End   == RHS.End;
	}

	bool operator < (FD3D12QueryRange const& RHS) const
	{
		return Start < RHS.Start;
	}
};

// The location of a single (timestamp or occlusion) query result.
struct FD3D12QueryLocation
{
	// The heap in which the result is contained.
	FD3D12QueryHeap* Heap = nullptr;

	// The index of the query within the heap.
	uint32 Index = 0;

	ED3D12QueryType Type = ED3D12QueryType::None;

	// The location into which the result is written by the interrupt thread.
	void* Target = nullptr;

	// Reads the query result from the heap
	inline void CopyResultTo(void* Dst) const;

	template <typename TValueType>
	inline TValueType GetResult() const;

	FD3D12QueryLocation() = default;
	FD3D12QueryLocation(FD3D12QueryHeap* Heap, uint32 Index, ED3D12QueryType Type, void* Target)
		: Heap	(Heap  )
		, Index	(Index )
		, Type	(Type  )
		, Target(Target)
	{}

	operator bool() const { return Heap != nullptr; }
};

inline void FD3D12QueryLocation::CopyResultTo(void* Dst) const
{
	check(Dst);
	check(Index < Heap->NumQueries);
	check(Heap->ResultPtr);

	void const* Src = Heap->ResultPtr + Index * Heap->GetResultSize();
	FMemory::Memcpy(Dst, Src, Heap->GetResultSize());
}

template <typename TValueType>
inline TValueType FD3D12QueryLocation::GetResult() const
{
	check(sizeof(TValueType) >= Heap->GetResultSize());

	TValueType Value;
	CopyResultTo(&Value);
	return Value;
}

inline bool FD3D12QueryRange::IsFull(FD3D12QueryHeap* Heap) const
{
	return End >= Heap->NumQueries;
}

class FD3D12QueryAllocator
{
public:
	FD3D12QueryAllocator(FD3D12Device* Device, ED3D12QueueType const QueueType, D3D12_QUERY_TYPE QueryType)
		: Device(Device)
		, QueueType(QueueType)
		, QueryType(QueryType)
	{}

	FD3D12Device*    const Device;
	ED3D12QueueType  const QueueType;
	D3D12_QUERY_TYPE const QueryType;

	// Allocate a query on a query heap, returning its location.
	// The "target" is where the interrupt thread will write the result when completed by the GPU.
	FD3D12QueryLocation Allocate(ED3D12QueryType Type, void* Target);

	// Resets the allocator and returns the used query ranges
	void CloseAndReset(TMap<TRefCountPtr<FD3D12QueryHeap>, TArray<FD3D12QueryRange>>& OutRanges);

	bool HasQueries() const
	{
		return !(CurrentRange == nullptr || CurrentRange->Start == CurrentRange->End);
	}

private:
	TMap<TRefCountPtr<FD3D12QueryHeap>, FD3D12QueryRange> Heaps;
	
	FD3D12QueryHeap* CurrentHeap = nullptr;
	FD3D12QueryRange* CurrentRange = nullptr;
};

/** D3D12 Render query */
class FD3D12RenderQuery : public FRHIRenderQuery, public FD3D12DeviceChild, public FD3D12LinkedAdapterObject<FD3D12RenderQuery>
{
public:
	FD3D12RenderQuery(FD3D12Device* Parent, ERenderQueryType InQueryType);
	~FD3D12RenderQuery();

	ERenderQueryType const Type;

	// Signaled when the result is available. Nullptr if the query has never been used.
	FD3D12SyncPointRef SyncPoint;

	// The query result, read from the GPU. Heap allocated since it is
	// accessed by the interrupt thread, and needs to outlive the RHI object.
	uint64* Result;
	
	// The current query location for occlusion queries.
	FD3D12QueryLocation ActiveLocation;
};

template<>
struct TD3D12ResourceTraits<FRHIRenderQuery>
{
	typedef FD3D12RenderQuery TConcreteType;
};
