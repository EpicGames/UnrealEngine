// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#include "D3D12RHIDefinitions.h"
#include "RHIPipeline.h"

// Defines a unique command queue type within a FD3D12Device (owner by the command list managers).
enum class ED3D12QueueType
{
	Direct = 0,
	Copy,
	Async,

	Count,
};

static constexpr uint32 GD3D12MaxNumQueues = MAX_NUM_GPUS * (uint32)ED3D12QueueType::Count;

inline ED3D12QueueType GetD3DCommandQueueType(ERHIPipeline Pipeline)
{
	switch (Pipeline)
	{
	default: checkNoEntry(); // fallthrough
	case ERHIPipeline::Graphics:     return ED3D12QueueType::Direct;
	case ERHIPipeline::AsyncCompute: return ED3D12QueueType::Async;
	}
}

inline const TCHAR* GetD3DCommandQueueTypeName(ED3D12QueueType QueueType)
{
	switch (QueueType)
	{
	default: checkNoEntry(); // fallthrough
	case ED3D12QueueType::Direct: return TEXT("3D");
	case ED3D12QueueType::Async:  return TEXT("Compute");
	case ED3D12QueueType::Copy:   return TEXT("Copy");
	}
}

inline FName GetD3DCommandQueueTypeFName(ED3D12QueueType QueueType)
{
	switch (QueueType)
	{
	default: checkNoEntry(); // fallthrough
	case ED3D12QueueType::Direct: { static const FName Name(TEXT("3D"));      return Name; }
	case ED3D12QueueType::Async:  { static const FName Name(TEXT("Compute")); return Name; }
	case ED3D12QueueType::Copy:   { static const FName Name(TEXT("Copy"));    return Name; }
	}
}

inline D3D12_COMMAND_LIST_TYPE GetD3DCommandListType(ED3D12QueueType QueueType)
{
	switch (QueueType)
	{
	default: checkNoEntry(); // fallthrough
	case ED3D12QueueType::Direct: return D3D12_COMMAND_LIST_TYPE_DIRECT;
	case ED3D12QueueType::Copy:   return D3D12RHI_PLATFORM_COPY_COMMAND_LIST_TYPE;
	case ED3D12QueueType::Async:  return D3D12_COMMAND_LIST_TYPE_COMPUTE;
	}
}

// TODO: move FD3D12Queue here
