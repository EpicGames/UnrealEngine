// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RHI.cpp: Unreal D3D RHI library implementation.
	=============================================================================*/

#include "D3D12RHI.h"
#include "D3D12RHIPrivate.h"
#include "D3D12RayTracing.h"
#include "RHIStaticStates.h"
#include "OneColorShader.h"
#include "DataDrivenShaderPlatformInfo.h"

#include "D3D12AmdExtensions.h"
#include "D3D12IntelExtensions.h"

#include "D3D12Tracing.h"

#if !defined(D3D12_PLATFORM_NEEDS_DISPLAY_MODE_ENUMERATION)
	#define D3D12_PLATFORM_NEEDS_DISPLAY_MODE_ENUMERATION 1
#endif

DEFINE_LOG_CATEGORY(LogD3D12RHI);
DEFINE_LOG_CATEGORY(LogD3D12GapRecorder);

int32 GD3D12RequestExitOnGPUTimeout = 0;
static FAutoConsoleVariableRef CVarD3D12RequestExitOnGPUTimeout(
	TEXT("D3D12.RequestExitOnGPUTimeout"),
	GD3D12RequestExitOnGPUTimeout,
	TEXT("Request exit on GPU timeouts, useful for external crash handlers"));

int32 GD3D12BindResourceLabels = 1;
static FAutoConsoleVariableRef CVarD3D12BindResourceLabels(
	TEXT("d3d12.BindResourceLabels"),
	GD3D12BindResourceLabels,
	TEXT("Whether to enable binding of debug names to D3D12 resources."));

static TAutoConsoleVariable<int32> CVarD3D12UseD24(
	TEXT("r.D3D12.Depth24Bit"),
	0,
	TEXT("0: Use 32-bit float depth buffer\n1: Use 24-bit fixed point depth buffer(default)\n"),
	ECVF_ReadOnly
);


TAutoConsoleVariable<int32> CVarD3D12ZeroBufferSizeInMB(
	TEXT("D3D12.ZeroBufferSizeInMB"),
	4,
	TEXT("The D3D12 RHI needs a static allocation of zeroes to use when streaming textures asynchronously. It should be large enough to support the largest mipmap you need to stream. The default is 4MB."),
	ECVF_ReadOnly
	);

static bool GPSOPrecacheD3D12DriverCacheAware = true;
static FAutoConsoleVariableRef CVarPSOPrecacheD3D12DriverCacheAware(
	TEXT("r.PSOPrecache.D3D12.DriverCacheAware"),
	GPSOPrecacheD3D12DriverCacheAware,
	TEXT("If enabled, the PSO precaching system will not precache PSOs that the D3D12 graphics driver considers similar for caching, i.e. it will not precache PSOs that while technically different will still result in a driver cache hit.\n")
	TEXT("This is not implemented for all GPU vendors and can result in performance issues or cache misses if the heuristics the engine uses does not match the graphics driver's behavior that decides whether a PSO is in the cache or not."),
	ECVF_ReadOnly);

FD3D12DynamicRHI* FD3D12DynamicRHI::SingleD3DRHI = nullptr;
bool FD3D12DynamicRHI::bFormatAliasedTexturesMustBeCreatedUsingCommonLayout = false;

FD3D12WorkaroundFlags GD3D12WorkaroundFlags;

using namespace D3D12RHI;

FD3D12DynamicRHI::FD3D12DynamicRHI(TSharedPtr<FD3D12Adapter>& InChosenAdapter, bool bInPixEventEnabled)
	: ChosenAdapter(InChosenAdapter)
	, bPixEventEnabled(bInPixEventEnabled)
	, AmdAgsContext(nullptr)
	, AmdSupportedExtensionFlags(0)
	, FlipEvent(INVALID_HANDLE_VALUE)
{
	// The FD3D12DynamicRHI must be a singleton
	check(SingleD3DRHI == nullptr);
	SingleD3DRHI = this;

	// This should be called once at the start 
	check(IsInGameThread());
	check(!GIsThreadedRendering);

	// Adapter must support FL11+
	FeatureLevel = GetAdapter().GetFeatureLevel();
	check(FeatureLevel >= D3D_FEATURE_LEVEL_11_0);

#if PLATFORM_WINDOWS
	// Allocate a buffer of zeroes. This is used when we need to pass D3D memory
	// that we don't care about and will overwrite with valid data in the future.
	ZeroBufferSize = FMath::Max(CVarD3D12ZeroBufferSizeInMB.GetValueOnAnyThread(), 0) * (1 << 20);
	ZeroBuffer = FMemory::Malloc(ZeroBufferSize);
	FMemory::Memzero(ZeroBuffer, ZeroBufferSize);
#else
	ZeroBufferSize = 0;
	ZeroBuffer = nullptr;
#endif // PLATFORM_WINDOWS

	bDriverCacheAwarePSOPrecaching = GPSOPrecacheD3D12DriverCacheAware;

	GRHIGlobals.SupportsMultiDrawIndirect = true;

	GRHISupportsMultithreading = true;
	GRHISupportsMultithreadedResources = true;
	GRHISupportsAsyncGetRenderQueryResult = true;
	GRHIMultiPipelineMergeableAccessMask = GRHIMergeableAccessMask;
	EnumRemoveFlags(GRHIMultiPipelineMergeableAccessMask, ERHIAccess::UAVMask);

	GPoolSizeVRAMPercentage = 0;
	GTexturePoolSize = 0;
	GConfig->GetInt(TEXT("TextureStreaming"), TEXT("PoolSizeVRAMPercentage"), GPoolSizeVRAMPercentage, GEngineIni);

	// Initialize the platform pixel format map.
	GPixelFormats[PF_Unknown		].PlatformFormat = DXGI_FORMAT_UNKNOWN;
	GPixelFormats[PF_A32B32G32R32F	].PlatformFormat = DXGI_FORMAT_R32G32B32A32_FLOAT;
	GPixelFormats[PF_B8G8R8A8		].PlatformFormat = DXGI_FORMAT_B8G8R8A8_TYPELESS;
	GPixelFormats[PF_G8				].PlatformFormat = DXGI_FORMAT_R8_UNORM;
	GPixelFormats[PF_G16			].PlatformFormat = DXGI_FORMAT_R16_UNORM;
	GPixelFormats[PF_DXT1			].PlatformFormat = DXGI_FORMAT_BC1_TYPELESS;
	GPixelFormats[PF_DXT3			].PlatformFormat = DXGI_FORMAT_BC2_TYPELESS;
	GPixelFormats[PF_DXT5			].PlatformFormat = DXGI_FORMAT_BC3_TYPELESS;
	GPixelFormats[PF_BC4			].PlatformFormat = DXGI_FORMAT_BC4_UNORM;
	GPixelFormats[PF_UYVY			].PlatformFormat = DXGI_FORMAT_UNKNOWN;		// TODO: Not supported in D3D11
	if (CVarD3D12UseD24.GetValueOnAnyThread())
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat = DXGI_FORMAT_R24G8_TYPELESS;
		GPixelFormats[PF_DepthStencil].BlockBytes = 4;
		GPixelFormats[PF_DepthStencil].Supported = true;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = true;
		GPixelFormats[PF_X24_G8].PlatformFormat = DXGI_FORMAT_X24_TYPELESS_G8_UINT;
		GPixelFormats[PF_X24_G8].BlockBytes = 4;
		GPixelFormats[PF_X24_G8].Supported = true;
	}
	else
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat = DXGI_FORMAT_R32G8X24_TYPELESS;
		GPixelFormats[PF_DepthStencil].BlockBytes = 5;
		GPixelFormats[PF_DepthStencil].Supported = true;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = false;
		GPixelFormats[PF_X24_G8].PlatformFormat = DXGI_FORMAT_X32_TYPELESS_G8X24_UINT;
		GPixelFormats[PF_X24_G8].BlockBytes = 5;
		GPixelFormats[PF_X24_G8].Supported = true;
	}
	GPixelFormats[PF_ShadowDepth	].PlatformFormat = DXGI_FORMAT_R16_TYPELESS;
	GPixelFormats[PF_ShadowDepth	].BlockBytes = 2;
	GPixelFormats[PF_ShadowDepth	].Supported = true;
	GPixelFormats[PF_R32_FLOAT		].PlatformFormat = DXGI_FORMAT_R32_FLOAT;
	GPixelFormats[PF_G16R16			].PlatformFormat = DXGI_FORMAT_R16G16_UNORM;
	GPixelFormats[PF_G16R16F		].PlatformFormat = DXGI_FORMAT_R16G16_FLOAT;
	GPixelFormats[PF_G16R16F_FILTER	].PlatformFormat = DXGI_FORMAT_R16G16_FLOAT;
	GPixelFormats[PF_G32R32F		].PlatformFormat = DXGI_FORMAT_R32G32_FLOAT;
	GPixelFormats[PF_A2B10G10R10	].PlatformFormat = DXGI_FORMAT_R10G10B10A2_UNORM;
	GPixelFormats[PF_A16B16G16R16	].PlatformFormat = DXGI_FORMAT_R16G16B16A16_UNORM;
	GPixelFormats[PF_D24			].PlatformFormat = DXGI_FORMAT_R24G8_TYPELESS;
	GPixelFormats[PF_R16F			].PlatformFormat = DXGI_FORMAT_R16_FLOAT;
	GPixelFormats[PF_R16F_FILTER	].PlatformFormat = DXGI_FORMAT_R16_FLOAT;

	GPixelFormats[PF_FloatRGB		].PlatformFormat = DXGI_FORMAT_R11G11B10_FLOAT;
	GPixelFormats[PF_FloatRGB		].BlockBytes = 4;
	GPixelFormats[PF_FloatRGBA		].PlatformFormat = DXGI_FORMAT_R16G16B16A16_FLOAT;
	GPixelFormats[PF_FloatRGBA		].BlockBytes = 8;
	GPixelFormats[PF_FloatR11G11B10	].PlatformFormat = DXGI_FORMAT_R11G11B10_FLOAT;
	GPixelFormats[PF_FloatR11G11B10	].Supported = true;
	GPixelFormats[PF_FloatR11G11B10	].BlockBytes = 4;

	GPixelFormats[PF_V8U8			].PlatformFormat = DXGI_FORMAT_R8G8_SNORM;
	GPixelFormats[PF_BC5			].PlatformFormat = DXGI_FORMAT_BC5_UNORM;
	GPixelFormats[PF_A1				].PlatformFormat = DXGI_FORMAT_R1_UNORM; // Not supported for rendering.
	GPixelFormats[PF_A8				].PlatformFormat = DXGI_FORMAT_A8_UNORM;
	GPixelFormats[PF_R32_UINT		].PlatformFormat = DXGI_FORMAT_R32_UINT;
	GPixelFormats[PF_R32_SINT		].PlatformFormat = DXGI_FORMAT_R32_SINT;

	GPixelFormats[PF_R16_UINT		].PlatformFormat = DXGI_FORMAT_R16_UINT;
	GPixelFormats[PF_R16_SINT		].PlatformFormat = DXGI_FORMAT_R16_SINT;
	GPixelFormats[PF_R16G16B16A16_UINT].PlatformFormat = DXGI_FORMAT_R16G16B16A16_UINT;
	GPixelFormats[PF_R16G16B16A16_SINT].PlatformFormat = DXGI_FORMAT_R16G16B16A16_SINT;

	GPixelFormats[PF_R5G6B5_UNORM	].PlatformFormat = DXGI_FORMAT_B5G6R5_UNORM;
	GPixelFormats[PF_R5G6B5_UNORM	].Supported = true;
	GPixelFormats[PF_B5G5R5A1_UNORM ].PlatformFormat = DXGI_FORMAT_B5G5R5A1_UNORM;
	GPixelFormats[PF_B5G5R5A1_UNORM ].Supported = true;
	GPixelFormats[PF_R8G8B8A8		].PlatformFormat = DXGI_FORMAT_R8G8B8A8_TYPELESS;
	GPixelFormats[PF_R8G8B8A8_UINT	].PlatformFormat = DXGI_FORMAT_R8G8B8A8_UINT;
	GPixelFormats[PF_R8G8B8A8_SNORM	].PlatformFormat = DXGI_FORMAT_R8G8B8A8_SNORM;

	GPixelFormats[PF_R8G8			].PlatformFormat = DXGI_FORMAT_R8G8_UNORM;
	GPixelFormats[PF_R32G32B32A32_UINT].PlatformFormat = DXGI_FORMAT_R32G32B32A32_UINT;
	GPixelFormats[PF_R16G16_UINT	].PlatformFormat = DXGI_FORMAT_R16G16_UINT;
	GPixelFormats[PF_R16G16_SINT	].PlatformFormat = DXGI_FORMAT_R16G16_SINT;
	GPixelFormats[PF_R32G32_UINT	].PlatformFormat = DXGI_FORMAT_R32G32_UINT;

	GPixelFormats[PF_BC6H			].PlatformFormat = DXGI_FORMAT_BC6H_UF16;
	GPixelFormats[PF_BC7			].PlatformFormat = DXGI_FORMAT_BC7_TYPELESS;
	GPixelFormats[PF_R8_UINT		].PlatformFormat = DXGI_FORMAT_R8_UINT;
	GPixelFormats[PF_R8				].PlatformFormat = DXGI_FORMAT_R8_UNORM;

	GPixelFormats[PF_R16G16B16A16_UNORM].PlatformFormat = DXGI_FORMAT_R16G16B16A16_UNORM;
	GPixelFormats[PF_R16G16B16A16_SNORM].PlatformFormat = DXGI_FORMAT_R16G16B16A16_SNORM;

	GPixelFormats[PF_NV12].PlatformFormat = DXGI_FORMAT_NV12;
	GPixelFormats[PF_NV12].Supported = true;

	GPixelFormats[PF_G16R16_SNORM	].PlatformFormat = DXGI_FORMAT_R16G16_SNORM;
	GPixelFormats[PF_R8G8_UINT		].PlatformFormat = DXGI_FORMAT_R8G8_UINT;
	GPixelFormats[PF_R32G32B32_UINT	].PlatformFormat = DXGI_FORMAT_R32G32B32_UINT;
	GPixelFormats[PF_R32G32B32_SINT	].PlatformFormat = DXGI_FORMAT_R32G32B32_SINT;
	GPixelFormats[PF_R32G32B32F		].PlatformFormat = DXGI_FORMAT_R32G32B32_FLOAT;
	GPixelFormats[PF_R8_SINT		].PlatformFormat = DXGI_FORMAT_R8_SINT;

	GPixelFormats[PF_R9G9B9EXP5	    ].PlatformFormat = DXGI_FORMAT_R9G9B9E5_SHAREDEXP;

	GPixelFormats[PF_P010			].PlatformFormat = DXGI_FORMAT_P010;
	GPixelFormats[PF_P010			].Supported = true;

	// MS - Not doing any feature level checks. D3D12 currently supports these limits.
	// However this may need to be revisited if new feature levels are introduced with different HW requirement
	GSupportsSeparateRenderTargetBlendState = true;
	GMaxTextureDimensions = D3D12_REQ_TEXTURE2D_U_OR_V_DIMENSION;
	GMaxCubeTextureDimensions = D3D12_REQ_TEXTURECUBE_DIMENSION;
	GMaxTextureArrayLayers = D3D12_REQ_TEXTURE2D_ARRAY_AXIS_DIMENSION;
	GRHISupportsMSAADepthSampleAccess = true;

	GMaxTextureMipCount = FMath::CeilLogTwo(GMaxTextureDimensions) + 1;
	GMaxTextureMipCount = FMath::Min<int32>(MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount);
	GMaxShadowDepthBufferSizeX = GMaxTextureDimensions;
	GMaxShadowDepthBufferSizeY = GMaxTextureDimensions;
	GRHISupportsArrayIndexFromAnyShader = true;

	GMaxTextureSamplers = FMath::Min<int32>(MAX_SAMPLERS, (GetAdapter().GetResourceBindingTier() >= D3D12_RESOURCE_BINDING_TIER_2 ? D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE : D3D12_COMMONSHADER_SAMPLER_REGISTER_COUNT));

	GRHIMaxDispatchThreadGroupsPerDimension.X = D3D12_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;
	GRHIMaxDispatchThreadGroupsPerDimension.Y = D3D12_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;
	GRHIMaxDispatchThreadGroupsPerDimension.Z = D3D12_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION;

	GRHISupportsRHIThread = true;

	GRHISupportsParallelRHIExecute = true;
	GRHIGlobals.ParallelRenderPasses.Supported = true;

	GRHISupportsRawViewsForAnyBuffer = true;

	GSupportsTimestampRenderQueries = true;
	GSupportsParallelOcclusionQueries = true;

	// Manually enable Async BVH build for D3D12 RHI
	GRHISupportsRayTracingAsyncBuildAccelerationStructure = true;

	GRHISupportsPipelineFileCache = PLATFORM_WINDOWS;
	GRHISupportsPSOPrecaching = PLATFORM_WINDOWS;

	GRHISupportsMapWriteNoOverwrite = true;

	GRHISupportsFrameCyclesBubblesRemoval = true;
	GRHISupportsGPUTimestampBubblesRemoval = true;
	GRHISupportsRHIOnTaskThread = true;

	GRHIGlobals.NeedsShaderUnbinds = true;

	// All D3D12 hardware supports binding UAVs to Vertex Shaders.
	// Enable run-time support if corresponding bit is set in DDSPI.
	GRHIGlobals.SupportsVertexShaderUAVs = true;

	GRHIGlobals.NeedsExtraTransitions = true;

#if PLATFORM_WINDOWS
	InitializeDeviceAgnosticVendorAPI();
#endif
}

void FD3D12DynamicRHI::PostInit()
{
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		Adapter->InitializeExplicitDescriptorHeap();

		if (GRHISupportsRayTracing)
		{
			Adapter->InitializeRayTracing();
		}
	}
}

FD3D12DynamicRHI::~FD3D12DynamicRHI()
{
	UE_LOGF(LogD3D12RHI, Log, "~FD3D12DynamicRHI");

	check(!ChosenAdapter.IsValid());
}

void FD3D12DynamicRHI::ForEachQueue(TFunctionRef<void(FD3D12Queue&)> Callback)
{
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		for (FD3D12Device* const Device : Adapter->GetDevices())
		{
			for (FD3D12Queue& Queue : Device->GetQueues())
			{
				Callback(Queue);
			}
		}
	}
}

FD3D12Device* FD3D12DynamicRHI::GetRHIDevice(uint32 GPUIndex) const
{
	return GetAdapter().GetDevice(GPUIndex);
}

void FD3D12DynamicRHI::Shutdown()
{
	check(IsInGameThread() && IsInRenderingThread());  // require that the render thread has been shut down

	// Reset the RHI initialized flag.
	GIsRHIInitialized = false;

#if WITH_NVAPI_RAYTRACING_VALIDATION
	if (RayTracingValidationCallbackHandle)
	{		
		const NvAPI_Status NvStatus = NvAPI_D3D12_UnregisterRaytracingValidationMessageCallback(GetAdapter().GetD3DDevice5(), RayTracingValidationCallbackHandle);
		check(NvStatus == NVAPI_OK);
		RayTracingValidationCallbackHandle = nullptr;
	}
#endif

#if WITH_AMD_AGS
	if (AmdAgsContext)
	{
		// Clean up the AMD extensions and shut down the AMD AGS utility library
		agsDeInitialize(AmdAgsContext);
		AmdAgsContext = nullptr;
	}
#endif

#if INTEL_EXTENSIONS
	if (IntelExtensionContext)
	{
		DestroyIntelExtensionsContext(IntelExtensionContext);
		IntelExtensionContext = nullptr;
	}
#endif

	// Ask all initialized FRenderResources to release their RHI resources.
	FRenderResource::ReleaseRHIForAllResources();

	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		Adapter->CleanupResources();
		Adapter->BlockUntilIdle();
	}

	// Flush all pending deletes before destroying the device or any command contexts.
	FRHICommandListImmediate::Get().ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	RHIShutdownFlipTracking();
	ShutdownSubmissionPipe();

	check(ObjectsToDelete.Num() == 0);

	// Delete adapters, devices, queues, command contexts etc
	ChosenAdapter.Reset();

	check(ObjectsToDelete.Num() == 0);

	// Release the buffer of zeroes.
	FMemory::Free(ZeroBuffer);
	ZeroBuffer = NULL;
	ZeroBufferSize = 0;

#if D3D12RHI_SUPPORTS_WIN_PIX
	if (WinPixGpuCapturerHandle)
	{
		FPlatformProcess::FreeDllHandle(WinPixGpuCapturerHandle);
		WinPixGpuCapturerHandle = nullptr;
	}
#endif
}

void FD3D12DynamicRHI::RHIGetMemoryStats(FRHIMemoryStats& OutStats)
{
	OutStats = MemoryStats;
}

FD3D12CommandContext* FD3D12DynamicRHI::CreateCommandContext(FD3D12Device* InParent, ED3D12QueueType InQueueType, bool InIsDefaultContext)
{
	return new FD3D12CommandContext(InParent, InQueueType, InIsDefaultContext);
}

void FD3D12DynamicRHI::CreateCommandQueue(FD3D12Device* Device, const D3D12_COMMAND_QUEUE_DESC& Desc, TRefCountPtr<ID3D12CommandQueue>& OutCommandQueue)
{
	VERIFYD3D12RESULT(Device->GetDevice()->CreateCommandQueue(&Desc, IID_PPV_ARGS(OutCommandQueue.GetInitReference())));
}

IRHICommandContext* FD3D12DynamicRHI::RHIGetDefaultContext()
{
	FD3D12Adapter& Adapter = GetAdapter();

	IRHICommandContext* DefaultCommandContext = nullptr;	
	if (GNumExplicitGPUsForRendering > 1)
	{
		DefaultCommandContext = static_cast<IRHICommandContext*>(&Adapter.GetDefaultContextRedirector());
	}
	else // Single GPU path
	{
		FD3D12Device* Device = Adapter.GetDevice(0);
		DefaultCommandContext = static_cast<IRHICommandContext*>(&Device->GetDefaultCommandContext());
	}

	check(DefaultCommandContext);
	return DefaultCommandContext;
}

void FD3D12DynamicRHI::EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FD3D12Payload&)> ModifyPayloadCallback)
{
	SCOPED_NAMED_EVENT_TEXT("EnqueueEndOfPipeTask", FColor::Yellow);

	FGraphEventArray Prereqs;
	Prereqs.Reserve(GD3D12MaxNumQueues + 1);
	if (EopTask)
	{
		Prereqs.Add(EopTask);
	}

	TArray<FD3D12Payload*> Payloads;
	Payloads.Reserve(GD3D12MaxNumQueues);

	ForEachQueue([&](FD3D12Queue& Queue)
	{
		FD3D12Payload* Payload = new FD3D12Payload(Queue);

		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, GetD3DCommandQueueTypeFName(Queue.QueueType));
		Payload->SyncPointsToSignal.Emplace(SyncPoint);
		Prereqs.Add(SyncPoint->GetGraphEvent());

		IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SignalSyncPoint(Payload, SyncPoint));

		if (ModifyPayloadCallback)
			ModifyPayloadCallback(*Payload);

		Payloads.Add(Payload);
	});

	SubmitPayloads(MoveTemp(Payloads));

	EopTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
		[Func = MoveTemp(TaskFunc)]()
		{
			SCOPED_NAMED_EVENT_TEXT("EndOfPipeTask", FColor::Red);
			Func();
		},
		QUICK_USE_CYCLE_STAT(FExecuteRHIThreadTask, STATGROUP_TaskGraphTasks),
		&Prereqs
	);
}

void FD3D12DynamicRHI::RHIProcessDeleteQueue()
{
	ProcessDeferredDeletionQueue_Platform();

	TArray<FD3D12DeferredDeleteObject> Local;
	{
		FScopeLock Lock(&ObjectsToDeleteCS);
		Local = MoveTemp(ObjectsToDelete);
	}

	if (Local.Num())
	{
		EnqueueEndOfPipeTask([Array = MoveTemp(Local)]()
		{
			SCOPED_NAMED_EVENT_TEXT("EndOfPipeTask_RHIProcessDeleteQueue", FColor::Silver);

			for (FD3D12DeferredDeleteObject const& ObjectToDelete : Array)
			{
				switch (ObjectToDelete.Type)
				{
				case FD3D12DeferredDeleteObject::EType::RHIObject:
					// This should be a final release.
					check(ObjectToDelete.RHIObject->GetRefCount() == 1);
					ObjectToDelete.RHIObject->Release();
					break;
				case FD3D12DeferredDeleteObject::EType::Heap:
					// Heaps can have additional references active.
					ObjectToDelete.Heap->Release();
					break;
				case FD3D12DeferredDeleteObject::EType::DescriptorHeap:
					ObjectToDelete.DescriptorHeap->Release();
					break;
				case FD3D12DeferredDeleteObject::EType::D3DObject:
					ObjectToDelete.D3DObject->Release();
					break;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
				case FD3D12DeferredDeleteObject::EType::DescriptorAllocation:
					ObjectToDelete.DescriptorAllocation.Device->GetBindlessDescriptorAllocator().FreeDescriptors(
						ObjectToDelete.DescriptorAllocation.Allocation,
						ObjectToDelete.DescriptorAllocation.TypeMask
					);
					break;
				case FD3D12DeferredDeleteObject::EType::BindlessDescriptor:
					ObjectToDelete.BindlessDescriptor.Device->GetBindlessDescriptorManager().ImmediateFree(ObjectToDelete.BindlessDescriptor.Handle);
					break;
				case FD3D12DeferredDeleteObject::EType::BindlessDescriptorHeap:
					ObjectToDelete.DescriptorHeap->GetParentDevice()->GetBindlessDescriptorManager().Recycle(ObjectToDelete.DescriptorHeap);
					break;
#endif
				case FD3D12DeferredDeleteObject::EType::CPUAllocation:
					FMemory::Free(ObjectToDelete.CPUAllocation);
					break;
				case FD3D12DeferredDeleteObject::EType::DescriptorBlock:
					ObjectToDelete.DescriptorBlock.Manager->Recycle(ObjectToDelete.DescriptorBlock.Block);
					break;
#if PLATFORM_SUPPORTS_VIRTUAL_TEXTURES	
				case FD3D12DeferredDeleteObject::EType::VirtualAllocation:
					FD3D12DynamicRHI::GetD3DRHI()->DestroyVirtualTexture(
						ObjectToDelete.VirtualAllocDescriptor.Flags,
						ObjectToDelete.VirtualAllocDescriptor.RawMemory,
						const_cast<FPlatformMemory::FPlatformVirtualMemoryBlock&>(ObjectToDelete.VirtualAllocDescriptor.VirtualBlock),
						ObjectToDelete.VirtualAllocDescriptor.CommittedTextureSize);
					break;
#endif

				case FD3D12DeferredDeleteObject::EType::Func:
					(*ObjectToDelete.Func)();
					delete ObjectToDelete.Func;
					break;

				case FD3D12DeferredDeleteObject::EType::TextureStagingBuffer:
					ObjectToDelete.TextureStagingBufferData.Texture->ReuseStagingBuffer(MoveTemp(*(TUniquePtr<FD3D12LockedResource>*)ObjectToDelete.TextureStagingBufferData.LockedResourceStorage), ObjectToDelete.TextureStagingBufferData.Subresource);
					ObjectToDelete.TextureStagingBufferData.Texture->Release();
					break;

				default:
					checkf(false, TEXT("Unknown ED3D12DeferredDeleteObjectType"));
					break;
				}
			}
		});
	}

	// Clear all bound resources since we are about to flush pending deletions.
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		for (FD3D12Device* const Device : Adapter->GetDevices())
		{
			Device->GetDefaultCommandContext().ClearState(FD3D12ContextCommon::EClearStateMode::TransientOnly);
		}
	}
}

void FD3D12DynamicRHI::RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		Adapter->GetFrameFence().AdvanceTOP();
	}

	// Base implementation flushes all prior work and results in a bottom-of-pipe call to RHIEndFrame() on the RHI thread.
	FDynamicRHI::RHIEndFrame_RenderThread(RHICmdList);

	// Start the next GPU profiler frame
	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("D3D12 BeginFrame"),
		[this](FD3D12ContextArray const& Contexts)
	{
		if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
		{
			for (FD3D12Device* const Device : Adapter->GetDevices())
			{
				Device->GetDefaultBufferAllocator().BeginFrame(Contexts);
				Device->GetTextureAllocator().BeginFrame(Contexts);

			#if D3D12_RHI_RAYTRACING
				// @todo dev-pr - explicit use of graphics context - nothing is synchronizing async compute - needs refactor
				FD3D12RayTracingCompactionRequestHandler* const RayTracingCompactionRequestHandler = Device->GetRayTracingCompactionRequestHandler();
				if (RayTracingCompactionRequestHandler)
				{
					RayTracingCompactionRequestHandler->Update(*Contexts[ERHIPipeline::Graphics]->GetSingleDeviceContext(Device->GetGPUIndex()));
				}
			#endif
			}
		}
	});
}

#if WITH_NVAPI_RAYTRACING_VALIDATION
void FD3D12DynamicRHI::FlushRaytracingValidationMessages()
{
	if (RayTracingValidationCallbackHandle)
	{
		const NvAPI_Status NvStatus = NvAPI_D3D12_FlushRaytracingValidationMessages(GetAdapter().GetD3DDevice5());
		ensure(NvStatus == NVAPI_OK);
	}
}
#endif

void FD3D12DynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		Adapter->EndFrame();

		for (auto& Device : Adapter->GetDevices())
		{
			Device->GetTextureAllocator().CleanUpAllocations();

			// Only delete free blocks when not used in the last 2 frames, to make sure we are not allocating and releasing
			// the same blocks every frame.
			uint64 BufferPoolDeletionFrameLag = 20;
			Device->GetDefaultBufferAllocator().CleanupFreeBlocks(BufferPoolDeletionFrameLag);

			uint64 FastAllocatorDeletionFrameLag = 10;
			Device->GetDefaultFastAllocator().CleanupPages(FastAllocatorDeletionFrameLag);
		}

		Adapter->GetFrameFence().AdvanceBOP();
	}

	UpdateMemoryStats();

	// Close the previous frame's timing and start a new one
	auto Lambda = [this, OldTiming = MoveTemp(CurrentTimingPerQueue)]()
	{
		SCOPED_NAMED_EVENT_TEXT("EndOfPipeTask_RHIEndFrame", FColor::Orange);
		ProcessTimestamps(OldTiming);
	};

	EnqueueEndOfPipeTask(MoveTemp(Lambda), [&](FD3D12Payload& Payload)
	{
		// Modify the payloads the EOP task will submit to include
		// a new timing struct and a frame boundary event.

		Payload.Timing = CurrentTimingPerQueue.CreateNew(Payload.Queue);

		ERHIPipeline Pipeline;
		switch (Payload.Queue.QueueType)
		{
		default: checkNoEntry(); [[fallthrough]];
		case ED3D12QueueType::Direct: Pipeline = ERHIPipeline::Graphics; break;
		case ED3D12QueueType::Async:  Pipeline = ERHIPipeline::AsyncCompute; break;

		case ED3D12QueueType::Copy:
			// There is currently no high level RHI copy queue support
			Pipeline = ERHIPipeline::None;
			break;
		}

		// CPU timestamp for the frame boundary event is filled in by the submission thread
		Payload.EndFrameEvent = UE::RHI::GPUProfiler::FEvent::FFrameBoundary(Pipeline, Args, 0);
	});

#if WITH_NVAPI_RAYTRACING_VALIDATION
	if (RayTracingValidationCallbackHandle)
	{
		EnqueueEndOfPipeTask([this]() { FlushRaytracingValidationMessages(); });
	}
#endif

	// Pump the interrupt queue to gather completed events
	// (required if we're not using an interrupt thread).
	ProcessInterruptQueueUntil(nullptr);
}

TArray<FD3D12MinimalAdapterDesc> FD3D12DynamicRHI::RHIGetAdapterDescs() const
{
	TArray<FD3D12MinimalAdapterDesc> Result;

	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		FD3D12MinimalAdapterDesc Desc{};
		Desc.Desc = Adapter->GetDesc().Desc;
		Desc.NumDeviceNodes = Adapter->GetDesc().NumDeviceNodes;

		Result.Add(Desc);
	}

	return Result;
}

bool FD3D12DynamicRHI::RHIIsPixEnabled() const
{
	return IsPixEventEnabled();
}

ID3D12CommandQueue* FD3D12DynamicRHI::RHIGetCommandQueue() const
{
	// Multi-GPU support : any code using this function needs validation.
	return GetAdapter().GetDevice(0)->GetQueue(ED3D12QueueType::Direct).D3DCommandQueue;
}

ID3D12Device* FD3D12DynamicRHI::RHIGetDevice_NoMGPU() const
{
	ensureMsgf(GNumExplicitGPUsForRendering == 1,
		TEXT("Use of RHIGetDevice_NoMGPU() while multi-GPU is enabled. ")
		TEXT("The calling code may not be handling multi-GPU scenarios correctly. ")
		TEXT("Switch to using RHIGetDevice(N) where N is the GPU index if the calling code supports MGPU.")
	);
	return GetAdapter().GetDevice(0)->GetDevice();
}

ID3D12Device* FD3D12DynamicRHI::RHIGetDevice(uint32 InIndex) const
{
	return GetAdapter().GetDevice(InIndex)->GetDevice();
}

uint32 FD3D12DynamicRHI::RHIGetDeviceNodeMask(uint32 InIndex) const
{
	return GetAdapter().GetDevice(InIndex)->GetGPUMask().GetNative();
}

ID3D12GraphicsCommandList* FD3D12DynamicRHI::RHIGetGraphicsCommandList(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex) const
{
	FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, InDeviceIndex);
	return Context.GraphicsCommandList().Get();
}

FRHIViewport* FD3D12DynamicRHI::RHIGetSingletonViewport() const
{
	return ChosenAdapter->GetViewports().Num() == 1
		? ChosenAdapter->GetViewports()[0]
		: nullptr;
}

DXGI_FORMAT FD3D12DynamicRHI::RHIGetSwapChainFormat(EPixelFormat InFormat) const
{
	const DXGI_FORMAT PlatformFormat = UE::DXGIUtilities::FindDepthStencilFormat(static_cast<DXGI_FORMAT>(GPixelFormats[InFormat].PlatformFormat));
	return UE::DXGIUtilities::FindShaderResourceFormat(PlatformFormat, true);
}

ID3D12Resource* FD3D12DynamicRHI::RHIGetResource(FRHIBuffer* InBuffer) const
{
	FD3D12Buffer* D3D12Buffer = ResourceCast(InBuffer);
	return D3D12Buffer->GetResource()->GetResource();
}

uint32 FD3D12DynamicRHI::RHIGetResourceDeviceIndex(FRHIBuffer* InBuffer) const
{
	FD3D12Buffer* D3D12Buffer = ResourceCast(InBuffer);
	return D3D12Buffer->GetParentDevice()->GetGPUIndex();
}

int64 FD3D12DynamicRHI::RHIGetResourceMemorySize(FRHIBuffer* InBuffer) const
{
	FD3D12Buffer* D3D12Buffer = ResourceCast(InBuffer);
	return D3D12Buffer->ResourceLocation.GetSize();
}

bool FD3D12DynamicRHI::RHIIsResourcePlaced(FRHIBuffer* InBuffer) const
{
	FD3D12Buffer* D3D12Buffer = ResourceCast(InBuffer);
	return D3D12Buffer->GetResource()->IsPlacedResource();
}

ID3D12Resource* FD3D12DynamicRHI::RHIGetResource(FRHITexture* InTexture) const
{
	return (ID3D12Resource*)InTexture->GetNativeResource();
}

uint32 FD3D12DynamicRHI::RHIGetResourceDeviceIndex(FRHITexture* InTexture) const
{
	FD3D12Texture* D3D12Texture = GetD3D12TextureFromRHITexture(InTexture);
	return D3D12Texture->GetParentDevice()->GetGPUIndex();
}

int64 FD3D12DynamicRHI::RHIGetResourceMemorySize(FRHITexture* InTexture) const
{
	FD3D12Texture* D3D12Texture = GetD3D12TextureFromRHITexture(InTexture);
	return D3D12Texture->ResourceLocation.GetSize();
}

bool FD3D12DynamicRHI::RHIIsResourcePlaced(FRHITexture* InTexture) const
{
	FD3D12Texture* D3D12Texture = GetD3D12TextureFromRHITexture(InTexture);
	return D3D12Texture->GetResource()->IsPlacedResource();
}

D3D12_CPU_DESCRIPTOR_HANDLE FD3D12DynamicRHI::RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex, int32 InArraySliceIndex) const
{
	FD3D12Texture* D3D12Texture = GetD3D12TextureFromRHITexture(InTexture);
	FD3D12RenderTargetView* RTV = D3D12Texture->GetRenderTargetView(InMipIndex, InArraySliceIndex);
	return RTV ? RTV->GetOfflineCpuHandle() : D3D12_CPU_DESCRIPTOR_HANDLE{};
}

void FD3D12DynamicRHI::RHIFinishExternalComputeWork(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex, ID3D12GraphicsCommandList* InCommandList)
{
	FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, InDeviceIndex);
	check(InCommandList == Context.GraphicsCommandList().GetNoRefCount());

	Context.StateCache.ForceSetComputeRootSignature();

	ED3D12SetDescriptorHeapsFlags SetHeapFlags = ED3D12SetDescriptorHeapsFlags::ForceChanged;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Make sure we restore the correct heaps
	if (Context.StateCache.GetDescriptorCache()->IsUsingBindlessHeap())
	{
		EnumAddFlags(SetHeapFlags, ED3D12SetDescriptorHeapsFlags::Bindless);
	}
#endif

	Context.StateCache.GetDescriptorCache()->SetDescriptorHeaps(SetHeapFlags);
}

void FD3D12DynamicRHI::RHITransitionResource(FRHICommandList& RHICmdList, FRHITexture* InTexture, D3D12_RESOURCE_STATES InState, uint32 InSubResource)
{
	UE_LOGF(LogD3D12RHI, Error, "RHITransitionResource is no longer functional. Please use RHICmdList.Transition() instead.");
}

void FD3D12DynamicRHI::RHISignalManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value)
{
	checkf(FRHIGPUMask::All() == FRHIGPUMask::GPU0(), TEXT("RHISignalManualFence cannot be used by multi-GPU code"));
	const uint32 GPUIndex = 0;

	FD3D12CommandContext& Context = FD3D12CommandContext::Get(RHICmdList, GPUIndex);
	Context.SignalManualFence(Fence, Value);
}

void FD3D12DynamicRHI::RHIWaitManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled)
{
	checkf(FRHIGPUMask::All() == FRHIGPUMask::GPU0(), TEXT("RHIWaitManualFence cannot be used by multi-GPU code"));
	const uint32 GPUIndex = 0;

	FD3D12CommandContext& Context = FD3D12CommandContext::Get(RHICmdList, GPUIndex);
	Context.WaitManualFence(Fence, Value, bHasSignalled);
}

void FD3D12DynamicRHI::RHIVerifyResult(ID3D12Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, FString Message) const
{
	D3D12RHI::VerifyD3D12Result(Result, Code, Filename, Line, Device, Message);
}

void FD3D12DynamicRHI::RHIFlushResourceBarriers(FRHICommandListBase& RHICmdList, uint32 InGPUIndex)
{
	FD3D12CommandContext::Get(RHICmdList, InGPUIndex).FlushResourceBarriers();
}

void FD3D12DynamicRHI::RHIUpdateResourceResidency(FRHICommandListBase& RHICmdList, uint32 InGPUIndex, FRHIResource* InResource)
{
	if (InResource->GetType() == RRT_Buffer)
	{
		FD3D12Buffer* D3D12Buffer = ResourceCast(static_cast<FRHIBuffer*>(InResource), InGPUIndex);
		FD3D12CommandContext::Get(RHICmdList, InGPUIndex).UpdateResidency(D3D12Buffer->GetResource());
	}
	else if (InResource->GetType() == RRT_Texture || InResource->GetType() == RRT_TextureReference)
	{
		FD3D12Texture* D3D12Texture = GetD3D12TextureFromRHITexture(static_cast<FRHITexture*>(InResource), InGPUIndex);
		FD3D12CommandContext::Get(RHICmdList, InGPUIndex).UpdateResidency(D3D12Texture->GetResource());
	}
}

void* FD3D12DynamicRHI::RHIGetNativeDevice()
{
	return (void*)GetAdapter().GetD3DDevice();
}

void* FD3D12DynamicRHI::RHIGetNativeGraphicsQueue()
{
	return (void*)RHIGetCommandQueue();
}

void* FD3D12DynamicRHI::RHIGetNativeComputeQueue()
{
	return (void*)RHIGetCommandQueue();
}

/**
* Returns a supported screen resolution that most closely matches the input.
* @param Width - Input: Desired resolution width in pixels. Output: A width that the platform supports.
* @param Height - Input: Desired resolution height in pixels. Output: A height that the platform supports.
*/
void FD3D12DynamicRHI::RHIGetSupportedResolution(uint32& Width, uint32& Height)
{
	uint32 InitializedMode = false;
	DXGI_MODE_DESC BestMode;
	BestMode.Width = 0;
	BestMode.Height = 0;

	{
		TRefCountPtr<IDXGIAdapter> Adapter;
		HRESULT HResult = GetAdapter().EnumAdapters(Adapter.GetInitReference());
		if (DXGI_ERROR_NOT_FOUND == HResult)
		{
			return;
		}
		if (FAILED(HResult))
		{
			return;
		}

		// get the description of the adapter
		DXGI_ADAPTER_DESC AdapterDesc;
		VERIFYD3D12RESULT(Adapter->GetDesc(&AdapterDesc));

#if D3D12_PLATFORM_NEEDS_DISPLAY_MODE_ENUMERATION
		// Enumerate outputs for this adapter
		// TODO: Cap at 1 for default output
		for (uint32 o = 0; o < 1; o++)
		{
			TRefCountPtr<IDXGIOutput> Output;
			HResult = Adapter->EnumOutputs(o, Output.GetInitReference());
			if (DXGI_ERROR_NOT_FOUND == HResult)
			{
				break;
			}
			if (FAILED(HResult))
			{
				return;
			}

			// TODO: GetDisplayModeList is a terribly SLOW call.  It can take up to a second per invocation.
			//  We might want to work around some DXGI badness here.
			DXGI_FORMAT Format = DXGI_FORMAT_R8G8B8A8_UNORM;
			uint32 NumModes = 0;
			HResult = Output->GetDisplayModeList(Format, 0, &NumModes, NULL);
			if (HResult == DXGI_ERROR_NOT_FOUND)
			{
				return;
			}
			else if (HResult == DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
			{
				UE_LOGF(LogD3D12RHI, Fatal,
					"This application cannot be run over a remote desktop configuration"
					);
				return;
			}
			DXGI_MODE_DESC* ModeList = new DXGI_MODE_DESC[NumModes];
			VERIFYD3D12RESULT(Output->GetDisplayModeList(Format, 0, &NumModes, ModeList));

			for (uint32 m = 0; m < NumModes; m++)
			{
				// Search for the best mode

				// Suppress static analysis warnings about a potentially out-of-bounds read access to ModeList. This is a false positive - Index is always within range.
				CA_SUPPRESS(6385);
				bool IsEqualOrBetterWidth = FMath::Abs((int32)ModeList[m].Width - (int32)Width) <= FMath::Abs((int32)BestMode.Width - (int32)Width);
				bool IsEqualOrBetterHeight = FMath::Abs((int32)ModeList[m].Height - (int32)Height) <= FMath::Abs((int32)BestMode.Height - (int32)Height);
				if (!InitializedMode || (IsEqualOrBetterWidth && IsEqualOrBetterHeight))
				{
					BestMode = ModeList[m];
					InitializedMode = true;
				}
			}

			delete[] ModeList;
		}
#endif // D3D12_PLATFORM_NEEDS_DISPLAY_MODE_ENUMERATION
	}

	check(InitializedMode);
	Width = BestMode.Width;
	Height = BestMode.Height;
}

void FD3D12DynamicRHI::GetBestSupportedMSAASetting(DXGI_FORMAT PlatformFormat, uint32 MSAACount, uint32& OutBestMSAACount, uint32& OutMSAAQualityLevels)
{
	// start counting down from current setting (indicated the current "best" count) and move down looking for support
	for (uint32 SampleCount = MSAACount; SampleCount > 0; SampleCount--)
	{
		// The multisampleQualityLevels struct serves as both the input and output to CheckFeatureSupport.
		D3D12_FEATURE_DATA_MULTISAMPLE_QUALITY_LEVELS multisampleQualityLevels = {};
		multisampleQualityLevels.SampleCount = SampleCount;

		if (SUCCEEDED(GetAdapter().GetD3DDevice()->CheckFeatureSupport(D3D12_FEATURE_MULTISAMPLE_QUALITY_LEVELS, &multisampleQualityLevels, sizeof(multisampleQualityLevels))))
		{
			OutBestMSAACount = SampleCount;
			OutMSAAQualityLevels = multisampleQualityLevels.NumQualityLevels;
			break;
		}
	}

	return;
}

void FD3D12DynamicRHI::HandleGpuTimeout(FD3D12Payload* Payload, double SecondsSinceSubmission)
{
	UE_LOGF(LogD3D12RHI, Warning, "GPU timeout: A payload (0x%p) on the [0x%p, %ls] queue has not completed after %f seconds."
		, Payload
		, &Payload->Queue
		, GetD3DCommandQueueTypeName(Payload->Queue.QueueType)
		, SecondsSinceSubmission
	);

#if WITH_RHI_BREADCRUMBS
	DumpActiveBreadcrumbs(FRHIBreadcrumbState::EVerbosity::Warning);
#endif

	if (GD3D12RequestExitOnGPUTimeout)
	{
		GIsCriticalError = true;
		FPlatformMisc::RequestExit(true, TEXT("FD3D12DynamicRHI::HandleGpuTimeout"));
	}
}

void FD3D12DynamicRHI::SetupD3D12Debug()
{
	// Use a debug device if specified on the command line.
	if (FParse::Param(FCommandLine::Get(), TEXT("d3ddebug")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3debug")) ||
		FParse::Param(FCommandLine::Get(), TEXT("dxdebug")))
	{
		GD3D12DebugCvar->Set(1, ECVF_SetByCommandline);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("d3dlogwarnings")))
	{
		GD3D12DebugCvar->Set(2, ECVF_SetByCommandline);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("d3dbreakonwarning")))
	{
		GD3D12DebugCvar->Set(3, ECVF_SetByCommandline);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("d3dcontinueonerrors")))
	{
		GD3D12DebugCvar->Set(4, ECVF_SetByCommandline);
	}
	GRHIGlobals.IsDebugLayerEnabled = (GD3D12DebugCvar.GetValueOnAnyThread() > 0);

}

void FD3D12DynamicRHI::RHIRunOnQueue(ED3D12RHIRunOnQueueType QueueType, TFunction<void(ID3D12CommandQueue*)>&& CodeToRun, bool bWaitForSubmission)
{
	FGraphEventRef SubmissionEvent;

	TArray<FD3D12Payload*> Payloads;
	FD3D12Payload* Payload = new FD3D12Payload(GetRHIDevice(0)->GetQueue((QueueType == ED3D12RHIRunOnQueueType::Graphics) ? ED3D12QueueType::Direct : ED3D12QueueType::Copy));
	Payloads.Add(Payload);

	Payload->PreExecuteCallback = MoveTemp(CodeToRun);

	if (bWaitForSubmission)
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		Payload->SubmissionEvent = SubmissionEvent;
	}

	SubmitPayloads(MoveTemp(Payloads));
	
	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SubmissionEvent->Wait();
	}
}

const TCHAR* LexToString(DXGI_FORMAT Format)
{
	switch (Format)
	{
	default:
	case DXGI_FORMAT_UNKNOWN: return TEXT("DXGI_FORMAT_UNKNOWN");
	case DXGI_FORMAT_R32G32B32A32_TYPELESS: return TEXT("DXGI_FORMAT_R32G32B32A32_TYPELESS");
	case DXGI_FORMAT_R32G32B32A32_FLOAT: return TEXT("DXGI_FORMAT_R32G32B32A32_FLOAT");
	case DXGI_FORMAT_R32G32B32A32_UINT: return TEXT("DXGI_FORMAT_R32G32B32A32_UINT");
	case DXGI_FORMAT_R32G32B32A32_SINT: return TEXT("DXGI_FORMAT_R32G32B32A32_SINT");
	case DXGI_FORMAT_R32G32B32_TYPELESS: return TEXT("DXGI_FORMAT_R32G32B32_TYPELESS");
	case DXGI_FORMAT_R32G32B32_FLOAT: return TEXT("DXGI_FORMAT_R32G32B32_FLOAT");
	case DXGI_FORMAT_R32G32B32_UINT: return TEXT("DXGI_FORMAT_R32G32B32_UINT");
	case DXGI_FORMAT_R32G32B32_SINT: return TEXT("DXGI_FORMAT_R32G32B32_SINT");
	case DXGI_FORMAT_R16G16B16A16_TYPELESS: return TEXT("DXGI_FORMAT_R16G16B16A16_TYPELESS");
	case DXGI_FORMAT_R16G16B16A16_FLOAT: return TEXT("DXGI_FORMAT_R16G16B16A16_FLOAT");
	case DXGI_FORMAT_R16G16B16A16_UNORM: return TEXT("DXGI_FORMAT_R16G16B16A16_UNORM");
	case DXGI_FORMAT_R16G16B16A16_UINT: return TEXT("DXGI_FORMAT_R16G16B16A16_UINT");
	case DXGI_FORMAT_R16G16B16A16_SNORM: return TEXT("DXGI_FORMAT_R16G16B16A16_SNORM");
	case DXGI_FORMAT_R16G16B16A16_SINT: return TEXT("DXGI_FORMAT_R16G16B16A16_SINT");
	case DXGI_FORMAT_R32G32_TYPELESS: return TEXT("DXGI_FORMAT_R32G32_TYPELESS");
	case DXGI_FORMAT_R32G32_FLOAT: return TEXT("DXGI_FORMAT_R32G32_FLOAT");
	case DXGI_FORMAT_R32G32_UINT: return TEXT("DXGI_FORMAT_R32G32_UINT");
	case DXGI_FORMAT_R32G32_SINT: return TEXT("DXGI_FORMAT_R32G32_SINT");
	case DXGI_FORMAT_R32G8X24_TYPELESS: return TEXT("DXGI_FORMAT_R32G8X24_TYPELESS");
	case DXGI_FORMAT_D32_FLOAT_S8X24_UINT: return TEXT("DXGI_FORMAT_D32_FLOAT_S8X24_UINT");
	case DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS: return TEXT("DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS");
	case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT: return TEXT("DXGI_FORMAT_X32_TYPELESS_G8X24_UINT");
	case DXGI_FORMAT_R10G10B10A2_TYPELESS: return TEXT("DXGI_FORMAT_R10G10B10A2_TYPELESS");
	case DXGI_FORMAT_R10G10B10A2_UNORM: return TEXT("DXGI_FORMAT_R10G10B10A2_UNORM");
	case DXGI_FORMAT_R10G10B10A2_UINT: return TEXT("DXGI_FORMAT_R10G10B10A2_UINT");
	case DXGI_FORMAT_R11G11B10_FLOAT: return TEXT("DXGI_FORMAT_R11G11B10_FLOAT");
	case DXGI_FORMAT_R8G8B8A8_TYPELESS: return TEXT("DXGI_FORMAT_R8G8B8A8_TYPELESS");
	case DXGI_FORMAT_R8G8B8A8_UNORM: return TEXT("DXGI_FORMAT_R8G8B8A8_UNORM");
	case DXGI_FORMAT_R8G8B8A8_UNORM_SRGB: return TEXT("DXGI_FORMAT_R8G8B8A8_UNORM_SRGB");
	case DXGI_FORMAT_R8G8B8A8_UINT: return TEXT("DXGI_FORMAT_R8G8B8A8_UINT");
	case DXGI_FORMAT_R8G8B8A8_SNORM: return TEXT("DXGI_FORMAT_R8G8B8A8_SNORM");
	case DXGI_FORMAT_R8G8B8A8_SINT: return TEXT("DXGI_FORMAT_R8G8B8A8_SINT");
	case DXGI_FORMAT_R16G16_TYPELESS: return TEXT("DXGI_FORMAT_R16G16_TYPELESS");
	case DXGI_FORMAT_R16G16_FLOAT: return TEXT("DXGI_FORMAT_R16G16_FLOAT");
	case DXGI_FORMAT_R16G16_UNORM: return TEXT("DXGI_FORMAT_R16G16_UNORM");
	case DXGI_FORMAT_R16G16_UINT: return TEXT("DXGI_FORMAT_R16G16_UINT");
	case DXGI_FORMAT_R16G16_SNORM: return TEXT("DXGI_FORMAT_R16G16_SNORM");
	case DXGI_FORMAT_R16G16_SINT: return TEXT("DXGI_FORMAT_R16G16_SINT");
	case DXGI_FORMAT_R32_TYPELESS: return TEXT("DXGI_FORMAT_R32_TYPELESS");
	case DXGI_FORMAT_D32_FLOAT: return TEXT("DXGI_FORMAT_D32_FLOAT");
	case DXGI_FORMAT_R32_FLOAT: return TEXT("DXGI_FORMAT_R32_FLOAT");
	case DXGI_FORMAT_R32_UINT: return TEXT("DXGI_FORMAT_R32_UINT");
	case DXGI_FORMAT_R32_SINT: return TEXT("DXGI_FORMAT_R32_SINT");
	case DXGI_FORMAT_R24G8_TYPELESS: return TEXT("DXGI_FORMAT_R24G8_TYPELESS");
	case DXGI_FORMAT_D24_UNORM_S8_UINT: return TEXT("DXGI_FORMAT_D24_UNORM_S8_UINT");
	case DXGI_FORMAT_R24_UNORM_X8_TYPELESS: return TEXT("DXGI_FORMAT_R24_UNORM_X8_TYPELESS");
	case DXGI_FORMAT_X24_TYPELESS_G8_UINT: return TEXT("DXGI_FORMAT_X24_TYPELESS_G8_UINT");
	case DXGI_FORMAT_R8G8_TYPELESS: return TEXT("DXGI_FORMAT_R8G8_TYPELESS");
	case DXGI_FORMAT_R8G8_UNORM: return TEXT("DXGI_FORMAT_R8G8_UNORM");
	case DXGI_FORMAT_R8G8_UINT: return TEXT("DXGI_FORMAT_R8G8_UINT");
	case DXGI_FORMAT_R8G8_SNORM: return TEXT("DXGI_FORMAT_R8G8_SNORM");
	case DXGI_FORMAT_R8G8_SINT: return TEXT("DXGI_FORMAT_R8G8_SINT");
	case DXGI_FORMAT_R16_TYPELESS: return TEXT("DXGI_FORMAT_R16_TYPELESS");
	case DXGI_FORMAT_R16_FLOAT: return TEXT("DXGI_FORMAT_R16_FLOAT");
	case DXGI_FORMAT_D16_UNORM: return TEXT("DXGI_FORMAT_D16_UNORM");
	case DXGI_FORMAT_R16_UNORM: return TEXT("DXGI_FORMAT_R16_UNORM");
	case DXGI_FORMAT_R16_UINT: return TEXT("DXGI_FORMAT_R16_UINT");
	case DXGI_FORMAT_R16_SNORM: return TEXT("DXGI_FORMAT_R16_SNORM");
	case DXGI_FORMAT_R16_SINT: return TEXT("DXGI_FORMAT_R16_SINT");
	case DXGI_FORMAT_R8_TYPELESS: return TEXT("DXGI_FORMAT_R8_TYPELESS");
	case DXGI_FORMAT_R8_UNORM: return TEXT("DXGI_FORMAT_R8_UNORM");
	case DXGI_FORMAT_R8_UINT: return TEXT("DXGI_FORMAT_R8_UINT");
	case DXGI_FORMAT_R8_SNORM: return TEXT("DXGI_FORMAT_R8_SNORM");
	case DXGI_FORMAT_R8_SINT: return TEXT("DXGI_FORMAT_R8_SINT");
	case DXGI_FORMAT_A8_UNORM: return TEXT("DXGI_FORMAT_A8_UNORM");
	case DXGI_FORMAT_R1_UNORM: return TEXT("DXGI_FORMAT_R1_UNORM");
	case DXGI_FORMAT_R9G9B9E5_SHAREDEXP: return TEXT("DXGI_FORMAT_R9G9B9E5_SHAREDEXP");
	case DXGI_FORMAT_R8G8_B8G8_UNORM: return TEXT("DXGI_FORMAT_R8G8_B8G8_UNORM");
	case DXGI_FORMAT_G8R8_G8B8_UNORM: return TEXT("DXGI_FORMAT_G8R8_G8B8_UNORM");
	case DXGI_FORMAT_BC1_TYPELESS: return TEXT("DXGI_FORMAT_BC1_TYPELESS");
	case DXGI_FORMAT_BC1_UNORM: return TEXT("DXGI_FORMAT_BC1_UNORM");
	case DXGI_FORMAT_BC1_UNORM_SRGB: return TEXT("DXGI_FORMAT_BC1_UNORM_SRGB");
	case DXGI_FORMAT_BC2_TYPELESS: return TEXT("DXGI_FORMAT_BC2_TYPELESS");
	case DXGI_FORMAT_BC2_UNORM: return TEXT("DXGI_FORMAT_BC2_UNORM");
	case DXGI_FORMAT_BC2_UNORM_SRGB: return TEXT("DXGI_FORMAT_BC2_UNORM_SRGB");
	case DXGI_FORMAT_BC3_TYPELESS: return TEXT("DXGI_FORMAT_BC3_TYPELESS");
	case DXGI_FORMAT_BC3_UNORM: return TEXT("DXGI_FORMAT_BC3_UNORM");
	case DXGI_FORMAT_BC3_UNORM_SRGB: return TEXT("DXGI_FORMAT_BC3_UNORM_SRGB");
	case DXGI_FORMAT_BC4_TYPELESS: return TEXT("DXGI_FORMAT_BC4_TYPELESS");
	case DXGI_FORMAT_BC4_UNORM: return TEXT("DXGI_FORMAT_BC4_UNORM");
	case DXGI_FORMAT_BC4_SNORM: return TEXT("DXGI_FORMAT_BC4_SNORM");
	case DXGI_FORMAT_BC5_TYPELESS: return TEXT("DXGI_FORMAT_BC5_TYPELESS");
	case DXGI_FORMAT_BC5_UNORM: return TEXT("DXGI_FORMAT_BC5_UNORM");
	case DXGI_FORMAT_BC5_SNORM: return TEXT("DXGI_FORMAT_BC5_SNORM");
	case DXGI_FORMAT_B5G6R5_UNORM: return TEXT("DXGI_FORMAT_B5G6R5_UNORM");
	case DXGI_FORMAT_B5G5R5A1_UNORM: return TEXT("DXGI_FORMAT_B5G5R5A1_UNORM");
	case DXGI_FORMAT_B8G8R8A8_UNORM: return TEXT("DXGI_FORMAT_B8G8R8A8_UNORM");
	case DXGI_FORMAT_B8G8R8X8_UNORM: return TEXT("DXGI_FORMAT_B8G8R8X8_UNORM");
	case DXGI_FORMAT_R10G10B10_XR_BIAS_A2_UNORM: return TEXT("DXGI_FORMAT_R10G10B10_XR_BIAS_A2_UNORM");
	case DXGI_FORMAT_B8G8R8A8_TYPELESS: return TEXT("DXGI_FORMAT_B8G8R8A8_TYPELESS");
	case DXGI_FORMAT_B8G8R8A8_UNORM_SRGB: return TEXT("DXGI_FORMAT_B8G8R8A8_UNORM_SRGB");
	case DXGI_FORMAT_B8G8R8X8_TYPELESS: return TEXT("DXGI_FORMAT_B8G8R8X8_TYPELESS");
	case DXGI_FORMAT_B8G8R8X8_UNORM_SRGB: return TEXT("DXGI_FORMAT_B8G8R8X8_UNORM_SRGB");
	case DXGI_FORMAT_BC6H_TYPELESS: return TEXT("DXGI_FORMAT_BC6H_TYPELESS");
	case DXGI_FORMAT_BC6H_UF16: return TEXT("DXGI_FORMAT_BC6H_UF16");
	case DXGI_FORMAT_BC6H_SF16: return TEXT("DXGI_FORMAT_BC6H_SF16");
	case DXGI_FORMAT_BC7_TYPELESS: return TEXT("DXGI_FORMAT_BC7_TYPELESS");
	case DXGI_FORMAT_BC7_UNORM: return TEXT("DXGI_FORMAT_BC7_UNORM");
	case DXGI_FORMAT_BC7_UNORM_SRGB: return TEXT("DXGI_FORMAT_BC7_UNORM_SRGB");
	case DXGI_FORMAT_AYUV: return TEXT("DXGI_FORMAT_AYUV");
	case DXGI_FORMAT_Y410: return TEXT("DXGI_FORMAT_Y410");
	case DXGI_FORMAT_Y416: return TEXT("DXGI_FORMAT_Y416");
	case DXGI_FORMAT_NV12: return TEXT("DXGI_FORMAT_NV12");
	case DXGI_FORMAT_P010: return TEXT("DXGI_FORMAT_P010");
	case DXGI_FORMAT_P016: return TEXT("DXGI_FORMAT_P016");
	case DXGI_FORMAT_420_OPAQUE: return TEXT("DXGI_FORMAT_420_OPAQUE");
	case DXGI_FORMAT_YUY2: return TEXT("DXGI_FORMAT_YUY2");
	case DXGI_FORMAT_Y210: return TEXT("DXGI_FORMAT_Y210");
	case DXGI_FORMAT_Y216: return TEXT("DXGI_FORMAT_Y216");
	case DXGI_FORMAT_NV11: return TEXT("DXGI_FORMAT_NV11");
	case DXGI_FORMAT_AI44: return TEXT("DXGI_FORMAT_AI44");
	case DXGI_FORMAT_IA44: return TEXT("DXGI_FORMAT_IA44");
	case DXGI_FORMAT_P8: return TEXT("DXGI_FORMAT_P8");
	case DXGI_FORMAT_A8P8: return TEXT("DXGI_FORMAT_A8P8");
	case DXGI_FORMAT_B4G4R4A4_UNORM: return TEXT("DXGI_FORMAT_B4G4R4A4_UNORM");
	case DXGI_FORMAT_P208: return TEXT("DXGI_FORMAT_P208");
	case DXGI_FORMAT_V208: return TEXT("DXGI_FORMAT_V208");
	case DXGI_FORMAT_V408: return TEXT("DXGI_FORMAT_V408");
	case 189: return TEXT("DXGI_FORMAT_SAMPLER_FEEDBACK_MIN_MIP_OPAQUE");
	case 190: return TEXT("DXGI_FORMAT_SAMPLER_FEEDBACK_MIP_REGION_USED_OPAQUE");
	}
}

