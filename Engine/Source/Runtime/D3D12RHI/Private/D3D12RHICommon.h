// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  D3D12RHICommon.h: Common D3D12 RHI definitions
  =============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "RHIPipeline.h"

#include "D3D12ThirdParty.h"
#include "D3D12RHI.h"
#include "D3D12RHIDefinitions.h"

#include "Containers/StaticArray.h"
#include "MultiGPU.h"
#include "Stats/Stats.h"

class FD3D12Adapter;
class FD3D12Device;

class FD3D12CommandContext;
class FD3D12ContextArray;

template <typename ObjectType0, typename ObjectType1>
class TD3D12DualLinkedObjectIterator;

typedef uint16 CBVSlotMask;

static_assert(MAX_ROOT_CBVS <= MAX_CBS, "MAX_ROOT_CBVS must be <= MAX_CBS.");
static_assert((8 * sizeof(CBVSlotMask)) >= MAX_CBS, "CBVSlotMask isn't large enough to cover all CBs. Please increase the size.");
static_assert((8 * sizeof(CBVSlotMask)) >= MAX_ROOT_CBVS, "CBVSlotMask isn't large enough to cover all CBs. Please increase the size.");
static const CBVSlotMask GRootCBVSlotMask = (1 << MAX_ROOT_CBVS) - 1; // Mask for all slots that are used by root descriptors.
static const CBVSlotMask GDescriptorTableCBVSlotMask = static_cast<CBVSlotMask>((1<<MAX_CBS) - 1) & ~(GRootCBVSlotMask); // Mask for all slots that are used by a root descriptor table.

#if MAX_SRVS > 32
	typedef uint64 SRVSlotMask;
#else
	typedef uint32 SRVSlotMask;
#endif
static_assert((8 * sizeof(SRVSlotMask)) >= MAX_SRVS, "SRVSlotMask isn't large enough to cover all SRVs. Please increase the size.");

typedef uint32 SamplerSlotMask;
static_assert((8 * sizeof(SamplerSlotMask)) >= MAX_SAMPLERS, "SamplerSlotMask isn't large enough to cover all Samplers. Please increase the size.");

typedef uint16 UAVSlotMask;
static_assert((8 * sizeof(UAVSlotMask)) >= MAX_UAVS, "UAVSlotMask isn't large enough to cover all UAVs. Please increase the size.");

DECLARE_LOG_CATEGORY_EXTERN(LogD3D12RHI, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogD3D12GapRecorder, Log, All);

class FD3D12AdapterChild
{
protected:
	FD3D12Adapter* ParentAdapter;

public:
	FD3D12AdapterChild(FD3D12Adapter* InParent = nullptr) : ParentAdapter(InParent) {}

	FORCEINLINE FD3D12Adapter* GetParentAdapter() const
	{
		// If this fires an object was likely created with a default constructor i.e in an STL container
		// and is therefore an orphan
		check(ParentAdapter != nullptr);
		return ParentAdapter;
	}

	// To be used with delayed setup
	inline void SetParentAdapter(FD3D12Adapter* InParent)
	{
		check(ParentAdapter == nullptr);
		ParentAdapter = InParent;
	}
};

class FD3D12DeviceChild
{
protected:
	FD3D12Device* Parent;

public:
	FD3D12DeviceChild(FD3D12Device* InParent = nullptr) : Parent(InParent) {}

	FORCEINLINE FD3D12Device* GetParentDevice() const
	{
		// If this fires an object was likely created with a default constructor i.e in an STL container
		// and is therefore an orphan
		check(Parent != nullptr);
		return Parent;
	}

	FD3D12Device* GetParentDevice_Unsafe() const
	{
		return Parent;
	}
};

class FD3D12GPUObject
{
public:
	FD3D12GPUObject(FRHIGPUMask InGPUMask, FRHIGPUMask InVisibiltyMask)
#if WITH_MGPU
		: GPUMask(InGPUMask)
		, VisibilityMask(InVisibiltyMask)
#endif
	{
		// Note that node mask can't be null.
	}

#if WITH_MGPU
	FORCEINLINE FRHIGPUMask GetGPUMask() const { return GPUMask; }
	FORCEINLINE FRHIGPUMask GetVisibilityMask() const { return VisibilityMask; }
#else
	SGPU_CONSTEXPR FRHIGPUMask GetGPUMask() const { return FRHIGPUMask::GPU0(); }
	SGPU_CONSTEXPR FRHIGPUMask GetVisibilityMask() const { return FRHIGPUMask::GPU0(); }
#endif

protected:
#if WITH_MGPU
	const FRHIGPUMask GPUMask;
	// Which GPUs have direct access to this object
	const FRHIGPUMask VisibilityMask;
#endif
};

class FD3D12SingleNodeGPUObject : public FD3D12GPUObject
{
public:
	FD3D12SingleNodeGPUObject(FRHIGPUMask GPUMask)
		: FD3D12GPUObject(GPUMask, GPUMask)
#if WITH_MGPU
		, GPUIndex(GPUMask.ToIndex())
#endif
	{}

#if WITH_MGPU
	FORCEINLINE uint32 GetGPUIndex() const
	{
		return GPUIndex;
	}
#else
	SGPU_CONSTEXPR uint32 GetGPUIndex() const { return 0; }
#endif

private:
#if WITH_MGPU
	uint32 GPUIndex;
#endif
};

class FD3D12MultiNodeGPUObject : public FD3D12GPUObject
{
public:
	FD3D12MultiNodeGPUObject(FRHIGPUMask NodeMask, FRHIGPUMask VisibiltyMask)
		: FD3D12GPUObject(NodeMask, VisibiltyMask)
	{
		check(NodeMask.Intersects(VisibiltyMask));// A GPU objects must be visible on the device it belongs to
	}
};

template <typename ObjectType>
class FD3D12LinkedAdapterObject
{
public:
	using LinkedObjectType = ObjectType;
	using FDualLinkedObjectIterator = TD3D12DualLinkedObjectIterator<ObjectType, ObjectType>;

	~FD3D12LinkedAdapterObject()
	{
		if (IsHeadLink())
		{
			// When DoNotDeferDelete is set on these objects releasing it could cause the next iterator
			// object to be invalid. Therefore we need to accumulate first and then release.
			TArray<FLinkedObjectIterator, TInlineAllocator<MAX_NUM_GPUS>> ObjectsToBeReleased;

			// Accumulate and release the references we added in CreateLinkedObjects.
			for (auto It = ++FLinkedObjectIterator(this); It; ++It)
			{
				ObjectsToBeReleased.Add(It);
			}
			for (auto ObjectToRelease : ObjectsToBeReleased)
			{
				ObjectToRelease->Release();
			}
			ObjectsToBeReleased.Empty();
		}
	}

	FORCEINLINE bool IsHeadLink() const
	{
		return GetFirstLinkedObject() == this;
	}

	// The creation lambda function is passed FD3D12Device, and a pointer to the first linked object, or nullptr if the object being constructed
	// is itself the first object.  The first object pointer is needed for cases where resources need to be shared for all linked objects,
	// such as bindless resource descriptor handles, which need to be the same across GPUs for use in platform independent logic which generates
	// descriptor tables.
	template <typename ReturnType, typename CreationCoreFunction, typename CreationParameterFunction>
	static ReturnType* CreateLinkedObjects(FRHIGPUMask GPUMask, const CreationParameterFunction& pfnGetCreationParameter, const CreationCoreFunction& pfnCreationCore)
	{
		ReturnType* ObjectOut = nullptr;

#if WITH_MGPU
		for (uint32 GPUIndex : GPUMask)
		{
			ReturnType* NewObject = pfnCreationCore(pfnGetCreationParameter(GPUIndex), ObjectOut);
			CA_ASSUME(NewObject != nullptr);
			if (ObjectOut == nullptr)
			{
				// Don't AddRef the first object or we'll create a reference loop.
				ObjectOut = NewObject;
			}
			else
			{
				NewObject->AddRef();
			}
			ObjectOut->LinkedObjects.Objects[GPUIndex] = NewObject;
		}

		if (ObjectOut != nullptr)
		{
			ObjectOut->LinkedObjects.GPUMask = GPUMask;

			// Copy the LinkedObjects array to all of the other linked objects.
			for (auto GPUIterator = ++FRHIGPUMask::FIterator(GPUMask); GPUIterator; ++GPUIterator)
			{
				ObjectOut->GetLinkedObject(*GPUIterator)->LinkedObjects = ObjectOut->LinkedObjects;
			}
		}
#else
		check(GPUMask == FRHIGPUMask::GPU0());
		ObjectOut = pfnCreationCore(pfnGetCreationParameter(0), nullptr);
#endif

		return ObjectOut;
	}

	ObjectType* GetLinkedObject(uint32 GPUIndex) const
	{
#if WITH_MGPU
		return LinkedObjects.Objects[GPUIndex];
#else
		checkSlow(GPUIndex == 0);
		return GetFirstLinkedObject();
#endif
	}

	ObjectType* GetFirstLinkedObject() const
	{
#if WITH_MGPU
		return LinkedObjects.Objects[LinkedObjects.GPUMask.GetFirstIndex()];
#else
		return static_cast<ObjectType*>(const_cast<FD3D12LinkedAdapterObject*>(this));
#endif
	}

	FRHIGPUMask GetLinkedObjectsGPUMask() const
	{
#if WITH_MGPU
		return LinkedObjects.GPUMask;
#else
		return FRHIGPUMask::GPU0();
#endif
	}

	class FLinkedObjectIterator
	{
	public:
		explicit FLinkedObjectIterator(FD3D12LinkedAdapterObject* InObject)
			: GPUIterator(0)
			, Object(nullptr)
		{
			if (InObject != nullptr)
			{
				GPUIterator = FRHIGPUMask::FIterator(InObject->GetLinkedObjectsGPUMask());
				Object = InObject->GetLinkedObject(*GPUIterator);
			}
		}

		FLinkedObjectIterator& operator++()
		{
			Object = ++GPUIterator ? Object->GetLinkedObject(*GPUIterator) : nullptr;
			return *this;
		}

		explicit operator bool() const { return Object != nullptr; }
		bool operator !() const { return Object == nullptr; }

		bool operator ==(FLinkedObjectIterator& Other) const { return Object == Other.Object; }
		bool operator !=(FLinkedObjectIterator& Other) const { return Object != Other.Object; }

		ObjectType& operator *() const { return *Object; }
		ObjectType* operator ->() const { return Object; }
		ObjectType* Get() const { return Object; }

	private:
		FRHIGPUMask::FIterator GPUIterator;
		ObjectType* Object;
	};

	FLinkedObjectIterator begin() { return FLinkedObjectIterator(this); }
	FLinkedObjectIterator end() { return FLinkedObjectIterator(nullptr); }

protected:
	FD3D12LinkedAdapterObject() {}

private:
#if WITH_MGPU
	struct FLinkedObjects
	{
		FLinkedObjects() 
			: Objects(InPlace, nullptr)
		{}

		FRHIGPUMask GPUMask;
		TStaticArray<ObjectType*, MAX_NUM_GPUS> Objects;
	};
	FLinkedObjects LinkedObjects;
#endif // WITH_MGPU
};

/**
 * Utility for iterating over a pair of FD3D12LinkedAdapterObjects. The linked
 * objects must have identical GPU masks. Useful for copying data from one object
 * list to another and for updating resource views.
 */
template <typename ObjectType0, typename ObjectType1>
class TD3D12DualLinkedObjectIterator
{
public:
	TD3D12DualLinkedObjectIterator(FD3D12LinkedAdapterObject<typename ObjectType0::LinkedObjectType>* InObject0, FD3D12LinkedAdapterObject<typename ObjectType1::LinkedObjectType>* InObject1)
		: GPUIterator(0)
	{
		const FRHIGPUMask GPUMask = InObject0->GetLinkedObjectsGPUMask();
		check(GPUMask == InObject1->GetLinkedObjectsGPUMask());

		GPUIterator = FRHIGPUMask::FIterator(GPUMask);
		Object0 = static_cast<ObjectType0*>(InObject0->GetLinkedObject(*GPUIterator));
		Object1 = static_cast<ObjectType1*>(InObject1->GetLinkedObject(*GPUIterator));
	}

	TD3D12DualLinkedObjectIterator& operator++()
	{
		if (++GPUIterator)
		{
			Object0 = static_cast<ObjectType0*>(Object0->GetLinkedObject(*GPUIterator));
			Object1 = static_cast<ObjectType1*>(Object1->GetLinkedObject(*GPUIterator));
		}
		else
		{
			Object0 = nullptr;
			Object1 = nullptr;
		}
		return *this;
	}

	explicit operator bool() const { return static_cast<bool>(GPUIterator); }
	bool operator !() const { return !GPUIterator; }

	ObjectType0* GetFirst() const { return Object0; }
	ObjectType1* GetSecond() const { return Object1; }
	uint32 GetGPUIndex() const { return *GPUIterator; }

private:
	FRHIGPUMask::FIterator GPUIterator;
	ObjectType0* Object0;
	ObjectType1* Object1;
};

// Template helper class for converting RHI resource types to D3D12RHI resource types
template<class T>
struct TD3D12ResourceTraits
{
};

// Lock free pointer list that auto-destructs items remaining in the list.
template <typename TObjectType>
struct TD3D12ObjectPool : public TLockFreePointerListUnordered<TObjectType, PLATFORM_CACHE_LINE_SIZE>
{
	~TD3D12ObjectPool()
	{
		CleanupResources();
	}

	void CleanupResources()
	{
		while (TObjectType* Object = TLockFreePointerListUnordered<TObjectType, PLATFORM_CACHE_LINE_SIZE>::Pop())
		{
			delete Object;
		}
	}
};

// Set of validation to enable in all builds (including shipping)

#ifndef ENABLE_COPY_BUFFER_REGION_CHECK
	#define ENABLE_COPY_BUFFER_REGION_CHECK  1
#endif // ENABLE_COPY_BUFFER_REGION_CHECK

#ifndef ENABLE_COPY_TEXTURE_REGION_CHECK
	#define ENABLE_COPY_TEXTURE_REGION_CHECK 1
#endif // ENABLE_COPY_TEXTURE_REGION_CHECK
