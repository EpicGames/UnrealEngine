// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#include COMPILED_PLATFORM_HEADER(D3D12RHIDefinitions.h)

// DX12 doesn't support higher MSAA count
#define DX_MAX_MSAA_COUNT	8

// How many residency packets can be in flight before the rendering thread
// blocks for them to drain. Should be ~ NumBufferedFrames * AvgNumSubmissionsPerFrame i.e.
// enough to ensure that the GPU is rarely blocked by residency work
#define RESIDENCY_PIPELINE_DEPTH	6

#define DEBUG_RESOURCE_STATES	0

// Useful for debugging. Only flushes the default context though.
#define D3D12RHI_IDLE_AFTER_EVERY_GPU_EVENT 0

#define D3D12_RHI_RAYTRACING (RHI_RAYTRACING)

#if D3D12_MAX_COMMANDLIST_INTERFACE >= 10
	#define D3D12_RHI_WORKGRAPHS 1
	#define D3D12_RHI_WORKGRAPHS_GRAPHICS 0
#else
	#define D3D12_RHI_WORKGRAPHS 0
	#define D3D12_RHI_WORKGRAPHS_GRAPHICS 0
#endif

#ifndef D3D12RHI_ALLOW_ENHANCED_BARRIERS
	#error "Please define D3D12RHI_ALLOW_ENHANCED_BARRIERS in <Platform>D3D12RHIDefinitions.h!"
#endif

#ifndef D3D12RHI_ALLOW_LEGACY_BARRIERS
	#error "Please define D3D12RHI_ALLOW_LEGACY_BARRIERS in <Platform>D3D12RHIDefinitions.h!"
#endif

#if D3D12_MAX_DEVICE_INTERFACE >= 11
	#define D3D12RHI_SUPPORTS_ENHANCED_BARRIERS (D3D12RHI_ALLOW_ENHANCED_BARRIERS)
#else
	#define D3D12RHI_SUPPORTS_ENHANCED_BARRIERS 0
#endif

#define D3D12RHI_SUPPORTS_LEGACY_BARRIERS (D3D12RHI_ALLOW_LEGACY_BARRIERS)

#ifndef D3D12RHI_BARRIER_LOGGING
	#define D3D12RHI_BARRIER_LOGGING !(UE_BUILD_SHIPPING)
#endif

#if D3D12_MAX_DEVICE_INTERFACE >= 12
	#define D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV 1
#else
	#define D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV 0
#endif

#if WITH_NVAPI && (D3D12_MAX_DEVICE_INTERFACE >= 5) && D3D12_RHI_RAYTRACING && !UE_BUILD_SHIPPING
	#define WITH_NVAPI_RAYTRACING_VALIDATION 1
#else
	#define WITH_NVAPI_RAYTRACING_VALIDATION 0
#endif

// This value controls how many root constant buffers can be used per shader stage in a root signature.
// Note: Using root descriptors significantly increases the size of root signatures (each root descriptor is 2 DWORDs).
#define MAX_ROOT_CBVS	MAX_CBS

#define EXECUTE_DEBUG_COMMAND_LISTS 0
#define NAME_OBJECTS RHI_USE_RESOURCE_DEBUG_NAME
#define LOG_PSO_CREATES (0 && STATS)	// Logs Create Pipeline State timings (also requires STATS)
#define TRACK_RESOURCE_ALLOCATIONS (PLATFORM_WINDOWS && !UE_BUILD_SHIPPING && !UE_BUILD_TEST)

#define READBACK_BUFFER_POOL_MAX_ALLOC_SIZE (64 * 1024)
#define READBACK_BUFFER_POOL_DEFAULT_POOL_SIZE (4 * 1024 * 1024)

#define TEXTURE_POOL_SIZE (8 * 1024 * 1024)

#if DEBUG_RESOURCE_STATES
	#define LOG_EXECUTE_COMMAND_LISTS 1
	#define LOG_PRESENT 1
#else
	#define LOG_EXECUTE_COMMAND_LISTS 0
	#define LOG_PRESENT 0
#endif

#define DEBUG_FRAME_TIMING 0

#if DEBUG_FRAME_TIMING
	#define LOG_VIEWPORT_EVENTS 1
	#define LOG_PRESENT 1
	#define LOG_EXECUTE_COMMAND_LISTS 1
#else
	#define LOG_VIEWPORT_EVENTS 0
#endif

#if EXECUTE_DEBUG_COMMAND_LISTS
	#define DEBUG_EXECUTE_COMMAND_LIST(scope) if ((scope)->ActiveQueries == 0) { (scope)->FlushCommands(ED3D12FlushFlags::WaitForCompletion); }
	#define DEBUG_EXECUTE_COMMAND_CONTEXT(context) if ((context).ActiveQueries == 0) { (context).FlushCommands(ED3D12FlushFlags::WaitForCompletion); }
	#define DEBUG_RHI_EXECUTE_COMMAND_LIST(scope) if ((scope)->GetRHIDevice(0)->GetDefaultCommandContext().ActiveQueries == 0) { (scope)->GetRHIDevice(0)->GetDefaultCommandContext().FlushCommands(ED3D12FlushFlags::WaitForCompletion); }
#else
	#define DEBUG_EXECUTE_COMMAND_LIST(scope) 
	#define DEBUG_EXECUTE_COMMAND_CONTEXT(context) 
	#define DEBUG_RHI_EXECUTE_COMMAND_LIST(scope) 
#endif

#ifndef D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	#define D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN 1
#endif

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	#ifndef DXGI_PRESENT_ALLOW_TEARING
		#define DXGI_PRESENT_ALLOW_TEARING          0x00000200UL
		#define DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING  2048
	#endif
#endif

#if D3D12_SUPPORTS_INFO_QUEUE
	#define EMBED_DXGI_ERROR_LIST(PerEntry, Terminator)	\
		PerEntry(DXGI_ERROR_UNSUPPORTED) Terminator \
		PerEntry(DXGI_ERROR_NOT_CURRENT) Terminator \
		PerEntry(DXGI_ERROR_MORE_DATA) Terminator \
		PerEntry(DXGI_ERROR_MODE_CHANGE_IN_PROGRESS) Terminator \
		PerEntry(DXGI_ERROR_ALREADY_EXISTS) Terminator \
		PerEntry(DXGI_ERROR_SESSION_DISCONNECTED) Terminator \
		PerEntry(DXGI_ERROR_ACCESS_DENIED) Terminator \
		PerEntry(DXGI_ERROR_NON_COMPOSITED_UI) Terminator \
		PerEntry(DXGI_ERROR_CACHE_FULL) Terminator \
		PerEntry(DXGI_ERROR_NOT_CURRENTLY_AVAILABLE) Terminator \
		PerEntry(DXGI_ERROR_CACHE_CORRUPT) Terminator \
		PerEntry(DXGI_ERROR_WAIT_TIMEOUT) Terminator \
		PerEntry(DXGI_ERROR_FRAME_STATISTICS_DISJOINT) Terminator \
		PerEntry(DXGI_ERROR_DYNAMIC_CODE_POLICY_VIOLATION) Terminator \
		PerEntry(DXGI_ERROR_REMOTE_OUTOFMEMORY) Terminator \
		PerEntry(DXGI_ERROR_ACCESS_LOST) Terminator
#endif

// Note: the following defines depend on D3D12RHIBasePrivate.h

//@TODO: Improve allocator efficiency so we can increase these thresholds and improve performance
// We measured 149MB of wastage in 340MB of allocations with DEFAULT_BUFFER_POOL_MAX_ALLOC_SIZE set to 512KB
#if !defined(DEFAULT_BUFFER_POOL_MAX_ALLOC_SIZE)
	#if D3D12_RHI_RAYTRACING
		// #dxr_todo: Reevaluate these values. Currently optimized to reduce number of CreateCommitedResource() calls, at the expense of memory use.
		#define DEFAULT_BUFFER_POOL_MAX_ALLOC_SIZE    (64 * 1024 * 1024)
		#define DEFAULT_BUFFER_POOL_DEFAULT_POOL_SIZE (16 * 1024 * 1024)
	#else
		// On PC, buffers are 64KB aligned, so anything smaller should be sub-allocated
		#define DEFAULT_BUFFER_POOL_MAX_ALLOC_SIZE    (64 * 1024)
		#define DEFAULT_BUFFER_POOL_DEFAULT_POOL_SIZE (8 * 1024 * 1024)
	#endif //D3D12_RHI_RAYTRACING
#endif
