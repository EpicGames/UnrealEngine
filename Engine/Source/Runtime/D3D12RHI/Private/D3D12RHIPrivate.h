// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RHIPrivate.h: Private D3D RHI definitions.
	=============================================================================*/

#pragma once

#include "D3D12RHICommon.h"
#include "ID3D12DynamicRHI.h"

#include "CoreMinimal.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadManager.h"
#include "Containers/ResourceArray.h"
#include "Serialization/MemoryReader.h"
#include "StaticBoundShaderState.h"

// Dependencies.
#include "DXGIUtilities.h"
#include "GPUProfiler.h"
#include "ShaderCore.h"
#include "HDRHelper.h"

#include "D3D12Submission.h"
#include "D3D12RHIDefinitions.h"

// TODO reorder includes so we just include D3D12PipelineState.h here
#include COMPILED_PLATFORM_HEADER(D3D12PipelineState.h)

#include "D3D12DiskCache.h"
#include "D3D12NvidiaExtensions.h"
#include "D3D12IntelExtensions.h"
#include "D3D12Residency.h"

// D3D RHI public headers.
#include "D3D12Util.h"
#include "D3D12State.h"
#include "D3D12Resources.h"
#include "D3D12RootSignature.h"
#include "D3D12Shader.h"
#include "D3D12View.h"
#include "D3D12CommandList.h"
#include "D3D12Texture.h"
#include "D3D12DirectCommandListManager.h"
#include "D3D12Viewport.h"
#include "D3D12ConstantBuffer.h"
#include "D3D12Query.h"
#include "D3D12DescriptorCache.h"
#include "D3D12StateCachePrivate.h"
#include "D3D12Allocation.h"
#include "D3D12TransientResourceAllocator.h"
#include "D3D12CommandContext.h"
#include "D3D12Stats.h"
#include "D3D12Device.h"
#include "D3D12Adapter.h"
#include "D3D12Access.h"

template< typename t_A, typename t_B >
inline t_A RoundUpToNextMultiple(const t_A& a, const t_B& b)
{
	return ((a - 1) / b + 1) * b;
}

using namespace D3D12RHI;

extern TAutoConsoleVariable<int32> GD3D12DebugCvar;

static bool D3D12RHI_ShouldCreateWithWarp()
{
	// Use the warp adapter if specified on the command line.
	static bool bCreateWithWarp = FParse::Param(FCommandLine::Get(), TEXT("warp"));
	return bCreateWithWarp;
}

static bool D3D12RHI_AllowSoftwareFallback()
{
	static bool bAllowSoftwareRendering = FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));
	return bAllowSoftwareRendering;
}

static bool D3D12RHI_ShouldAllowAsyncResourceCreation()
{
	static bool bAllowAsyncResourceCreation = !FParse::Param(FCommandLine::Get(), TEXT("nod3dasync"));
	return bAllowAsyncResourceCreation;
}

static bool D3D12RHI_ShouldForceCompatibility()
{
	// Suppress the use of newer D3D12 features.
	static bool bForceCompatibility =
		FParse::Param(FCommandLine::Get(), TEXT("d3dcompat")) ||
		FParse::Param(FCommandLine::Get(), TEXT("d3d12compat"));
	return bForceCompatibility;
}

static bool D3D12RHI_IsRenderDocPresent(ID3D12Device* Device)
{
	IID RenderDocID;
	if (SUCCEEDED(IIDFromString(L"{A7AA6116-9C8D-4BBA-9083-B4D816B71B78}", &RenderDocID)))
	{
		TRefCountPtr<IUnknown> RenderDoc;
		if (SUCCEEDED(Device->QueryInterface(RenderDocID, (void**)RenderDoc.GetInitReference())))
		{
			return true;
		}
	}

	return false;
}

struct FD3D12UpdateTexture3DData
{
	FD3D12ResourceLocation* UploadHeapResourceLocation;
	bool bComputeShaderCopy;
};

/**
* Structure that represents various RTPSO properties (0 if unknown).
* These can be used to report performance characteristics, sort shaders by occupancy, etc.
*/
struct FD3D12RayTracingPipelineInfo
{
	static constexpr uint32 MaxPerformanceGroups = 10;

	// Estimated RTPSO group based on occupancy or other platform-specific heuristics.
	// Group 0 is expected to be performing worst, 9 (MaxPerformanceGroups-1) is expected to be the best.
	uint32 PerformanceGroup = 0;

	uint32 NumVGPR = 0;
	uint32 NumSGPR = 0;
	uint32 StackSize = 0;
	uint32 ScratchSize = 0;
};

/**
* Struct used for driver work around on certain driver versions
*/
struct FD3D12WorkaroundFlags
{
};

extern FD3D12WorkaroundFlags GD3D12WorkaroundFlags;

/** Forward declare the context for the AMD AGS utility library. */
struct AGSContext;

struct INTCExtensionContext;

/** The interface which is implemented by the dynamically bound RHI. */
class FD3D12DynamicRHI : public ID3D12PlatformDynamicRHI
{
	friend class FD3D12CommandContext;

	static FD3D12DynamicRHI* SingleD3DRHI;
	static bool bFormatAliasedTexturesMustBeCreatedUsingCommonLayout;

public:

	static FD3D12DynamicRHI* GetD3DRHI() { return SingleD3DRHI; }

	void TerminateOnOutOfMemory(ID3D12Device* InDevice, HRESULT D3DResult, bool bCreatingTextures);

	// Dump & Log all the information we have on a GPU crash (NvAfterMath, DRED, Breadcrumbs, ...)
	void OutputGPUCrashReport(FTextBuilder& ErrorMessage);

	// Returns true if the calling thread is the interrupt thread
	// (or acting as the interrupt thread when D3D12_USE_INTERRUPT_THREAD is zero).
	bool IsInInterruptThread() const;

	// Main crash function for handling D3D API errors
	void HandleFailedD3D12Result(HRESULT D3DResult, ID3D12Device* Device, bool bCreatingTextures, const TCHAR* Message);

#if WITH_RHI_BREADCRUMBS
	// Prints the active breadcrumb tree to the log.
	// Can only be called on the interrupt thread.
	void DumpActiveBreadcrumbs(FRHIBreadcrumbState::EVerbosity Verbosity);
#endif

private:
	// Calls OutputGpuCrashReport, and also forces a quit.
	void TerminateOnGPUCrash(TOptional<HRESULT> D3DResult);

	/** Texture pool size */
	int64 RequestedTexturePoolSize;

	friend class FD3D12Thread;
	class FD3D12Thread* SubmissionThread = nullptr;
	class FD3D12Thread* InterruptThread = nullptr;

	// Used when a dedicated interrupt thread is disabled (D3D12_USE_INTERRUPT_THREAD)
	// Tracks the thread ID of the thread currently acting as the interrupt thread.
	uint32 InterruptThreadID = 0;

	enum class EQueueStatus
	{
		None      = 0,

		// Work was processed through the queue.
		Processed = 1 << 0,

		// The queue has further, unprocessed work.
		Pending   = 1 << 1
	};
	FRIEND_ENUM_CLASS_FLAGS(EQueueStatus);

	struct FProcessResult
	{
		EQueueStatus Status = EQueueStatus::None;
		uint32 WaitTimeout = INFINITE;
	};

	TQueue<TArray<FD3D12Payload*>*, EQueueMode::Mpsc> PendingPayloadsForSubmission;

	// Ensures all prior batched command lists have reached the driver ID3D12Queue object.
	void FlushBatchedPayloads(FD3D12Queue::FPayloadArray& PayloadsToHandDown);

	// Executes reserved resource commit operations and cross-queue sync when required
	void UpdateReservedResources(FD3D12Payload* Payload);

	FCriticalSection SubmissionCS;
	FCriticalSection InterruptCS;

	FProcessResult ProcessSubmissionQueue();
	FProcessResult ProcessInterruptQueue();

	FCriticalSection ObjectsToDeleteCS;
	TArray<FD3D12DeferredDeleteObject> ObjectsToDelete;
	
	virtual void UpdateMemoryStats();

#if WITH_NVAPI_RAYTRACING_VALIDATION
	void FlushRaytracingValidationMessages();
#endif

public:
	template <typename ...Args>
	void DeferredDelete(Args&&... InArgs)
	{
		FScopeLock Lock(&ObjectsToDeleteCS);
		ObjectsToDelete.Emplace(Forward<Args>(InArgs)...);
	}

	void SubmitCommands(TConstArrayView<struct FD3D12FinalizedCommands*> Commands);
	void SubmitPayloads(TArray<FD3D12Payload*>&& Payloads);

	// Processes the interrupt queue on the calling thread, until the specified GraphEvent is signaled.
	// If the GraphEvent is nullptr, processes the queue until no further progress is made.
	void ProcessInterruptQueueUntil(FGraphEvent* GraphEvent);

	// Called when the GPU has crashed. This function will not return.
	void ProcessInterruptQueueOnGPUCrash(HRESULT D3DResult);

	struct FD3D12TimingArray : public TArray<TUniquePtr<FD3D12Timing>, TInlineAllocator<GD3D12MaxNumQueues>>
	{
		FD3D12Timing* CreateNew(FD3D12Queue& Queue)
		{
			return Emplace_GetRef(MakeUnique<FD3D12Timing>(Queue)).Get();
		}
	};

	FD3D12TimingArray CurrentTimingPerQueue;
	void ProcessTimestamps(FD3D12TimingArray const& TimingPerQueue);

	void InitializeSubmissionPipe();
	void ShutdownSubmissionPipe();

	// Inserts a task graph task which is executed once all previously submitted GPU work has completed (across all queues, device and adapters).
	void EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FD3D12Payload&)> ModifyPayloadCallback = {});
	FGraphEventRef EopTask;

	// Enumerates all queues across all devices and active adapters
	void ForEachQueue(TFunctionRef<void(FD3D12Queue&)> Callback);

	/** Initialization constructor. */
	FD3D12DynamicRHI(TSharedPtr<FD3D12Adapter>& InChosenAdapter, bool bInPixEventEnabled);

	/** Destructor */
	virtual ~FD3D12DynamicRHI();

	// FDynamicRHI interface.
	virtual void Init() override;
	virtual void PostInit() override;
	virtual void Shutdown() override;
	virtual const TCHAR* GetName() override { return TEXT("D3D12"); }

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE TReturnType* ResourceCast(TRHIType* Resource)
	{
		return static_cast<TReturnType*>(Resource);
	}

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE const TReturnType* ResourceCast(const TRHIType* Resource)
	{
		return static_cast<const TReturnType*>(Resource);
	}

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE_DEBUGGABLE TReturnType* ResourceCast(TRHIType* Resource, uint32 GPUIndex)
	{
		TReturnType* Object = ResourceCast<TRHIType, TReturnType>(Resource);
		return Object ? static_cast<TReturnType*>(Object->GetLinkedObject(GPUIndex)) : nullptr;
	}

	template<typename TRHIType, typename TReturnType = typename TD3D12ResourceTraits<TRHIType>::TConcreteType>
	static FORCEINLINE_DEBUGGABLE const TReturnType* ResourceCast(const TRHIType* Resource, uint32 GPUIndex)
	{
		const TReturnType* Object = ResourceCast<const TRHIType, const TReturnType>(Resource);
		return Object ? static_cast<const TReturnType*>(Object->GetLinkedObject(GPUIndex)) : nullptr;
	}

	virtual bool QueueSupportsTileMapping(ED3D12QueueType /*InQueueType*/) { return true; }

	virtual FD3D12CommandContext* CreateCommandContext(FD3D12Device* InParent, ED3D12QueueType InQueueType, bool InIsDefaultContext);
	virtual void CreateCommandQueue(FD3D12Device* Device, const D3D12_COMMAND_QUEUE_DESC& Desc, TRefCountPtr<ID3D12CommandQueue>& OutCommandQueue);

	virtual void RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList) final override;
	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;

	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override;
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override;
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override;
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override;
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override;
	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) override;
	virtual FWorkGraphShaderRHIRef RHICreateWorkGraphShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency) override;
	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName& Name) final override;
	virtual FStagingBufferRHIRef RHICreateStagingBuffer() final override;
	virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
    virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) final override;
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) final override;
	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual FComputePipelineStateRHIRef RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) final override;
	virtual FWorkGraphPipelineStateRHIRef RHICreateWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer) final override;
	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo) final override;
	virtual void RHIReleaseTransition(FRHITransition* Transition) final override;
	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override;
	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override;

	[[nodiscard]] virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) override;

	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override;
	virtual void RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr) final override;

	virtual void* RHILockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	virtual void* RHILockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	virtual void RHIUnlockBuffer(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#endif
	virtual void RHIUnlockBufferMGPU(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 GPUIndex) final override;
	virtual FTextureReferenceRHIRef RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture) final override;
	virtual void RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* NewTexture) final override;
	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(const FRHITextureDesc& Desc, uint32 FirstMipIndex) override;
	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override;
	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData, int32 SizeX, int32 SizeY, int32 Pitch, int32 PixelSize) final override;
	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override;
	
	[[nodiscard]] virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) override;

	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override;

	virtual void RHIGetMemoryStats(FRHIMemoryStats& OutStats) final override;

	virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	virtual void                  RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override;
	virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override;
	virtual FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion) final override;
	virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData) final override;
	virtual void RHIEndMultiUpdateTexture3D(FRHICommandListBase& RHICmdList, TArray<FUpdateTexture3DData>& UpdateDataArray) final override;
	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override;
	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, const TCHAR* Name) final override;
	virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override;
	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) final override;
	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex) final override;
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData) final override;
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture, FIntRect Rect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags) final override;
	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override;
	virtual void RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	virtual void RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	virtual void RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) final override;
	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override;
	virtual uint32 RHIGetViewportNextPresentGPUIndex(FRHIViewport* Viewport) final override;
	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override;
#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
	virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SrcTexture) final override;
	virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture) final override;
#endif
	virtual void RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation) final override;
	virtual uint64 RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS) final override;
	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override;
#if PLATFORM_WINDOWS
	virtual void RHIHandleDisplayChange() final override;
#endif
	virtual void RHITick(float DeltaTime) final override;
	virtual void RHIBlockUntilGPUIdle() final override;
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override;
	virtual bool RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle) final override;
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override;
	virtual void* RHIGetNativeDevice() final override;
	virtual void* RHIGetNativeGraphicsQueue() final override;
	virtual void* RHIGetNativeComputeQueue() final override;
	virtual class IRHICommandContext* RHIGetDefaultContext() final override;
	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override;
	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override;
	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override;
	virtual void RHIProcessDeleteQueue() final override;

	// Some RHIs can be suspended, most likely due to the HW going into suspend mode, this allows device dependent threads to wait until it's safe
	virtual FRWLock* RHIGetSuspendedLock() { return nullptr; }

	virtual void RHIRunOnQueue(ED3D12RHIRunOnQueueType QueueType, TFunction<void(ID3D12CommandQueue*)>&& CodeToRun, bool bWaitForSubmission) final override;
	virtual void RHIFlushResourceBarriers(FRHICommandListBase& RHICmdList, uint32 InGPUIndex) final override;

	// Allows external plugins to add the given FRHIResource's residency handle to the current RHICmdList's context.
	// This ensures resources used by external plugins and made resident before the work begins on the GPU.
	virtual void RHIUpdateResourceResidency(FRHICommandListBase& RHICmdList, uint32 InGPUIndex, FRHIResource* InResource) final override;

	virtual IRHITransientResourceAllocator* RHICreateTransientResourceAllocator() override;

	virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) final override;

	// SRV / UAV creation functions
	virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) override;
	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) override;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	virtual FRHIResourceCollectionRef RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers) override;
	virtual void RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates) override;

	virtual FDescriptorRangeRHIRef RHICreateDescriptorRange(FRHICommandListBase& RHICmdList, const FRHIDescriptorRangeDesc& Desc, TConstArrayView<FRHIDescriptorResource> InitialResources) override;
	virtual void RHIUpdateDescriptorRange(FRHICommandListBase& RHICmdList, FRHIDescriptorRange* Range, uint32 StartIndex, TConstArrayView<FRHIDescriptorResource> Resources) override;
#endif

	// ID3D12DynamicRHI interface.
	virtual TArray<FD3D12MinimalAdapterDesc> RHIGetAdapterDescs() const final override;
	virtual bool RHIIsPixEnabled() const final override;
	virtual ID3D12CommandQueue* RHIGetCommandQueue() const final override;
	virtual ID3D12Device* RHIGetDevice_NoMGPU() const final override;
	virtual ID3D12Device* RHIGetDevice(uint32 InIndex) const final override;
	virtual uint32 RHIGetDeviceNodeMask(uint32 InIndex) const final override;
	virtual ID3D12GraphicsCommandList* RHIGetGraphicsCommandList(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex) const final override;
	virtual FRHIViewport* RHIGetSingletonViewport() const final override;
	virtual DXGI_FORMAT RHIGetSwapChainFormat(EPixelFormat InFormat) const final override;
	virtual FTextureRHIRef RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) final override;
	virtual FTextureRHIRef RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) final override;
	virtual FTextureRHIRef RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) final override;
	virtual ID3D12Resource* RHIGetResource(FRHIBuffer* InBuffer) const final override;
	virtual uint32 RHIGetResourceDeviceIndex(FRHIBuffer* InBuffer) const final override;
	virtual int64 RHIGetResourceMemorySize(FRHIBuffer* InBuffer) const final override;
	virtual bool RHIIsResourcePlaced(FRHIBuffer* InBuffer) const final override;
	virtual ID3D12Resource* RHIGetResource(FRHITexture* InTexture) const final override;
	virtual uint32 RHIGetResourceDeviceIndex(FRHITexture* InTexture) const final override;
	virtual int64 RHIGetResourceMemorySize(FRHITexture* InTexture) const final override;
	virtual bool RHIIsResourcePlaced(FRHITexture* InTexture) const final override;
	virtual D3D12_CPU_DESCRIPTOR_HANDLE RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex = 0, int32 InArraySliceIndex = 0) const final override;
	virtual void RHIFinishExternalComputeWork(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex, ID3D12GraphicsCommandList* InCommandList) final override;
	virtual void RHITransitionResource(FRHICommandList& RHICmdList, FRHITexture* InTexture, D3D12_RESOURCE_STATES InState, uint32 InSubResource) final override;
	virtual void RHISignalManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value) final override;
	virtual void RHIWaitManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled) final override;
	virtual void RHIVerifyResult(ID3D12Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, FString Message = FString()) const final override;

#if D3D12_RHI_RAYTRACING

	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer) final override;
	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer) final override;
	virtual FRayTracingClusterOperationSize RHICalcRayTracingClusterOperationSize(const FRayTracingClusterOperationInitializer& Initializer) final override;

	virtual FRayTracingAccelerationStructureOfflineMetadata RHIGetRayTracingGeometryOfflineMetadata(const FRayTracingGeometryOfflineDataHeader& OfflineDataHeader) final override;

	virtual FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer) final override;
	virtual FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer) final override;
	virtual FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency) final override;
	virtual FRayTracingPipelineStateRHIRef RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer) final override;

	virtual FShaderBindingTableRHIRef RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer) final override;
#endif //D3D12_RHI_RAYTRACING

	virtual FShaderBundleRHIRef RHICreateShaderBundle(const FShaderBundleCreateInfo& CreateInfo) override;

	virtual void HandleGpuTimeout(FD3D12Payload* Payload, double SecondsSinceSubmission);

	bool IsQuadBufferStereoEnabled() const;
	void DisableQuadBufferStereo();

	void* LockBuffer(FRHICommandListBase& RHICmdList, FD3D12Buffer* Buffer, uint32 BufferSize, EBufferUsageFlags BufferUsage, uint32 Offset, uint32 Size, EResourceLockMode LockMode);
	void UnlockBuffer(FRHICommandListBase& RHICmdList, FD3D12Buffer* Buffer, EBufferUsageFlags BufferUsage);

	virtual bool BeginUpdateTexture3D_ComputeShader(FUpdateTexture3DData& UpdateData, FD3D12UpdateTexture3DData* UpdateDataD3D12)
	{
		// Not supported on PC
		return false;
	}
	virtual void EndUpdateTexture3D_ComputeShader(FRHIComputeCommandList& RHICmdList, FUpdateTexture3DData& UpdateData, FD3D12UpdateTexture3DData* UpdateDataD3D12)
	{
		// Not supported on PC
	}

	FUpdateTexture3DData BeginUpdateTexture3D_Internal(FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion);
	void EndUpdateTexture3D_Internal(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData);

public:

#if	PLATFORM_SUPPORTS_VIRTUAL_TEXTURES
	virtual void* CreateVirtualTexture(
		ETextureCreateFlags InFlags,
		D3D12_RESOURCE_DESC& ResourceDesc,
		const struct FD3D12TextureLayout& TextureLayout,
		FD3D12Resource** ppResource,
		FPlatformMemory::FPlatformVirtualMemoryBlock& RawTextureBlock,
		ED3D12Access InitialD3D12Access) = 0;
	virtual void DestroyVirtualTexture(ETextureCreateFlags InFlags, void* RawTextureMemory, FPlatformMemory::FPlatformVirtualMemoryBlock& RawTextureBlock, uint64 CommittedTextureSize) = 0;
#endif
	virtual bool HandleSpecialLock(FRHILockTextureResult& Result, FD3D12Texture* InTexture, const FRHILockTextureArgs& Arguments) { return false; }
	virtual bool HandleSpecialUnlock(FRHICommandListBase& RHICmdList, FD3D12Texture* InTexture, const FRHILockTextureArgs& Arguments) { return false; }

	FD3D12Adapter& GetAdapter() { return *ChosenAdapter; }
	const FD3D12Adapter& GetAdapter() const { return *ChosenAdapter; }

	bool HasAdapter() const { return ChosenAdapter.IsValid(); }

	bool IsPixEventEnabled() const { return bPixEventEnabled; }

	template<typename PerDeviceFunction>
	void ForEachDevice(ID3D12Device* inDevice, const PerDeviceFunction& pfPerDeviceFunction);

	AGSContext* GetAmdAgsContext() { return AmdAgsContext; }
	void SetAmdSupportedExtensionFlags(uint32 Flags) { AmdSupportedExtensionFlags = Flags; }
	uint32 GetAmdSupportedExtensionFlags() const { return AmdSupportedExtensionFlags; }

	INTCExtensionContext* GetIntelExtensionContext() { return IntelExtensionContext; }

protected:

	TSharedPtr<FD3D12Adapter> ChosenAdapter;
	
	FRHIMemoryStats MemoryStats;

#if D3D12RHI_SUPPORTS_WIN_PIX
	void* WinPixGpuCapturerHandle = nullptr;
#endif

	/** Can pix events be used */
	bool bPixEventEnabled = false;

	/** The feature level of the device. */
	D3D_FEATURE_LEVEL FeatureLevel;

	/**
	 * The context for the AMD AGS utility library.
	 * AGSContext does not implement AddRef/Release.
	 * Just use a bare pointer.
	 */
	AGSContext* AmdAgsContext;
	uint32 AmdSupportedExtensionFlags;

	bool bNvAPIInitialized = false;

	INTCExtensionContext* IntelExtensionContext = nullptr;

#if WITH_NVAPI_RAYTRACING_VALIDATION
	void* RayTracingValidationCallbackHandle = nullptr;
#endif

	// See r.PSOPrecache.D3D12.DriverCacheAware. The CVar is cached
	// to prevent hotfixes or config dynamic layers from altering the
	// value at runtime, which is not supported.
	bool bDriverCacheAwarePSOPrecaching = false;

	/** A buffer in system memory containing all zeroes of the specified size. */
	void* ZeroBuffer;
	uint32 ZeroBufferSize;

#if PLATFORM_WINDOWS
	TRefCountPtr<IDXGIFactory2> DXGIFactoryForDisplayList;
	TRefCountPtr<IDXGIFactory2> DXGIFactoryForResolutions;
#endif

public:

	static void SetFormatAliasedTexturesMustBeCreatedUsingCommonLayout(bool bValue);

	virtual FD3D12ResourceDesc GetResourceDesc(const FRHITextureDesc& CreateInfo) const;

	struct FCreateTextureInternalResult
	{
		FD3D12Texture* Texture{};
		ED3D12Access CreateD3D12Access{};
		ED3D12Access DesiredD3D12Access{};
		bool bNeedsDiscard{};
	};

	// Per-platform D3D12Texture creation.
	virtual FCreateTextureInternalResult CreateTextureInternal(const FRHITextureCreateDesc& CreateDesc, ID3D12ResourceAllocator* ResourceAllocator = nullptr);

	// Create D3D12 texture with a resource allocator. Does not currently support any post-creation state or uploading of data.
	FD3D12Texture* CreateD3D12Texture(const FRHITextureCreateDesc& CreateDesc, ID3D12ResourceAllocator* ResourceAllocator)
	{
		check(CreateDesc.BulkData == nullptr);

		FCreateTextureInternalResult CreateResult = CreateTextureInternal(CreateDesc, ResourceAllocator);
		checkSlow(CreateResult.CreateD3D12Access == CreateResult.DesiredD3D12Access);
		return CreateResult.Texture;
	}

	virtual FD3D12Texture* CreateNewD3D12Texture(const FRHITextureCreateDesc& CreateDesc, class FD3D12Device* Device);

	struct FCreateBufferInternalResult
	{
		FD3D12Buffer* Buffer{};
		ED3D12Access DesiredD3D12Access{};
	};

	FCreateBufferInternalResult CreateBufferInternal(const FRHIBufferCreateDesc& CreateDesc, bool bHasInitialData, ID3D12ResourceAllocator* ResourceAllocator);

	FD3D12Buffer* CreateD3D12Buffer(FRHICommandListBase* RHICmdList, const FRHIBufferCreateDesc& CreateDesc, ID3D12ResourceAllocator* ResourceAllocator = nullptr)
	{
		FCreateBufferInternalResult CreateResult = CreateBufferInternal(CreateDesc, false, ResourceAllocator);
		return CreateResult.Buffer;
	}

	FD3D12Buffer* CreateD3D12Buffer(const FRHIBufferCreateDesc& CreateDesc, ID3D12ResourceAllocator* ResourceAllocator)
	{
		return CreateD3D12Buffer(nullptr, CreateDesc, ResourceAllocator);
	}

	bool SetupDisplayHDRMetaData();

protected:

	FD3D12Texture* CreateTextureFromResource(bool bTextureArray, bool bCubeTexture, EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource);
#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
	FD3D12Texture* CreateAliasedD3D12Texture2D(FD3D12Texture* SourceTexture);
#endif

	/**
	 * Gets the best supported MSAA settings from the provided MSAA count to check against.
	 *
	 * @param PlatformFormat		The format of the texture being created
	 * @param MSAACount				The MSAA count to check against.
	 * @param OutBestMSAACount		The best MSAA count that is suppored.  Could be smaller than MSAACount if it is not supported
	 * @param OutMSAAQualityLevels	The number MSAA quality levels for the best msaa count supported
	 */
	void GetBestSupportedMSAASetting(DXGI_FORMAT PlatformFormat, uint32 MSAACount, uint32& OutBestMSAACount, uint32& OutMSAAQualityLevels);

	/**
	* Returns a pointer to a texture resource that can be used for CPU reads.
	* Note: the returned resource could be the original texture or a new temporary texture.
	* @param TextureRHI - Source texture to create a staging texture from.
	* @param InRect - rectangle to 'stage'.
	* @param StagingRectOUT - parameter is filled with the rectangle to read from the returned texture.
	* @return The CPU readable Texture object.
	*/
	TRefCountPtr<FD3D12Resource> GetStagingTexture(FRHITexture* TextureRHI, FIntRect InRect, FIntRect& OutRect, FReadSurfaceDataFlags InFlags, D3D12_PLACED_SUBRESOURCE_FOOTPRINT &readBackHeapDesc, uint32 GPUIndex);

	void ReadSurfaceDataNoMSAARaw(FRHITexture* TextureRHI, FIntRect Rect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags);

	void ReadSurfaceDataMSAARaw(FRHITexture* TextureRHI, FIntRect Rect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags);

	// This should only be called by Dynamic RHI member functions
	FD3D12Device* GetRHIDevice(uint32 GPUIndex) const;

	void SetupD3D12Debug();

	HANDLE FlipEvent;

	FDisplayInformationArray DisplayList;

	void ProcessDeferredDeletionQueue_Platform();

#if PLATFORM_WINDOWS
	void InitializeDeviceAgnosticVendorAPI();
#endif
};

ENUM_CLASS_FLAGS(FD3D12DynamicRHI::EQueueStatus);

/** Implements the D3D12RHI module as a dynamic RHI providing module. */
class FD3D12DynamicRHIModule : public IDynamicRHIModule
{
public:

	FD3D12DynamicRHIModule()
	{
	}

	~FD3D12DynamicRHIModule() = default;

	// IModuleInterface
	virtual bool SupportsDynamicReloading() override { return false; }
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IDynamicRHIModule
	virtual bool IsSupported() override { return IsSupported(ERHIFeatureLevel::SM5); }
	virtual bool IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel) override;
	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override;

private:

#if D3D12RHI_SUPPORTS_WIN_PIX
	void* WindowsPixDllHandle = nullptr;
	void* WinPixGpuCapturerHandle = nullptr;
#endif

	TSharedPtr<FD3D12Adapter> ChosenAdapter;

	// set MaxSupportedFeatureLevel and ChosenAdapter
	void FindAdapter();
};

// Helper to push/pop a desired state on a resource. Handles both tracked and untracked resources.
class FScopedResourceBarrier
{
private:
	FD3D12ContextCommon&  Context;
	const FD3D12Resource* pResource;
	ED3D12Access BeforeD3D12Access;
	const ED3D12Access DesiredD3D12Access;
	const uint32 Subresource;
	bool bRestoreState;

public:
	FScopedResourceBarrier(
		FD3D12ContextCommon& InContext,
		const FD3D12Resource* InResource,
		ED3D12Access InBeforeD3D12Access,
		ED3D12Access InDesiredD3D12Access,
		uint32 InSubresource)
	: Context(InContext)
	, pResource(InResource)
	, BeforeD3D12Access(InBeforeD3D12Access)
	, DesiredD3D12Access(InDesiredD3D12Access)
	, Subresource (InSubresource)
	, bRestoreState(false)
	{
		check(IsValidAccess(DesiredD3D12Access));
		check(DesiredD3D12Access != ED3D12Access::Unknown);

		const bool bBeforED3D12AccessIsKnown = BeforeD3D12Access != ED3D12Access::Unknown;
		if (!pResource->RequiresResourceStateTracking() || bBeforED3D12AccessIsKnown)
		{
			BeforeD3D12Access = bBeforED3D12AccessIsKnown ? BeforeD3D12Access : pResource->GetDefaultAccess();
			check(IsValidAccess(BeforeD3D12Access));
			check(BeforeD3D12Access != ED3D12Access::Unknown);

			Context.AddBarrier(pResource, BeforeD3D12Access, DesiredD3D12Access, Subresource);
			bRestoreState = true;
		}
	}

	FScopedResourceBarrier(
		FD3D12ContextCommon& InContext,
		const FD3D12Resource* InResource,
		ED3D12Access InDesiredD3D12Access,
		uint32 InSubresource)
	: FScopedResourceBarrier(
		InContext,
		InResource,
		ED3D12Access::Unknown,
		InDesiredD3D12Access,
		InSubresource)
	{}

	~FScopedResourceBarrier()
	{
		if (bRestoreState)
		{
			Context.AddBarrier(pResource, DesiredD3D12Access, BeforeD3D12Access, Subresource);
		}
	}
};

// Returns the given format as a string. Unsupported formats are treated as DXGI_FORMAT_UNKNOWN.
const TCHAR* LexToString(DXGI_FORMAT Format);
