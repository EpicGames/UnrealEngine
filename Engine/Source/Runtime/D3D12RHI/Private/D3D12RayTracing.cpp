// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RayTracing.h"

#if D3D12_RHI_RAYTRACING

#include "D3D12Resources.h"
#include "D3D12Util.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Experimental/Containers/SherwoodHashTable.h"
#include "BuiltInRayTracingShaders.h"
#include "RayTracingValidationShaders.h"
#include "Hash/xxhash.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/ScopeLock.h"
#include "Async/ParallelFor.h"
#include "Misc/BufferedOutputDevice.h"
#include "String/LexFromString.h"
#include "GlobalRenderResources.h"
#include "D3D12RayTracingDebug.h"
#include "D3D12ExplicitDescriptorCache.h"
#include "D3D12ResourceCollection.h"
#include "RHIShaderBindingLayout.h"
#include "RHIUniformBufferUtilities.h"
#include "RHIResourceUtils.h"
#include "D3D12TextureReference.h"

#include "NaniteRayTracingDefinitions.h"

#if WITH_NVAPI
DEFINE_ENUM_FLAG_OPERATORS(NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAGS);
#endif

extern int32 GD3D12ExplicitViewDescriptorHeapSize;
extern int32 GD3D12ExplicitViewDescriptorHeapOverflowReported;

static int32 GRayTracingDebugForceBuildMode = 0;
static FAutoConsoleVariableRef CVarRayTracingDebugForceFastTrace(
	TEXT("r.D3D12.RayTracing.DebugForceBuildMode"),
	GRayTracingDebugForceBuildMode,
	TEXT("Forces specific acceleration structure build mode (not runtime-tweakable).\n")
	TEXT("0: Use build mode requested by high-level code (Default)\n")
	TEXT("1: Force fast build mode\n")
	TEXT("2: Force fast trace mode\n"),
	ECVF_ReadOnly
);

static int32 GRayTracingCacheShaderRecords = 1;
static FAutoConsoleVariableRef CVarRayTracingShaderRecordCache(
	TEXT("r.D3D12.RayTracing.CacheShaderRecords"),
	GRayTracingCacheShaderRecords,
	TEXT("Automatically cache and re-use SBT hit group records. This significantly improves CPU performance in large scenes with many identical mesh instances. (default = 1)\n")
	TEXT("This mode assumes that contents of uniform buffers does not change during ray tracing resource binding.")
);

static int32 GD3D12RayTracingAllowCompaction = 1;
static FAutoConsoleVariableRef CVarD3D12RayTracingAllowCompaction(
	TEXT("r.D3D12.RayTracing.AllowCompaction"),
	GD3D12RayTracingAllowCompaction,
	TEXT("Whether to automatically perform compaction for static acceleration structures to save GPU memory. (default = 1)\n"),
	ECVF_ReadOnly
);

static int32 GD3D12RayTracingMaxBatchedCompaction = 64;
static FAutoConsoleVariableRef CVarD3D12RayTracingMaxBatchedCompaction(
	TEXT("r.D3D12.RayTracing.MaxBatchedCompaction"),
	GD3D12RayTracingMaxBatchedCompaction,
	TEXT("Maximum of amount of compaction requests and rebuilds per frame. (default = 64)\n"),
	ECVF_ReadOnly
);

static int32 GD3D12RayTracingCompactionMinPrimitiveCount = 128;
static FAutoConsoleVariableRef CVarD3D12RayTracingCompactionMinPrimitiveCount(
	TEXT("r.D3D12.RayTracing.Compaction.MinPrimitiveCount"),
	GD3D12RayTracingCompactionMinPrimitiveCount,
	TEXT("Sets the minimum primitive count threshold below which geometry skips the compaction. (default = 128)\n")
);

static int32 GRayTracingSpecializeStateObjects = 0;
static FAutoConsoleVariableRef CVarRayTracingSpecializeStateObjects(
	TEXT("r.D3D12.RayTracing.SpecializeStateObjects"),
	GRayTracingSpecializeStateObjects,
	TEXT("Whether to create specialized unique ray tracing pipeline state objects for each ray generation shader. (default = 0)\n")
	TEXT("This option can produce more more efficient PSOs for the GPU at the cost of longer creation times and more memory. Requires DXR 1.1.\n"),
	ECVF_ReadOnly
);

static int32 GRayTracingAllowSpecializedStateObjects = 1;
static FAutoConsoleVariableRef CVarRayTracingAllowSpecializedStateObjects(
	TEXT("r.D3D12.RayTracing.AllowSpecializedStateObjects"),
	GRayTracingAllowSpecializedStateObjects,
	TEXT("Whether to use specialized RTPSOs if they have been created. ")
	TEXT("This is intended for performance testingand has no effect if r.D3D12.RayTracing.SpecializeStateObjects is 0. (default = 1)\n")
);

static int32 GD3D12RayTracingGPUValidation = 0;
static FAutoConsoleVariableRef CVarD3D12RayTracingGPUValidation(
	TEXT("r.D3D12.RayTracing.GPUValidation"),
	GD3D12RayTracingGPUValidation,
	TEXT("Whether to perform validation of ray tracing geometry and other structures on the GPU. Requires Shader Model 6. (default = 0)")
);

// This is required to avoid redundant code static analysis warnings
// If the static_assert fires the assumptions it is predicated on have been
// violated and the code should be revisited
#if WITH_MGPU
#define FOREACH_GPU(Condition, Function) for (uint32 GPUIndex = 0; Condition; ++GPUIndex) Function
#else
static_assert(MAX_NUM_GPUS == 1 && GNumExplicitGPUsForRendering == 1);
#define FOREACH_GPU(Condition, Function) { constexpr uint32 GPUIndex = 0; Function }
#endif

// Ray tracing stat counters

DECLARE_STATS_GROUP(TEXT("D3D12RHI: Ray Tracing"), STATGROUP_D3D12RayTracing, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Created pipelines (total)"), STAT_D3D12RayTracingCreatedPipelines, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Compiled shaders (total)"), STAT_D3D12RayTracingCompiledShaders, STATGROUP_D3D12RayTracing);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Allocated bottom level acceleration structures"), STAT_D3D12RayTracingAllocatedBLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Allocated top level acceleration structures"), STAT_D3D12RayTracingAllocatedTLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Triangles in all BL acceleration structures"), STAT_D3D12RayTracingTrianglesBLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Procedural primitives in all BL acceleration structures"), STAT_D3D12RayTracingProceduralPrimitivesBLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("LinearSweptSpheres in all BL acceleration structures"), STAT_D3D12RayTracingLinearSweptSpheresBLAS, STATGROUP_D3D12RayTracing);

DECLARE_DWORD_COUNTER_STAT(TEXT("Built BL AS (per frame)"), STAT_D3D12RayTracingBuiltBLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Updated BL AS (per frame)"), STAT_D3D12RayTracingUpdatedBLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Built TL AS (per frame)"), STAT_D3D12RayTracingBuiltTLAS, STATGROUP_D3D12RayTracing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Updated TL AS (per frame)"), STAT_D3D12RayTracingUpdatedTLAS, STATGROUP_D3D12RayTracing);

DECLARE_DWORD_COUNTER_STAT(TEXT("UniformBuffer Record Updates (per frame)"), STAT_D3D12RayTracingUBRecordUpdate, STATGROUP_D3D12RayTracing);

DECLARE_MEMORY_STAT(TEXT("Total BL AS Memory"), STAT_D3D12RayTracingBLASMemory, STATGROUP_D3D12RayTracing);
DECLARE_MEMORY_STAT(TEXT("Static BL AS Memory"), STAT_D3D12RayTracingStaticBLASMemory, STATGROUP_D3D12RayTracing);
DECLARE_MEMORY_STAT(TEXT("Dynamic BL AS Memory"), STAT_D3D12RayTracingDynamicBLASMemory, STATGROUP_D3D12RayTracing);
DECLARE_MEMORY_STAT(TEXT("TL AS Memory"), STAT_D3D12RayTracingTLASMemory, STATGROUP_D3D12RayTracing);
DECLARE_MEMORY_STAT(TEXT("Total Used Video Memory"), STAT_D3D12RayTracingUsedVideoMemory, STATGROUP_D3D12RayTracing);

DECLARE_CYCLE_STAT(TEXT("RTPSO Compile Shader"), STAT_RTPSO_CompileShader, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("RTPSO Create Pipeline"), STAT_RTPSO_CreatePipeline, STATGROUP_D3D12RayTracing);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Allocated shader binding tables"), STAT_D3D12RayTracingAllocatedSBT, STATGROUP_D3D12RayTracing);

DECLARE_CYCLE_STAT(TEXT("SetBindingsOnShaderBindingTable"), STAT_D3D12SetBindingsOnShaderBindingTable, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("CreateShaderTable"), STAT_D3D12CreateShaderTable, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("BuildTopLevel"), STAT_D3D12BuildTLAS, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("BuildBottomLevel"), STAT_D3D12BuildBLAS, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("DispatchRays"), STAT_D3D12DispatchRays, STATGROUP_D3D12RayTracing);
DECLARE_CYCLE_STAT(TEXT("ClusterOperation"), STAT_D3D12ClusterOperation, STATGROUP_D3D12RayTracing);

static ERayTracingAccelerationStructureFlags GetRayTracingAccelerationStructureBuildFlags(const FRayTracingGeometryInitializer& Initializer);

#if WITH_NVAPI
DEFINE_ENUM_FLAG_OPERATORS(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS_EX);
#endif

#if UE_BUILD_SHIPPING
inline void RegisterD3D12RayTracingGeometry(FD3D12RayTracingGeometry* Geometry) {};
inline void UnregisterD3D12RayTracingGeometry(FD3D12RayTracingGeometry* Geometry) {};
#else
struct FD3D12RayTracingGeometryTracker
{
	TSet<FD3D12RayTracingGeometry*> Geometries;
	uint64 TotalBLASSize = 0;
	uint64 MaxTotalBLASSize = 0;
	FCriticalSection CS;

	uint64 GetGeometrySize(FD3D12RayTracingGeometry& Geometry)
	{
		if (Geometry.AccelerationStructureCompactedSize != 0)
		{
			return Geometry.AccelerationStructureCompactedSize;
		}
		else
		{
			return Geometry.SizeInfo.ResultSize;
		}
	}

	void Add(FD3D12RayTracingGeometry* Geometry)
	{
		uint64 BLASSize = GetGeometrySize(*Geometry);

		FScopeLock Lock(&CS);
		Geometries.Add(Geometry);
		TotalBLASSize += BLASSize;

		MaxTotalBLASSize = FMath::Max(MaxTotalBLASSize, TotalBLASSize);
	}

	void Remove(FD3D12RayTracingGeometry* Geometry)
	{
		uint64 BLASSize = GetGeometrySize(*Geometry);

		FScopeLock Lock(&CS);
		Geometries.Remove(Geometry);

		TotalBLASSize -= BLASSize;
	}
};

static FD3D12RayTracingGeometryTracker& GetD3D12RayTracingGeometryTracker()
{
	static FD3D12RayTracingGeometryTracker Instance;
	return Instance;
}

enum class EDumpRayTracingGeometryMode
{
	Top,
	All,
};

static void DumpRayTracingGeometries(EDumpRayTracingGeometryMode Mode, int32 NumEntriesToShow, const FString& NameFilter, bool bCSV, FBufferedOutputDevice& BufferedOutput)
{
	FD3D12RayTracingGeometryTracker& Tracker = GetD3D12RayTracingGeometryTracker();
	FScopeLock Lock(&Tracker.CS);
	
	auto GetGeometrySize = [](FD3D12RayTracingGeometry& Geometry)
	{
		if (Geometry.AccelerationStructureCompactedSize != 0)
		{
			return Geometry.AccelerationStructureCompactedSize;
		}
		else
		{
			return Geometry.SizeInfo.ResultSize;
		}
	};

	TArray<FD3D12RayTracingGeometry*> Geometries = Tracker.Geometries.Array();
	Geometries.Sort([GetGeometrySize](FD3D12RayTracingGeometry& A, FD3D12RayTracingGeometry& B)
	{
		return GetGeometrySize(A) > GetGeometrySize(B);
	});

	FName CategoryName("D3D12RayTracing");
	uint64 TotalSizeBytes = 0;
	uint64 TopSizeBytes = 0;
	BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Tracked FD3D12RayTracingGeometry objects"));

	if (NumEntriesToShow < 0 || NumEntriesToShow > Geometries.Num())
	{
		NumEntriesToShow = Geometries.Num();
	}

	if (NumEntriesToShow != Geometries.Num())
	{
		BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Showing %d out of %d"), NumEntriesToShow, Geometries.Num());
	}

	auto ShouldShow = [&NameFilter](FD3D12RayTracingGeometry* Entry)
	{
		if (NameFilter.IsEmpty())
		{
			return true;
		}

		FString DebugName = Entry->DebugName.ToString();
		if (DebugName.Find(NameFilter, ESearchCase::IgnoreCase) != INDEX_NONE)
		{
			return true;
		}
		else
		{
			return false;
		}
	};

	FArchive* CSVFile{ nullptr };
	if (bCSV)
	{
		const FString Filename = FString::Printf(TEXT("%sd3d12DumpRayTracingGeometries-%s.csv"), *FPaths::ProfilingDir(), *FDateTime::Now().ToString());
		CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);

		const TCHAR* Header = TEXT("Name,Size (MBs),Prims,Segments,Compaction,Update,MarkedForDelete\n");
		CSVFile->Serialize(TCHAR_TO_ANSI(Header), FPlatformString::Strlen(Header));
	}

	int32 ShownEntries = 0;
	for (int32 i=0; i< Geometries.Num(); ++i)
	{
		FD3D12RayTracingGeometry* Geometry = Geometries[i];
		uint64 SizeBytes = GetGeometrySize(*Geometry);

		ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(Geometry->Initializer);

		if (ShownEntries < NumEntriesToShow && ShouldShow(Geometry))
		{
			if (bCSV)
			{
				const FString Row = FString::Printf(TEXT("%s,%.3f,%d,%d,%d,%d,%d\n"),
					!Geometry->DebugName.IsNone() ? *Geometry->DebugName.ToString() : TEXT("*UNKNOWN*"),
					SizeBytes / double(1 << 20),
					Geometry->Initializer.TotalPrimitiveCount,
					Geometry->Initializer.Segments.Num(),
					(int32)EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction),
					(int32)EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate),
					!Geometry->IsValid());
				CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
			}
			else
			{
				BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Name: %s - Size: %.3f MB - Prims: %d - Segments: %d -  Compaction: %d - Update: %d"),
					!Geometry->DebugName.IsNone() ? *Geometry->DebugName.ToString() : TEXT("*UNKNOWN*"),
					SizeBytes / double(1 << 20),
					Geometry->Initializer.TotalPrimitiveCount,
					Geometry->Initializer.Segments.Num(),
					(int32)EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction),
					(int32)EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate));
			}
			TopSizeBytes += SizeBytes;
			++ShownEntries;
		}

		TotalSizeBytes += SizeBytes;
	}

	if (bCSV)
	{
		delete CSVFile;
		CSVFile = nullptr;
	}
	else
	{
		double TotalSizeF = double(TotalSizeBytes) / double(1 << 20);
		double TopSizeF = double(TopSizeBytes) / double(1 << 20);

		if (ShownEntries != Geometries.Num() && ShownEntries)
		{
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log,
				TEXT("Use command `D3D12.DumpRayTracingGeometries all/N [name]` to dump all or N objects. ")
				TEXT("Optionally add 'name' to filter entries, such as 'skm_'."));
			BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Shown %d entries. Size: %.3f MB (%.2f%% of total)"),
				ShownEntries, TopSizeF, 100.0 * TopSizeF / TotalSizeF);
		}

		BufferedOutput.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Total size: %.3f MB"), TotalSizeF);
	}
}

static FAutoConsoleCommandWithWorldArgsAndOutputDevice GD3D12DumpRayTracingGeometriesCmd(
	TEXT("D3D12.DumpRayTracingGeometries"),
	TEXT("Dump memory allocations for ray tracing resources."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld*, FOutputDevice& OutputDevice)
{
	// Default: show top 50 largest objects.
	EDumpRayTracingGeometryMode Mode = EDumpRayTracingGeometryMode::Top;
	int32 NumEntriesToShow = 50;
	bool bCSV = false;

	FString NameFilter;

	if (Args.Num())
	{
		if (Args[0] == TEXT("all"))
		{
			Mode = EDumpRayTracingGeometryMode::All;
			NumEntriesToShow = -1;
		}
		else if (FCString::IsNumeric(*Args[0]))
		{
			Mode = EDumpRayTracingGeometryMode::Top;
			LexFromString(NumEntriesToShow, *Args[0]);
		}

		if (Args.Num() > 1)
		{
			NameFilter = Args[1];
		}
	}

	FBufferedOutputDevice BufferedOutput;
	DumpRayTracingGeometries(Mode, NumEntriesToShow, NameFilter, bCSV, BufferedOutput);
	BufferedOutput.RedirectTo(OutputDevice);
}));


static FAutoConsoleCommandWithWorldArgsAndOutputDevice GD3D12DumpRayTracingGeometriesToCSVCmd(
	TEXT("D3D12.DumpRayTracingGeometriesToCSV"),
	TEXT("Dump all memory allocations for ray tracing resources to a CSV file on disc."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld*, FOutputDevice& OutputDevice)
{
	// CSV dumps all entries
	EDumpRayTracingGeometryMode Mode = EDumpRayTracingGeometryMode::All;
	int32 NumEntriesToShow = -1;
	bool bCSV = true;
	FString NameFilter;

	FBufferedOutputDevice BufferedOutput;
	DumpRayTracingGeometries(Mode, NumEntriesToShow, NameFilter, bCSV, BufferedOutput);
	BufferedOutput.RedirectTo(OutputDevice);
}));

inline void RegisterD3D12RayTracingGeometry(FD3D12RayTracingGeometry* Geometry)
{
	GetD3D12RayTracingGeometryTracker().Add(Geometry);
}
inline void UnregisterD3D12RayTracingGeometry(FD3D12RayTracingGeometry* Geometry)
{
	GetD3D12RayTracingGeometryTracker().Remove(Geometry);
}
#endif // UE_BUILD_SHIPPING


const FD3D12ShaderIdentifier FD3D12ShaderIdentifier::Null = { 0, 0, 0, 0 };

static_assert(sizeof(FD3D12ShaderIdentifier) == D3D12_SHADER_IDENTIFIER_SIZE_IN_BYTES, "Unexpected shader identifier size");

static bool ShouldRunRayTracingGPUValidation()
{
	// Wave ops are required to run ray tracing validation shaders
	const bool bSupportsWaveOps = GRHISupportsWaveOperations && RHISupportsWaveOperations(GMaxRHIShaderPlatform);
	return GD3D12RayTracingGPUValidation && bSupportsWaveOps;
}

static D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS TranslateRayTracingAccelerationStructureFlags(ERayTracingAccelerationStructureFlags Flags)
{
	uint32 Result = {};

	auto HandleFlag = [&Flags, &Result](ERayTracingAccelerationStructureFlags Engine, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS Native)
	{
		if (EnumHasAllFlags(Flags, Engine))
		{
			Result |= (uint32)Native;
			EnumRemoveFlags(Flags, Engine);
		}
	};

	HandleFlag(ERayTracingAccelerationStructureFlags::AllowUpdate, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_ALLOW_UPDATE);
	HandleFlag(ERayTracingAccelerationStructureFlags::AllowCompaction, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_ALLOW_COMPACTION);
	HandleFlag(ERayTracingAccelerationStructureFlags::FastTrace, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_PREFER_FAST_TRACE);
	HandleFlag(ERayTracingAccelerationStructureFlags::FastBuild, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_PREFER_FAST_BUILD);
	HandleFlag(ERayTracingAccelerationStructureFlags::MinimizeMemory, D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_MINIMIZE_MEMORY);

	checkf(!EnumHasAnyFlags(Flags, Flags), TEXT("Some ERayTracingAccelerationStructureFlags entries were not handled"));

	return D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS(Result);
}

static DXGI_FORMAT TranslateRayTracingVertexElementType(EVertexElementType Type)
{
	switch (Type)
	{
		case VET_Float4:
			// While the DXGI_FORMAT_R32G32B32A32_FLOAT format is not supported by DXR, since we manually load vertex 
			// data when we are building the BLAS, we can just rely on the vertex stride to offset the read index, 
			// and read only the 3 vertex components, and so use the DXGI_FORMAT_R32G32B32_FLOAT vertex format
		case VET_Float3:
			return DXGI_FORMAT_R32G32B32_FLOAT;
		case VET_Float2:
			return DXGI_FORMAT_R32G32_FLOAT;
		case VET_Float1:
			return DXGI_FORMAT_R32_FLOAT;
		case VET_Half2:
			return DXGI_FORMAT_R16G16_FLOAT;
	}
	return DXGI_FORMAT_UNKNOWN;
}

static D3D12_RAYTRACING_GEOMETRY_FLAGS TranslateSegmentToGeometryFlags(const FRayTracingGeometrySegment& Segment)
{
	D3D12_RAYTRACING_GEOMETRY_FLAGS Flags = D3D12_RAYTRACING_GEOMETRY_FLAG_NONE;

	if (Segment.bForceOpaque)
	{
		// Deny anyhit shader invocations when this segment is hit
		Flags |= D3D12_RAYTRACING_GEOMETRY_FLAG_OPAQUE;
	}

	if (!Segment.bAllowDuplicateAnyHitShaderInvocation)
	{
		// Allow only a single any-hit shader invocation per primitive
		Flags |= D3D12_RAYTRACING_GEOMETRY_FLAG_NO_DUPLICATE_ANYHIT_INVOCATION;
	}
	return Flags;
}

struct FDXILLibrary
{
	// No copy assignment or move because FDXILLibrary points to internal struct memory
	UE_NONCOPYABLE(FDXILLibrary)

	FDXILLibrary() = default;

	void InitFromDXIL(const void* Bytecode, uint32 BytecodeLength, const LPCWSTR* InEntryNames, const LPCWSTR* InExportNames, uint32 NumEntryNames)
	{
		check(NumEntryNames != 0);
		check(InEntryNames);
		check(InExportNames);

		EntryNames.SetNum(NumEntryNames);
		ExportNames.SetNum(NumEntryNames);
		ExportDesc.SetNum(NumEntryNames);

		for (uint32 EntryIndex = 0; EntryIndex < NumEntryNames; ++EntryIndex)
		{
			EntryNames[EntryIndex] = InEntryNames[EntryIndex];
			ExportNames[EntryIndex] = InExportNames[EntryIndex];

			ExportDesc[EntryIndex].ExportToRename = *(EntryNames[EntryIndex]);
			ExportDesc[EntryIndex].Flags = D3D12_EXPORT_FLAG_NONE;
			ExportDesc[EntryIndex].Name = *(ExportNames[EntryIndex]);
		}

		Desc.DXILLibrary.pShaderBytecode = Bytecode;
		Desc.DXILLibrary.BytecodeLength = BytecodeLength;
		Desc.NumExports = ExportDesc.Num();
		Desc.pExports = ExportDesc.GetData();
	}

	void InitFromDXIL(const D3D12_SHADER_BYTECODE& ShaderBytecode, LPCWSTR* InEntryNames, LPCWSTR* InExportNames, uint32 NumEntryNames)
	{
		InitFromDXIL(ShaderBytecode.pShaderBytecode, ShaderBytecode.BytecodeLength, InEntryNames, InExportNames, NumEntryNames);
	}

	D3D12_STATE_SUBOBJECT GetSubobject() const
	{
		D3D12_STATE_SUBOBJECT Subobject = {};
		Subobject.Type = D3D12_STATE_SUBOBJECT_TYPE_DXIL_LIBRARY;
		Subobject.pDesc = &Desc;
		return Subobject;
	}

	// NOTE: typical DXIL library may contain up to 3 entry points (i.e. hit groups with closest hit, any hit and intersection shaders)
	// Typical case is 1 (RGS, MS or CHS only) or 2 (CHS + AHS for shaders with alpha masking)
	static constexpr uint32 ExpectedEntryPoints = 3;
	TArray<D3D12_EXPORT_DESC, TInlineAllocator<ExpectedEntryPoints>> ExportDesc;
	TArray<FString, TInlineAllocator<ExpectedEntryPoints>> EntryNames;
	TArray<FString, TInlineAllocator<ExpectedEntryPoints>> ExportNames;

	D3D12_DXIL_LIBRARY_DESC Desc = {};
};

static TRefCountPtr<ID3D12StateObject> CreateRayTracingStateObject(
	ID3D12Device5* RayTracingDevice,
	const TArrayView<const FDXILLibrary*>& ShaderLibraries,
	const TArrayView<LPCWSTR>& Exports,
	uint32 MaxAttributeSizeInBytes,
	uint32 MaxPayloadSizeInBytes,
	const TArrayView<const D3D12_HIT_GROUP_DESC>& HitGroups,
	const ID3D12RootSignature* GlobalRootSignature,
	const TArrayView<ID3D12RootSignature*>& LocalRootSignatures,
	const TArrayView<uint32>& LocalRootSignatureAssociations, // indices into LocalRootSignatures, one per export (may be empty, which assumes single root signature used for everything)
	const TArrayView<D3D12_EXISTING_COLLECTION_DESC>& ExistingCollections,
	D3D12_STATE_OBJECT_TYPE StateObjectType // Full RTPSO or a Collection
)
{
	checkf((LocalRootSignatureAssociations.Num() == 0 && LocalRootSignatures.Num() == 1)
		|| (LocalRootSignatureAssociations.Num() == Exports.Num()),
		TEXT("There must be exactly one local root signature association per export."));

	TRefCountPtr<ID3D12StateObject> Result;

	// There are several pipeline sub-objects that are always required:
	// 1) D3D12_RAYTRACING_SHADER_CONFIG
	// 2) D3D12_SUBOBJECT_TO_EXPORTS_ASSOCIATION
	// 3) D3D12_RAYTRACING_PIPELINE_CONFIG
	// 4) D3D12_STATE_OBJECT_CONFIG
	// 5) Global root signature
	static constexpr uint32 NumRequiredSubobjects = 5;

	TArray<D3D12_STATE_SUBOBJECT> Subobjects;
	Subobjects.SetNumUninitialized(NumRequiredSubobjects
		+ ShaderLibraries.Num()
		+ HitGroups.Num()
		+ LocalRootSignatures.Num()
		+ Exports.Num()
		+ ExistingCollections.Num()
	);

	TArray<D3D12_SUBOBJECT_TO_EXPORTS_ASSOCIATION> ExportAssociations;
	ExportAssociations.SetNumUninitialized(Exports.Num());

	uint32 Index = 0;

	const uint32 NumExports = Exports.Num();

	// Shader libraries

	for (const FDXILLibrary* Library : ShaderLibraries)
	{
		Subobjects[Index++] = Library->GetSubobject();
	}

	// Shader config

	D3D12_RAYTRACING_SHADER_CONFIG ShaderConfig = {};
	ShaderConfig.MaxAttributeSizeInBytes = MaxAttributeSizeInBytes;
	check(ShaderConfig.MaxAttributeSizeInBytes <= RAY_TRACING_MAX_ALLOWED_ATTRIBUTE_SIZE);
	ShaderConfig.MaxPayloadSizeInBytes = MaxPayloadSizeInBytes;

	const uint32 ShaderConfigIndex = Index;
	Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_RAYTRACING_SHADER_CONFIG, &ShaderConfig};

	// Shader config association

	D3D12_SUBOBJECT_TO_EXPORTS_ASSOCIATION ShaderConfigAssociation = {};
	ShaderConfigAssociation.NumExports = Exports.Num();
	ShaderConfigAssociation.pExports = Exports.GetData();
	ShaderConfigAssociation.pSubobjectToAssociate = &Subobjects[ShaderConfigIndex];
	Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_SUBOBJECT_TO_EXPORTS_ASSOCIATION, &ShaderConfigAssociation };

	// Hit groups

	for (const D3D12_HIT_GROUP_DESC& HitGroupDesc : HitGroups)
	{
		Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_HIT_GROUP, &HitGroupDesc };
	}

	// Pipeline config

	D3D12_RAYTRACING_PIPELINE_CONFIG PipelineConfig = {};
	PipelineConfig.MaxTraceRecursionDepth = RAY_TRACING_MAX_ALLOWED_RECURSION_DEPTH;
	const uint32 PipelineConfigIndex = Index;
	Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_RAYTRACING_PIPELINE_CONFIG, &PipelineConfig };

	// State object config

	D3D12_STATE_OBJECT_CONFIG StateObjectConfig = {};
	if (GRHISupportsRayTracingPSOAdditions)
	{
		StateObjectConfig.Flags = D3D12_STATE_OBJECT_FLAG_ALLOW_STATE_OBJECT_ADDITIONS;
	}
	Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_STATE_OBJECT_CONFIG, &StateObjectConfig };

	// Global root signature

	Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_GLOBAL_ROOT_SIGNATURE, &GlobalRootSignature };

	// Local root signatures

	const uint32 LocalRootSignatureBaseIndex = Index;
	for (int32 SignatureIndex = 0; SignatureIndex < LocalRootSignatures.Num(); ++SignatureIndex)
	{
		checkf(LocalRootSignatures[SignatureIndex], TEXT("All local root signatures must be valid"));
		Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_LOCAL_ROOT_SIGNATURE, &LocalRootSignatures[SignatureIndex] };
	}

	// Local root signature associations

	for (int32 ExportIndex = 0; ExportIndex < Exports.Num(); ++ExportIndex)
	{
		// If custom LocalRootSignatureAssociations data is not provided, then assume same default local RS association.
		const int32 LocalRootSignatureIndex = LocalRootSignatureAssociations.Num() != 0
			? LocalRootSignatureAssociations[ExportIndex]
			: 0;

		D3D12_SUBOBJECT_TO_EXPORTS_ASSOCIATION& Association = ExportAssociations[ExportIndex];
		Association = D3D12_SUBOBJECT_TO_EXPORTS_ASSOCIATION{};
		Association.NumExports = 1;
		Association.pExports = &Exports[ExportIndex];

		check(LocalRootSignatureIndex < LocalRootSignatures.Num());
		Association.pSubobjectToAssociate = &Subobjects[LocalRootSignatureBaseIndex + LocalRootSignatureIndex];

		Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_SUBOBJECT_TO_EXPORTS_ASSOCIATION, &ExportAssociations[ExportIndex] };
	}

	// Existing collection objects

	for (int32 CollectionIndex = 0; CollectionIndex < ExistingCollections.Num(); ++CollectionIndex)
	{
		Subobjects[Index++] = D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_EXISTING_COLLECTION, &ExistingCollections[CollectionIndex] };
	}

	// Done!

	checkf(Index == Subobjects.Num(), TEXT("All pipeline subobjects must be initialized."));

	// Create ray tracing pipeline state object

	D3D12_STATE_OBJECT_DESC Desc = {};
	Desc.NumSubobjects = Index;
	Desc.pSubobjects = &Subobjects[0];
	Desc.Type = StateObjectType;

#if WITH_NVAPI
	bool bReorderingDisabled = false;
	bool bClusterOpsDisabled = false;
	if (IsRHIDeviceNVIDIA())
	{
		// Enable the NVAPI extension to allow shader extensions in ray tracing for shader execution reordering and cluster ops
		NvAPI_Status NvapiStatus = NvAPI_D3D12_SetNvShaderExtnSlotSpaceLocalThread(RayTracingDevice, UE_HLSL_SLOT_NV_SHADER_EXTN, UE_HLSL_SPACE_NV_SHADER_EXTN);
		bReorderingDisabled = bClusterOpsDisabled = ensureMsgf(NvapiStatus == NVAPI_OK, TEXT("NVAPI ERROR %d on setup extensions\n"), NvapiStatus);
	}
#endif

	HRESULT hr = RayTracingDevice->CreateStateObject(&Desc, IID_PPV_ARGS(Result.GetInitReference()));
	if (FAILED(hr))
	{
		// Failed state objects are not fatal if it's coming from partial/non-required RTPSO 
		// Will be fatal when it's actually needed for rendering
		UE_LOGF(LogD3D12RHI, Warning, "Failed to create raytracing RTPSO - add '-d3ddebug' to the command line to get more info on RTPSO compilation errors from the debug device");

		// Catch GPU crashes explicitly.
		if (hr == DXGI_ERROR_DEVICE_REMOVED || hr == DXGI_ERROR_DEVICE_HUNG)
		{
			VERIFYD3D12RESULT_EX(hr, RayTracingDevice);
		}
	}

#if WITH_NVAPI
	if (bReorderingDisabled && bClusterOpsDisabled)
	{
		// Disable the NVAPI extension slot again after state object creation.
		NvAPI_Status NvapiStatus = NvAPI_D3D12_SetNvShaderExtnSlotSpaceLocalThread(RayTracingDevice, ~0u, 0);
		checkf(NvapiStatus == NVAPI_OK, TEXT("NVAPI ERROR %d on clear extensions\n"), NvapiStatus);
	}
#endif

	INC_DWORD_STAT(STAT_D3D12RayTracingCreatedPipelines);
	INC_DWORD_STAT_BY(STAT_D3D12RayTracingCompiledShaders, NumExports);

	return Result;
}

inline uint64 GetShaderHash64(FRHIRayTracingShader* ShaderRHI)
{
	return ShaderRHI->GetHash().Hash;
}

// Generates a stable symbol name for a ray tracing shader, used for RT PSO creation.

inline FString GenerateShaderName(const TCHAR* Prefix, uint64 Hash)
{
	return FString::Printf(TEXT("%s_%016llx"), Prefix, Hash);
}

inline FString GenerateShaderName(FRHIRayTracingShader* ShaderRHI)
{
	const FD3D12RayTracingShader* Shader = FD3D12DynamicRHI::ResourceCast(ShaderRHI);
	uint64 ShaderHash = GetShaderHash64(ShaderRHI);
	return GenerateShaderName(*(Shader->EntryPoint), ShaderHash);
}

static FD3D12ShaderIdentifier GetShaderIdentifier(ID3D12StateObjectProperties* PipelineProperties, const TCHAR* ExportName)
{
	const void* ShaderIdData = PipelineProperties->GetShaderIdentifier(ExportName);
	checkf(ShaderIdData, TEXT("Couldn't find requested export in the ray tracing shader pipeline"));

	FD3D12ShaderIdentifier Result;
	Result.SetData(ShaderIdData);

	return Result;
}

static FD3D12ShaderIdentifier GetShaderIdentifier(ID3D12StateObject* StateObject, const TCHAR* ExportName)
{
	TRefCountPtr<ID3D12StateObjectProperties> PipelineProperties;
	HRESULT QueryInterfaceResult = StateObject->QueryInterface(IID_PPV_ARGS(PipelineProperties.GetInitReference()));
	checkf(SUCCEEDED(QueryInterfaceResult), TEXT("Failed to query pipeline properties from the ray tracing pipeline state object. Result=%08x"), QueryInterfaceResult);

	return GetShaderIdentifier(PipelineProperties, ExportName);
}

FD3D12RayTracingCompactionRequestHandler::FD3D12RayTracingCompactionRequestHandler(FD3D12Device* Device)
	: FD3D12DeviceChild(Device)
{
	const size_t BufferSize = GD3D12RayTracingMaxBatchedCompaction * sizeof(uint64);

	const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(BufferSize, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("PostBuildInfoBuffer"), BufferSize, 0, BUF_UnorderedAccess | BUF_SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute)
		.SetGPUMask(FRHIGPUMask::FromIndex(GetParentDevice()->GetGPUIndex()));

	PostBuildInfoBuffer = GetParentDevice()->GetParentAdapter()->CreateRHIBuffer(
		ResourceDesc,
		8,
		CreateDesc,
		ED3D12ResourceStateMode::MultiState,
		ED3D12Access::UAVCompute
	);
	SetD3D12ResourceName(PostBuildInfoBuffer->GetResource(), TEXT("PostBuildInfoBuffer"));

	PostBuildInfoStagingBuffer = RHICreateStagingBuffer();
}

void FD3D12RayTracingCompactionRequestHandler::RequestCompact(FD3D12RayTracingGeometry* InRTGeometry)
{
	uint32 GPUIndex = GetParentDevice()->GetGPUIndex();
	check(InRTGeometry->PerGPUData[GPUIndex].AccelerationStructureBuffer);
	ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(InRTGeometry->Initializer);
	check(EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction) &&
		EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::FastTrace) &&
		!EnumHasAnyFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate));

	FScopeLock Lock(&CS);
	PendingRequests.Add(InRTGeometry);
}

bool FD3D12RayTracingCompactionRequestHandler::ReleaseRequest(FD3D12RayTracingGeometry* InRTGeometry)
{
	FScopeLock Lock(&CS);

	// Remove from pending list, not found then try active requests
	if (PendingRequests.Remove(InRTGeometry) <= 0)
	{
		// If currently enqueued, then clear pointer to not handle the compaction request anymore			
		for (int32 BLASIndex = 0; BLASIndex < ActiveBLASGPUAddresses.Num(); ++BLASIndex)
		{
			if (ActiveRequests[BLASIndex] == InRTGeometry)
			{
				ActiveRequests[BLASIndex] = nullptr;
				return true;
			}
		}

		return false;
	}
	else
	{
		return true;
	}
}

void FD3D12RayTracingCompactionRequestHandler::Update(FD3D12CommandContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("FD3D12RT/Compaction"));
	FScopeLock Lock(&CS);

	// process previous build request data retrieval
	uint32 GPUIndex = GetParentDevice()->GetGPUIndex();

	if (ActiveBLASGPUAddresses.Num() > 0)
	{
		// Ensure that our builds & copies have finished on GPU when enqueued - if still busy then wait until done
		if (PostBuildInfoBufferReadbackSyncPoint && !PostBuildInfoBufferReadbackSyncPoint->IsComplete())
		{
			return;
		}

		// Readback the sizes from the readback buffer and schedule new builds ops on the RTGeometry objects
		uint64* SizesAfterCompaction = (uint64*)PostBuildInfoStagingBuffer->Lock(0, ActiveBLASGPUAddresses.Num() * sizeof(uint64));
		for (int32 BLASIndex = 0; BLASIndex < ActiveBLASGPUAddresses.Num(); ++BLASIndex)
		{
			if (ActiveRequests[BLASIndex] != nullptr)
			{
				ActiveRequests[BLASIndex]->CompactAccelerationStructure(Context, GPUIndex, SizesAfterCompaction[BLASIndex]);
			}
		}
		PostBuildInfoStagingBuffer->Unlock();

		// reset working values
		PostBuildInfoBufferReadbackSyncPoint = nullptr;
		ActiveRequests.Empty(ActiveRequests.Num());
		ActiveBLASGPUAddresses.Empty(ActiveBLASGPUAddresses.Num());
	}

	// build a new set of build requests to extract the build data	
	for (FD3D12RayTracingGeometry* RTGeometry : PendingRequests)
	{
		ActiveRequests.Add(RTGeometry);

		FD3D12ResourceLocation& ResourceLocation = RTGeometry->PerGPUData[GPUIndex].AccelerationStructureBuffer.GetReference()->ResourceLocation;
		ActiveBLASGPUAddresses.Add(ResourceLocation.GetGPUVirtualAddress());

		Context.UpdateResidency(ResourceLocation.GetResource());

		// enqueued enough requests for this update round
		if (ActiveRequests.Num() >= GD3D12RayTracingMaxBatchedCompaction)
		{
			break;
		}
	}

	// Do we have requests?
	if (ActiveRequests.Num() > 0)
	{
		// clear out all of the pending requests, don't allow the array to shrink
		PendingRequests.RemoveAt(0, ActiveRequests.Num(), EAllowShrinking::No);

		D3D12_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO_DESC PostBuildInfoDesc = {};
		PostBuildInfoDesc.DestBuffer = PostBuildInfoBuffer->ResourceLocation.GetGPUVirtualAddress();
		PostBuildInfoDesc.InfoType = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_POSTBUILD_INFO_COMPACTED_SIZE;

		//PostBuildInfoBuffer enters in D3D12_RESOURCE_STATE_UNORDERED_ACCESS
		//Context.TransitionResource(PostBuildInfoBuffer->GetResource(), D3D12_RESOURCE_STATE_COPY_SOURCE, D3D12_RESOURCE_STATE_UNORDERED_ACCESS, 0);

		// @TODO - Might do better if we had a way to say "BVHPostBuildInfoRead"
		// Force UAV barrier to make sure all previous builds ops are finished
		Context.AddGlobalBarrier(
			ED3D12Access::BVHWrite,
			ED3D12Access::BVHRead);

		Context.UpdateResidency(PostBuildInfoBuffer->GetResource());

		// Must flush barriers before making a direct call to a D3D API
		Context.FlushResourceBarriers();

		// Emit the RT post build info from the selected requests
		Context.RayTracingCommandList()->EmitRaytracingAccelerationStructurePostbuildInfo(&PostBuildInfoDesc, ActiveBLASGPUAddresses.Num(), ActiveBLASGPUAddresses.GetData());

		// Transition to copy source and perform the copy to readback
		Context.AddBarrier(
			PostBuildInfoBuffer->GetResource(),
			ED3D12Access::UAVCompute,
			ED3D12Access::CopySrc,
			0);

		Context.RHICopyToStagingBuffer(PostBuildInfoBuffer, PostBuildInfoStagingBuffer, 0, sizeof(uint64) * ActiveBLASGPUAddresses.Num());

		Context.AddBarrier(
			PostBuildInfoBuffer->GetResource(),
			ED3D12Access::CopySrc,
			ED3D12Access::UAVCompute,
			0);

		// Update the sync point
		PostBuildInfoBufferReadbackSyncPoint = Context.GetContextSyncPoint();
	}
}


// Cache for ray tracing pipeline collection objects, containing single shaders that can be linked into full pipelines.
class FD3D12RayTracingPipelineCache : FD3D12AdapterChild
{
public:

	UE_NONCOPYABLE(FD3D12RayTracingPipelineCache)

	FD3D12RayTracingPipelineCache(FD3D12Adapter* Adapter)
		: FD3D12AdapterChild(Adapter)
		, DefaultLocalRootSignature(Adapter)
	{
		// Default empty local root signature
		LLM_SCOPE_BYNAME(TEXT("FD3D12RT/PipelineCache"));
		D3D12_VERSIONED_ROOT_SIGNATURE_DESC LocalRootSignatureDesc = {};
		if (GetParentAdapter()->GetRootSignatureVersion() >= D3D_ROOT_SIGNATURE_VERSION_1_1)
		{
			LocalRootSignatureDesc.Version = D3D_ROOT_SIGNATURE_VERSION_1_1;
			LocalRootSignatureDesc.Desc_1_1.Flags |= D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE;
		}
		else
		{
			LocalRootSignatureDesc.Version = D3D_ROOT_SIGNATURE_VERSION_1_0;
			LocalRootSignatureDesc.Desc_1_0.Flags |= D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE;
		}
		
		DefaultLocalRootSignature.Init(LocalRootSignatureDesc, RS_RayTracingLocal);
	}

	~FD3D12RayTracingPipelineCache()
	{
		Reset();
	}

	struct FKey
	{
		uint64 ShaderHash = 0;
		uint32 MaxAttributeSizeInBytes = 0;
		uint32 MaxPayloadSizeInBytes = 0;
		ID3D12RootSignature* GlobalRootSignature = nullptr;
		ID3D12RootSignature* LocalRootSignature = nullptr;

		bool operator == (const FKey& Other) const
		{
			return ShaderHash == Other.ShaderHash
				&& MaxAttributeSizeInBytes == Other.MaxAttributeSizeInBytes
				&& MaxPayloadSizeInBytes == Other.MaxPayloadSizeInBytes
				&& GlobalRootSignature == Other.GlobalRootSignature
				&& LocalRootSignature == Other.LocalRootSignature;
		}

		inline friend uint32 GetTypeHash(const FKey& Key)
		{
			return Key.ShaderHash;
		}
	};

	enum class ECollectionType
	{
		Unknown,
		RayGen,
		Miss,
		HitGroup,
		Callable,
	};

	struct FEntry
	{
		// Move-only type
		FEntry() = default;
		FEntry(FEntry&& Other) = default;

		FEntry(const FEntry&) = delete;
		FEntry& operator = (const FEntry&) = delete;
		FEntry& operator = (FEntry&& Other) = delete;

		D3D12_EXISTING_COLLECTION_DESC GetCollectionDesc()
		{
			check(bDeserialized || (CompileEvent.IsValid() && CompileEvent->IsComplete()));
			check(StateObject);
			if (StateObject == nullptr)
			{
				UE_LOGF(LogD3D12RHI, Fatal, "Trying to add a raytracing state collection which failed to compile to a RTPSO collection.");
			}

			D3D12_EXISTING_COLLECTION_DESC Result = {};
			Result.pExistingCollection = StateObject;

			return Result;
		}

		const TCHAR* GetPrimaryExportNameChars()
		{
			checkf(ExportNames.Num()!=0, TEXT("This ray tracing shader collection does not export any symbols."));
			return *(ExportNames[0]);
		}

		ECollectionType CollectionType = ECollectionType::Unknown;

		TRefCountPtr<FD3D12RayTracingShader> Shader;

		TRefCountPtr<ID3D12StateObject> StateObject;
		FD3D12RayTracingPipelineInfo PipelineInfo;

		FGraphEventRef CompileEvent;
		bool bDeserialized = false;

		static constexpr uint32 MaxExports = 4;
		TArray<FString, TFixedAllocator<MaxExports>> ExportNames;

		FD3D12ShaderIdentifier Identifier;

		float CompileTimeMS = 0.0f;
	};

	static const TCHAR* GetCollectionTypeName(ECollectionType Type)
	{
		switch (Type)
		{
		case ECollectionType::Unknown:
			return TEXT("Unknown");
		case ECollectionType::RayGen:
			return TEXT("RayGen");
		case ECollectionType::Miss:
			return TEXT("Miss");
		case ECollectionType::HitGroup:
			return TEXT("HitGroup");
		case ECollectionType::Callable:
			return TEXT("Callable");
		default:
			return TEXT("");
		}
	}

	class FShaderCompileTask
	{
	public:

		UE_NONCOPYABLE(FShaderCompileTask)

		FShaderCompileTask(
				FEntry& InEntry,
				FKey InCacheKey,
				FD3D12Device* InDevice,
				ECollectionType InCollectionType,
				bool bInRequired)
			: Entry(InEntry)
			, CacheKey(InCacheKey)
			, Device(InDevice)
			, RayTracingDevice(InDevice->GetParentAdapter()->GetD3DDevice5())
			, CollectionType(InCollectionType)
			, bRequired(bInRequired)
		{
		}

		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
		{
			SCOPE_CYCLE_COUNTER(STAT_RTPSO_CompileShader);
			TRACE_CPUPROFILER_EVENT_SCOPE(ShaderCompileTask);

			uint64 CompileTimeCycles = 0;
			CompileTimeCycles -= FPlatformTime::Cycles64();

			FD3D12RayTracingShader* Shader = Entry.Shader;

			static constexpr uint32 MaxEntryPoints = 3; // CHS+AHS+IS for HitGroup or just a single entry point for other collection types
			TArray<LPCWSTR, TFixedAllocator<MaxEntryPoints>> OriginalEntryPoints;
			TArray<LPCWSTR, TFixedAllocator<MaxEntryPoints>> RenamedEntryPoints;

			const uint32 NumHitGroups = CollectionType == ECollectionType::HitGroup ? 1 : 0;
			const uint64 ShaderHash = CacheKey.ShaderHash;
			ID3D12RootSignature* GlobalRootSignature = CacheKey.GlobalRootSignature;
			ID3D12RootSignature* LocalRootSignature = CacheKey.LocalRootSignature;
			const uint32 DefaultLocalRootSignatureIndex = 0;
			uint32 MaxAttributeSizeInBytes = CacheKey.MaxAttributeSizeInBytes;
			uint32 MaxPayloadSizeInBytes = CacheKey.MaxPayloadSizeInBytes;

			D3D12_HIT_GROUP_DESC HitGroupDesc = {};

			if (CollectionType == ECollectionType::HitGroup)
			{
				HitGroupDesc.HitGroupExport = Entry.GetPrimaryExportNameChars();
				HitGroupDesc.Type = Shader->IntersectionEntryPoint.IsEmpty() ? D3D12_HIT_GROUP_TYPE_TRIANGLES : D3D12_HIT_GROUP_TYPE_PROCEDURAL_PRIMITIVE;

				{
					const FString& ExportName = Entry.ExportNames.Add_GetRef(GenerateShaderName(TEXT("CHS"), ShaderHash));

					HitGroupDesc.ClosestHitShaderImport = *ExportName;

					OriginalEntryPoints.Add(*(Shader->EntryPoint));
					RenamedEntryPoints.Add(*ExportName);
				}

				if (!Shader->AnyHitEntryPoint.IsEmpty())
				{
					const FString& ExportName = Entry.ExportNames.Add_GetRef(GenerateShaderName(TEXT("AHS"), ShaderHash));

					HitGroupDesc.AnyHitShaderImport = *ExportName;

					OriginalEntryPoints.Add(*(Shader->AnyHitEntryPoint));
					RenamedEntryPoints.Add(*ExportName);
				}

				if (!Shader->IntersectionEntryPoint.IsEmpty())
				{
					const FString& ExportName = Entry.ExportNames.Add_GetRef(GenerateShaderName(TEXT("IS"), ShaderHash));

					HitGroupDesc.IntersectionShaderImport = *ExportName;

					OriginalEntryPoints.Add(*(Shader->IntersectionEntryPoint));
					RenamedEntryPoints.Add(*ExportName);
				}
			}
			else
			{
				checkf(CollectionType == ECollectionType::Miss || CollectionType == ECollectionType::RayGen || CollectionType == ECollectionType::Callable, TEXT("Unexpected RT shader collection type"));

				OriginalEntryPoints.Add(*(Shader->EntryPoint));
				RenamedEntryPoints.Add(Entry.GetPrimaryExportNameChars());
			}

			// Validate that memory reservation was correct

			check(Entry.ExportNames.Num() <= Entry.MaxExports);

			FDXILLibrary Library;
			Library.InitFromDXIL(Shader->GetShaderBytecode(), OriginalEntryPoints.GetData(), RenamedEntryPoints.GetData(), OriginalEntryPoints.Num());

			const FDXILLibrary* LibraryPtr = &Library;

			Entry.StateObject = CreateRayTracingStateObject(
				RayTracingDevice,
				MakeArrayView(&LibraryPtr, 1),
				RenamedEntryPoints,
				MaxAttributeSizeInBytes,
				MaxPayloadSizeInBytes,
				MakeArrayView(&HitGroupDesc, NumHitGroups),
				GlobalRootSignature,
				MakeArrayView(&LocalRootSignature, 1),
				{}, // LocalRootSignatureAssociations (single RS will be used for all exports since this is null)
				{}, // ExistingCollections
				D3D12_STATE_OBJECT_TYPE_COLLECTION);

			if (Entry.StateObject)
			{
				Device->GetRayTracingPipelineInfo(Entry.StateObject, &Entry.PipelineInfo);

				// Retrieve the identifier from the library
				Entry.Identifier = GetShaderIdentifier(Entry.StateObject, Entry.GetPrimaryExportNameChars());
			}
			else if (bRequired)
			{
				UE_LOGF(LogD3D12RHI, Fatal, "Failed to create a required RTPSO pipeline state collection");
			}
			
			CompileTimeCycles += FPlatformTime::Cycles64();

			Entry.CompileTimeMS = float(FPlatformTime::ToMilliseconds64(CompileTimeCycles));

			if (Entry.CompileTimeMS >= 1000.0f)
			{
				// Log compilations of individual shaders that took more than 1 second
				UE_LOGF(LogD3D12RHI, Log, "Compiled %ls for RTPSO in %.2f ms.", OriginalEntryPoints[0], Entry.CompileTimeMS);
			}
		}

		FORCEINLINE TStatId GetStatId() const
		{
			return GET_STATID(STAT_RTPSO_CompileShader);
		}

		ENamedThreads::Type GetDesiredThread()
		{
			return ENamedThreads::AnyHiPriThreadHiPriTask;
		}

		FEntry& Entry;
		FKey CacheKey;
		FD3D12Device* Device;
		ID3D12Device5* RayTracingDevice;
		ECollectionType CollectionType;
		bool bRequired;
	};

	FEntry* GetOrCompileShader(
		FD3D12Device* Device,
		FD3D12RayTracingShader* Shader,
		ID3D12RootSignature* GlobalRootSignature,
		uint32 MaxAttributeSizeInBytes,
		uint32 MaxPayloadSizeInBytes,
		bool bRequired,
		ECollectionType CollectionType,
		FGraphEventArray& CompletionList,
		bool* bOutCacheHit = nullptr)
	{
		FScopeLock Lock(&CriticalSection);

		const uint64 ShaderHash = GetShaderHash64(Shader);

		ID3D12RootSignature* LocalRootSignature = nullptr;
		if (CollectionType == ECollectionType::RayGen)
		{
			// RayGen shaders use a default empty local root signature as all their resources bound via global RS.
			LocalRootSignature = DefaultLocalRootSignature.GetRootSignature();
		}
		else
		{
			// All other shaders (hit groups, miss, callable) use custom root signatures.
			LocalRootSignature = Shader->LocalRootSignature->GetRootSignature();
		}

		FKey CacheKey;
		CacheKey.ShaderHash = ShaderHash;
		CacheKey.MaxAttributeSizeInBytes = MaxAttributeSizeInBytes;
		CacheKey.MaxPayloadSizeInBytes = MaxPayloadSizeInBytes;
		CacheKey.GlobalRootSignature = GlobalRootSignature;
		CacheKey.LocalRootSignature = LocalRootSignature;

		FEntry*& FindResult = Cache.FindOrAdd(CacheKey);

		if (FindResult)
		{
			if (bOutCacheHit) *bOutCacheHit = true;
		}
		else
		{
			if (bOutCacheHit) *bOutCacheHit = false;

			if (FindResult == nullptr)
			{
				FindResult = new FEntry;
			}

			FEntry& Entry = *FindResult;

			Entry.CollectionType = CollectionType;
			Entry.Shader = Shader;

			if (Shader->bPrecompiledPSO)
			{
				D3D12_SHADER_BYTECODE Bytecode = Shader->GetShaderBytecode();
				Entry.StateObject = Device->DeserializeRayTracingStateObject(Bytecode, GlobalRootSignature);
				if (Entry.StateObject)
				{
					Device->GetRayTracingPipelineInfo(Entry.StateObject, &Entry.PipelineInfo);
				}

				checkf(Entry.StateObject != nullptr, TEXT("Failed to deserialize RTPSO"));

				Entry.ExportNames.Add(Shader->EntryPoint);
				Entry.Identifier = GetShaderIdentifier(Entry.StateObject, *Shader->EntryPoint);
				Entry.bDeserialized = true;
			}
			else
			{
				// Generate primary export name, which is immediately required on the PSO creation thread.
				Entry.ExportNames.Add(GenerateShaderName(GetCollectionTypeName(CollectionType), ShaderHash));
				checkf(Entry.ExportNames.Num() == 1, TEXT("Primary export name must always be first."));

				// Defer actual compilation to another task, as there may be many shaders that may be compiled in parallel.
				// Result of the compilation (the collection PSO) is not needed until final RT PSO is linked.
				Entry.CompileEvent = TGraphTask<FShaderCompileTask>::CreateTask().ConstructAndDispatchWhenReady(
					Entry,
					CacheKey,
					Device,
					CollectionType,
					bRequired
				);
			}
		}

		if (FindResult->CompileEvent.IsValid())
		{
			if (!FindResult->CompileEvent->IsComplete())
			{
				CompletionList.Add(FindResult->CompileEvent);
			}
		}
		else
		{
			// required PSOs collection are fatal if not valid (can have failed during precaching of this
			// RTPSO collection and wasn't fatal during precaching, but did fill the cache)
			if (bRequired && FindResult->StateObject == nullptr)
			{
				UE_LOGF(LogD3D12RHI, Fatal, "Failed to create a required RTPSO collection");
			}
		}

		return FindResult;
	}

	void Reset()
	{
		FScopeLock Lock(&CriticalSection);

		for (auto It : Cache)
		{
			delete It.Value;
		}

		Cache.Reset();
	}

	ID3D12RootSignature* GetGlobalRootSignature(const FRHIShaderBindingLayout& ShaderBindingLayout)
	{
		FD3D12Adapter* Adapter = GetParentAdapter();
		const FD3D12RootSignature* RootSignature = Adapter->GetGlobalRayTracingRootSignature(ShaderBindingLayout);
		return RootSignature->GetRootSignature();
	}

private:

	FCriticalSection CriticalSection;
	TMap<FKey, FEntry*> Cache;
	FD3D12RootSignature DefaultLocalRootSignature; // Default empty root signature used for default hit shaders.
};

inline bool AreBindlessResourcesEnabledForRayTracing(FD3D12Adapter* Adapter)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorManager& Manager = Adapter->GetDevice(0)->GetBindlessDescriptorManager();
	if (IsBindlessEnabledForRayTracing(Manager.GetConfiguration()))
	{
		return true;
	}
#endif
	return false;
}

// Helper class used to manage SBT buffer for a specific GPU
class FD3D12RayTracingShaderBindingTableInternal
{
private:
	void WriteData(uint32 WriteOffset, const void* InData, uint32 InDataSize)
	{
#if DO_CHECK && DO_GUARD_SLOW
		Data.RangeCheck(WriteOffset);
		Data.RangeCheck(WriteOffset + InDataSize - 1);
#endif // DO_CHECK && DO_GUARD_SLOW

		FMemory::Memcpy(Data.GetData() + WriteOffset, InData, InDataSize);
	}

	void CompareData(uint32 Offset, const void* InData, uint32 InDataSize)
	{
#if DO_CHECK
		const uint8* CurrentData = Data.GetData() + Offset;
		const uint8* InBytes = static_cast<const uint8*>(InData);

		if (FMemory::Memcmp(CurrentData, InBytes, InDataSize) != 0)
		{
			// Find first differing byte
			uint32 FirstDiff = 0;
			for (uint32 i = 0; i < InDataSize; ++i)
			{
				if (CurrentData[i] != InBytes[i])
				{
					FirstDiff = i;
					break;
				}
			}

			const int32 BytesToDump = FMath::Min<int32>(64, InDataSize - FirstDiff);

			UE_LOGF(LogD3D12RHI, Error, "CompareData mismatch at buffer offset %u + %u (absolute byte %u), comparing %u bytes. Dumping %d bytes from first diff:",
				Offset, FirstDiff, Offset + FirstDiff, InDataSize, BytesToDump);
			UE_LOGF(LogD3D12RHI, Error, "  Current:  %ls", *BytesToHex(CurrentData + FirstDiff, BytesToDump));
			UE_LOGF(LogD3D12RHI, Error, "  Expected: %ls", *BytesToHex(InBytes + FirstDiff, BytesToDump));

			ensureMsgf(false, TEXT("SBT CompareData mismatch detected. See above log messages for byte offset and hex dump details."));
		}
#endif // DO_CHECK
	}

	void WriteLocalShaderRecord(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize)
	{
		checkfSlow(OffsetWithinRecord % 4 == 0, TEXT("SBT record parameters must be written on DWORD-aligned boundary"));
		checkfSlow(InDataSize % 4 == 0, TEXT("SBT record parameters must be DWORD-aligned"));
		checkfSlow(OffsetWithinRecord + InDataSize <= LocalRecordSizeUnaligned, TEXT("SBT record write request is out of bounds"));

		const uint32 WriteOffset = ShaderTableOffset + LocalRecordStride * RecordIndex + OffsetWithinRecord;

		WriteData(WriteOffset, InData, InDataSize);
	}

	void CompareLocalShaderRecord(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize)
	{
		const uint32 Offset = ShaderTableOffset + LocalRecordStride * RecordIndex + OffsetWithinRecord;
		CompareData(Offset, InData, InDataSize);
	}

public:

	UE_NONCOPYABLE(FD3D12RayTracingShaderBindingTableInternal)

	// Ray tracing shader bindings can be processed in parallel.
	// Each concurrent worker gets its own dedicated descriptor cache instance to avoid contention or locking.
	// Scaling beyond 5 total threads does not yield any speedup in practice.
	static constexpr uint32 MaxBindingWorkers = 5; // RHI thread + 4 parallel workers.

	FD3D12RayTracingShaderBindingTableInternal(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer, FD3D12Device* InDevice)
		: UniqueId(NextUniqueId++)
	{
		checkf(Initializer.LocalBindingDataSize <= 4096, TEXT("The maximum size of a local root signature is 4KB.")); // as per section 4.22.1 of DXR spec v1.0
		check(Initializer.ShaderBindingMode != ERayTracingShaderBindingMode::Disabled);
		
		Device = InDevice;

		Lifetime = Initializer.Lifetime;
		HitGroupIndexingMode = Initializer.HitGroupIndexingMode;
		ShaderBindingMode = Initializer.ShaderBindingMode;
		NumShaderSlotsPerGeometrySegment = Initializer.NumShaderSlotsPerGeometrySegment;

		const uint32 NumHitGroupSlots = Initializer.HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow ? Initializer.NumGeometrySegments * Initializer.NumShaderSlotsPerGeometrySegment : 1;

		NumMissRecords = Initializer.NumMissShaderSlots;
		NumHitRecords = NumHitGroupSlots;
		NumCallableRecords = Initializer.NumCallableShaderSlots;

		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
		{
			checkf(Initializer.LocalBindingDataSize >= sizeof(FD3D12HitGroupSystemParameters), TEXT("All local root signatures are expected to contain ray tracing system root parameters"));

			LocalRecordSizeUnaligned = ShaderIdentifierSize + Initializer.LocalBindingDataSize;
			LocalRecordStride = RoundUpToNextMultiple(LocalRecordSizeUnaligned, D3D12_RAYTRACING_SHADER_RECORD_BYTE_ALIGNMENT);

			// Custom descriptor cache is only required when local resources may be bound.
			// If only global resources are used, then transient descriptor cache can be used.
			const bool bNeedsDescriptorCache = (NumHitGroupSlots + Initializer.NumCallableShaderSlots + Initializer.NumMissShaderSlots) * Initializer.LocalBindingDataSize != 0;

			if (bNeedsDescriptorCache)
			{
				// #dxr_todo UE-72158: Remove this when RT descriptors are sub-allocated from the global view descriptor heap.

				if (GD3D12ExplicitViewDescriptorHeapOverflowReported)
				{
					GD3D12ExplicitViewDescriptorHeapSize = GD3D12ExplicitViewDescriptorHeapSize * 2;
					GD3D12ExplicitViewDescriptorHeapOverflowReported = 0;
				}

				// D3D12 is guaranteed to support 1M (D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_1) descriptors in a CBV/SRV/UAV heap, so clamp the size to this.
				// https://docs.microsoft.com/en-us/windows/desktop/direct3d12/hardware-support
				const uint32 NumViewDescriptors = FMath::Min(D3D12_MAX_SHADER_VISIBLE_DESCRIPTOR_HEAP_SIZE_TIER_1, GD3D12ExplicitViewDescriptorHeapSize);
				const uint32 NumSamplerDescriptors = D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE;

				DescriptorCache = new FD3D12ExplicitDescriptorCache(Device, MaxBindingWorkers);
				DescriptorCache->Init(0, NumViewDescriptors, NumSamplerDescriptors, ERHIBindlessConfiguration::RayTracing);
			}

			uint32 TotalDataSize = 0;

			HitGroupShaderTableOffset = TotalDataSize;
			TotalDataSize += NumHitGroupSlots * LocalRecordStride;
			TotalDataSize = RoundUpToNextMultiple(TotalDataSize, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

			CallableShaderTableOffset = TotalDataSize;
			TotalDataSize += Initializer.NumCallableShaderSlots * LocalRecordStride;
			TotalDataSize = RoundUpToNextMultiple(TotalDataSize, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

			MissShaderTableOffset = TotalDataSize;
			TotalDataSize += Initializer.NumMissShaderSlots * LocalRecordStride;
			TotalDataSize = RoundUpToNextMultiple(TotalDataSize, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

			Data.SetNumZeroed(TotalDataSize);
#if DO_CHECK
			bWasDefaultMissShaderSet = false;
#endif
			SetDefaultHitGroupIdentifier(FD3D12ShaderIdentifier::Null);
			SetDefaultMissShaderIdentifier(FD3D12ShaderIdentifier::Null);
			SetDefaultCallableShaderIdentifier(FD3D12ShaderIdentifier::Null);

			// Keep CPU-side data after upload
			Data.SetAllowCPUAccess(true);
		}

		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && Initializer.NumGeometrySegments > 0)
		{
			// Doesn't make sense to have inline SBT without hitgroup indexing
			check(HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow);

			const uint32 ParameterBufferSize = Initializer.NumGeometrySegments * sizeof(FD3D12HitGroupSystemParameters);
			InlineGeometryParameterData.SetNumZeroed(ParameterBufferSize);
		}

		if (Lifetime == ERayTracingShaderBindingTableLifetime::Persistent)
		{
			HitRecordData.SetNum(NumHitRecords);
		}
	}

	~FD3D12RayTracingShaderBindingTableInternal()
	{
		// Might be slow - iterate faster somehow?
		for (int32 RecordIndex = 0; RecordIndex <= WorkerData[0].MaxUsedHitRecordIndex; ++RecordIndex)
		{
			ClearHitRecordData(0, RecordIndex);
		}

		delete DescriptorCache;
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		for (FWorkerThreadData& ThisWorkerData : WorkerData)
		{
			for (FD3D12ConstantBufferView* CBV : ThisWorkerData.LooseParameterCBVs)
			{
				delete CBV;
			}
		}
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	}

	template <typename T>
	void SetLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void SetLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const void* InData, uint32 InDataSize)
	{
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, InData, InDataSize);
	}

	template <typename T>
	void CompareLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		CompareLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void CompareLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const void* InData, uint32 InDataSize)
	{
		CompareLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, InData, InDataSize);
	}

	template <typename T>
	void SetMissShaderParameters(uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		const uint32 ShaderTableOffset = MissShaderTableOffset;
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	template <typename T>
	void SetCallableShaderParameters(uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		const uint32 ShaderTableOffset = CallableShaderTableOffset;
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void CopyLocalShaderParameters(uint32 InShaderTableOffset, uint32 InDestRecordIndex, uint32 InSourceRecordIndex, uint32 InOffsetWithinRootSignature)
	{
		const uint32 BaseOffset = InShaderTableOffset + ShaderIdentifierSize + InOffsetWithinRootSignature;
		const uint32 DestOffset = BaseOffset + LocalRecordStride * InDestRecordIndex;
		const uint32 SourceOffset = BaseOffset + LocalRecordStride * InSourceRecordIndex;
		const uint32 CopySize = LocalRecordStride - ShaderIdentifierSize - InOffsetWithinRootSignature;
		checkSlow(CopySize <= LocalRecordStride);

		FMemory::Memcpy(
			Data.GetData() + DestOffset,
			Data.GetData() + SourceOffset,
			CopySize);
	}

	void CopyHitGroupParameters(uint32 InDestRecordIndex, uint32 InSourceRecordIndex, uint32 InOffsetWithinRootSignature)
	{
		const uint32 ShaderTableOffset = HitGroupShaderTableOffset;
		CopyLocalShaderParameters(ShaderTableOffset, InDestRecordIndex, InSourceRecordIndex, InOffsetWithinRootSignature);
	}	

	void SetMissIdentifier(uint32 RecordIndex, const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = MissShaderTableOffset + RecordIndex * LocalRecordStride;
#if DO_CHECK
		if (RecordIndex == 0)
		{
			bWasDefaultMissShaderSet = true;
		}
#endif
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetCallableIdentifier(uint32 RecordIndex, const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = CallableShaderTableOffset + RecordIndex * LocalRecordStride;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetDefaultHitGroupIdentifier(const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = HitGroupShaderTableOffset;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetHitGroupSystemParameters(uint32 RecordIndex, const FD3D12HitGroupSystemParameters& SystemParameters)
	{
		const uint32 OffsetWithinRootSignature = 0; // System parameters are always first in the RS.
		const uint32 ShaderTableOffset = HitGroupShaderTableOffset;
		SetLocalShaderParameters(ShaderTableOffset, RecordIndex, OffsetWithinRootSignature, SystemParameters);
	}

	void CompareHitGroupSystemParameters(uint32 RecordIndex, const FD3D12HitGroupSystemParameters& SystemParameters)
	{
		const uint32 OffsetWithinRootSignature = 0; // System parameters are always first in the RS.
		const uint32 ShaderTableOffset = HitGroupShaderTableOffset;
		CompareLocalShaderParameters(ShaderTableOffset, RecordIndex, OffsetWithinRootSignature, SystemParameters);
	}

	void SetHitGroupIdentifier(uint32 RecordIndex, const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		checkfSlow(ShaderIdentifier.IsValid(), TEXT("Shader identifier must be initialized FD3D12RayTracingPipelineState::GetShaderIdentifier() before use."));
		checkSlow(sizeof(ShaderIdentifier.Data) >= ShaderIdentifierSize);

		const uint32 WriteOffset = HitGroupShaderTableOffset + RecordIndex * LocalRecordStride;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}	

	void CompareHitGroupIdentifier(uint32 RecordIndex, const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		checkfSlow(ShaderIdentifier.IsValid(), TEXT("Shader identifier must be initialized FD3D12RayTracingPipelineState::GetShaderIdentifier() before use."));
		checkSlow(sizeof(ShaderIdentifier.Data) >= ShaderIdentifierSize);

		const uint32 WriteOffset = HitGroupShaderTableOffset + RecordIndex * LocalRecordStride;
		CompareData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetDefaultMissShaderIdentifier(const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		// Set all slots to the same default
		for (uint32 Index = 0; Index < NumMissRecords; ++Index)
		{
			SetMissIdentifier(Index, ShaderIdentifier);
		}

#if DO_CHECK
		bWasDefaultMissShaderSet = false;
#endif
	}

	void SetDefaultCallableShaderIdentifier(const FD3D12ShaderIdentifier& ShaderIdentifier)
	{
		for (uint32 Index = 0; Index < NumCallableRecords; ++Index)
		{
			SetCallableIdentifier(Index, ShaderIdentifier);
		}
	}

	void SetInlineGeometryParameters(uint32 InlineRecordIndex, const FD3D12HitGroupSystemParameters& SystemParameters)
	{
		const uint32 DataSize = sizeof(FD3D12HitGroupSystemParameters);
		const uint32 WriteOffset = DataSize * InlineRecordIndex;
		FMemory::Memcpy(&InlineGeometryParameterData[WriteOffset], &SystemParameters, DataSize);
	}

	void CompareInlineGeometryParameters(uint32 InlineRecordIndex, const FD3D12HitGroupSystemParameters& SystemParameters)
	{
		const uint32 DataSize = sizeof(FD3D12HitGroupSystemParameters);
		const uint32 WriteOffset = DataSize * InlineRecordIndex;
		const void* CurrentData = InlineGeometryParameterData.GetData() + WriteOffset;
		ensure(FMemory::Memcmp(CurrentData, &SystemParameters, DataSize) == 0);
	}
	   
	FRHISizeAndStride GetInlineBindingDataSizeAndStride() const
	{
		return FRHISizeAndStride { (uint64)InlineGeometryParameterData.Num(), sizeof(FD3D12HitGroupSystemParameters) };
	}

	void Commit(FD3D12CommandContext& Context, FRHIBuffer* InInlineBindingDataBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ShaderTableCommit);

		check(IsInRHIThread() || !IsRunningRHIInSeparateThread());
		checkf(bIsDirty, TEXT("bIsDirty should be checked before calling Commit()"));

	#if ENABLE_RESIDENCY_MANAGEMENT
		// Merge all data from worker threads into the main set
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MergeWorkerData);

			for (uint32 WorkerIndex = 1; WorkerIndex < MaxBindingWorkers; ++WorkerIndex)
			{
				for (FD3D12Resource* Resource : WorkerData[WorkerIndex].DynamicReferencedResources)
				{
					AddDynamicReferencedResource(0, Resource);
				}
				for (auto Iter = WorkerData[WorkerIndex].PersistentReferencedResources.CreateIterator(); Iter; ++Iter)
				{
					UpdatePersistentReferencedResource(0, Iter.Key(), Iter.Value());
				}
				WorkerData[WorkerIndex].DynamicReferencedResources.Empty();
				WorkerData[WorkerIndex].DynamicReferencedResourceSet.Empty();
				WorkerData[WorkerIndex].PersistentReferencedResources.Empty();
				WorkerData[WorkerIndex].LastTrackedBaseShaderResources.Empty();
			}
		}

		// Build the used persistent d3d12 resource set from the views
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildUsedResourceSet);
			
			ReferencedResources.Empty(WorkerData[0].PersistentReferencedResources.Num() + WorkerData[0].DynamicReferencedResources.Num());

			// Add tracked resources for persistent SBT records
			for (auto Iter = WorkerData[0].PersistentReferencedResources.CreateIterator(); Iter; ++Iter)
			{
				ReferencedResources.Add(Iter.Key());
			}

			// Also add all the collected dynamic resources and reset for the next 'frame'			
			ReferencedResources.Append(WorkerData[0].DynamicReferencedResources);
			WorkerData[0].DynamicReferencedResources.Empty();
			WorkerData[0].DynamicReferencedResourceSet.Empty();
			WorkerData[0].LastTrackedBaseShaderResources.Empty();
		}
	#endif // ENABLE_RESIDENCY_MANAGEMENT

		// Merge max record index from worker data into index 0
		for (uint32 WorkerIndex = 1; WorkerIndex < MaxBindingWorkers; ++WorkerIndex)
		{
			WorkerData[0].MaxUsedHitRecordIndex = FMath::Max(WorkerData[0].MaxUsedHitRecordIndex, WorkerData[WorkerIndex].MaxUsedHitRecordIndex);
			WorkerData[WorkerIndex].MaxUsedHitRecordIndex = 0;
		}

		// Clear the record cache for reuse for all worker threads (including index 0)
		for (uint32 WorkerIndex = 0; WorkerIndex < MaxBindingWorkers; ++WorkerIndex)
		{
			WorkerData[WorkerIndex].ShaderRecordCache.Empty();
		}

		FD3D12Adapter* Adapter = Device->GetParentAdapter();

		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
		{
			checkf(Data.Num(), TEXT("Shader table is expected to be initialized before copying to GPU."));
			#if DO_CHECK
			// Need to split out hit binding data from miss and callable because it's possible that hit binding data wasn't set or needed (no valid binding),
			// but miss data is still needed, and we also don't need to copy the complete allocated hit binding data but only upto MaxUsedHitRecordIndex
			// which is not easily possible with the 1 big allocation.
			//checkf(bWasDefaultMissShaderSet, TEXT("At least the first miss shader must have been set before copying to GPU."));
			#endif

			const uint32 BufferSize = Data.GetResourceDataSize();

			D3D12_RESOURCE_DESC BufferDesc = CD3DX12_RESOURCE_DESC::Buffer(BufferSize, D3D12_RESOURCE_FLAG_NONE, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("ShaderBindingTable"), BufferSize, 0, BUF_Static)
			.SetInitialState(ERHIAccess::CopyDest)
			.SetGPUMask(FRHIGPUMask::FromIndex(Device->GetGPUIndex()));

			ID3D12ResourceAllocator* ResourceAllocator = nullptr;
			Buffer = Adapter->CreateRHIBuffer(
				BufferDesc,
				BufferDesc.Alignment,
				CreateDesc,
				ED3D12ResourceStateMode::MultiState,
				ED3D12Access::CopyDest,
				/*bHasInitialData*/ true
			);

			// Use copy queue for uploading the data
			Context.BatchedSyncPoints.ToWait.Emplace(Buffer->UploadResourceDataViaCopyQueue(Context, &Data));

			// Enqueue transition to SRV
			Context.AddBarrier(
				Buffer->GetResource(),
				ED3D12Access::CopyDest,
				ED3D12Access::SRVCompute,
				0);
		}

		// Also copy geometry parameter data to the GPU buffer
		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && WorkerData[0].MaxUsedHitRecordIndex >= 0)
		{
			check(InInlineBindingDataBuffer);

			FD3D12Buffer* D3D12InlineBindingDataBuffer = FD3D12DynamicRHI::ResourceCast(InInlineBindingDataBuffer, Device->GetGPUIndex());

			uint32 MaxUsedInlineRecordIndex = WorkerData[0].MaxUsedHitRecordIndex / NumShaderSlotsPerGeometrySegment;
			uint32 MaxUsedInlineGeometryParamaterDataSize = (MaxUsedInlineRecordIndex + 1) * sizeof(FD3D12HitGroupSystemParameters);

			FD3D12ResourceLocation UploadResourceLocation(Device);
			void* MappedUploadMemory = Adapter->GetUploadHeapAllocator(Device->GetGPUIndex()).AllocUploadResource(MaxUsedInlineGeometryParamaterDataSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, UploadResourceLocation);
			FMemory::Memcpy(MappedUploadMemory, InlineGeometryParameterData.GetData(), MaxUsedInlineGeometryParamaterDataSize);

			FD3D12Resource* SourceResource = UploadResourceLocation.GetResource();
			FD3D12Resource* DestResource = D3D12InlineBindingDataBuffer->ResourceLocation.GetResource();

			Context.FlushResourceBarriers();
			Context.UpdateResidency(DestResource);
			Context.UpdateResidency(SourceResource);

			Context.CopyBufferRegionChecked(
				DestResource->GetResource(), DestResource->GetName(),
				D3D12InlineBindingDataBuffer->ResourceLocation.GetOffsetFromBaseOfResource(),
				SourceResource->GetResource(), SourceResource->GetName(),
				UploadResourceLocation.GetOffsetFromBaseOfResource(),
				MaxUsedInlineGeometryParamaterDataSize
			);

			#if ENABLE_RESIDENCY_MANAGEMENT
			// assume no references resources when inline SBT only - inline SBT only has references to IB/VB buffers and these are 
			// residency tracked via the TLAS
			check(ShaderBindingMode != ERayTracingShaderBindingMode::Inline || ReferencedResources.IsEmpty());
			#endif // ENABLE_RESIDENCY_MANAGEMENT
		}

		bIsDirty = false;
	}

	D3D12_GPU_VIRTUAL_ADDRESS GetShaderTableAddress() const
	{
		checkf(!bIsDirty, TEXT("Shader table update is pending, therefore GPU address is not available. Use Commit() to upload data and acquire a valid GPU buffer address."));
		return Buffer->ResourceLocation.GetGPUVirtualAddress();
	}

	D3D12_DISPATCH_RAYS_DESC GetDispatchRaysDesc(FD3D12Device* InDevice, const FD3D12ShaderIdentifier& RayGenShaderIdentifier, FD3D12Resource*& OutRayGenUploadResource) const
	{
		check(Device == InDevice);

		// Allocate memory for the ray gen shader identifier storage
		check(ShaderIdentifierSize == sizeof(FD3D12ShaderIdentifier));
		FD3D12ResourceLocation UploadResourceLocation(Device);
		void* RayGenGPUData = Device->GetDefaultFastAllocator().Allocate(RayGenRecordStride, 256, &UploadResourceLocation);
		FMemory::Memcpy(RayGenGPUData, &RayGenShaderIdentifier, ShaderIdentifierSize);
		D3D12_GPU_VIRTUAL_ADDRESS RayGenStartShaderIdentifierAddress = UploadResourceLocation.GetGPUVirtualAddress();
		OutRayGenUploadResource = UploadResourceLocation.GetResource();
		
		D3D12_GPU_VIRTUAL_ADDRESS ShaderTableAddress = GetShaderTableAddress();

		D3D12_DISPATCH_RAYS_DESC Desc = {};

		Desc.RayGenerationShaderRecord.StartAddress = RayGenStartShaderIdentifierAddress; 
		Desc.RayGenerationShaderRecord.SizeInBytes = RayGenRecordStride;

		Desc.MissShaderTable.StartAddress = ShaderTableAddress + MissShaderTableOffset;
		Desc.MissShaderTable.StrideInBytes = LocalRecordStride;
		Desc.MissShaderTable.SizeInBytes = LocalRecordStride * NumMissRecords;


		if (NumCallableRecords)
		{
			Desc.CallableShaderTable.StartAddress = ShaderTableAddress + CallableShaderTableOffset;
			Desc.CallableShaderTable.StrideInBytes = LocalRecordStride;
			Desc.CallableShaderTable.SizeInBytes = NumCallableRecords * LocalRecordStride;
		}

		if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow)
		{
			Desc.HitGroupTable.StartAddress = ShaderTableAddress + HitGroupShaderTableOffset;
			Desc.HitGroupTable.StrideInBytes = LocalRecordStride;
			Desc.HitGroupTable.SizeInBytes = NumHitRecords * LocalRecordStride;
		}
		else
		{
			Desc.HitGroupTable.StartAddress = ShaderTableAddress + HitGroupShaderTableOffset;
			Desc.HitGroupTable.StrideInBytes = 0; // Zero stride effectively disables SBT indexing
			Desc.HitGroupTable.SizeInBytes = D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT; // Minimal table with only one record
		}

		return Desc;
	}

	static constexpr uint32 ShaderIdentifierSize = D3D12_SHADER_IDENTIFIER_SIZE_IN_BYTES;
	uint32 NumHitRecords = 0;
	uint32 NumCallableRecords = 0;
	uint32 NumMissRecords = 0;

	uint32 MissShaderTableOffset = 0;
	uint32 HitGroupShaderTableOffset = 0;
	uint32 CallableShaderTableOffset = 0;

	FD3D12Device* Device  = nullptr;

	ERayTracingShaderBindingTableLifetime Lifetime = ERayTracingShaderBindingTableLifetime::Transient;
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;
	ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
	uint32 NumShaderSlotsPerGeometrySegment = 0;

	// Note: TABLE_BYTE_ALIGNMENT is used instead of RECORD_BYTE_ALIGNMENT to allow arbitrary switching 
	// between multiple RayGen and Miss shaders within the same underlying table.
	static constexpr uint32 RayGenRecordStride = D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT;

	uint32 LocalRecordSizeUnaligned = 0; // size of the shader identifier + local root parameters, not aligned to SHADER_RECORD_BYTE_ALIGNMENT (used for out-of-bounds access checks)
	uint32 LocalRecordStride = 0; // size of shader identifier + local root parameters, aligned to SHADER_RECORD_BYTE_ALIGNMENT (same for hit groups and callable shaders)
	TResourceArray<uint8, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT> Data;

	// Buffer that contains per-hitrecord index and vertex buffer binding data
	TResourceArray<uint8, D3D12_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT> InlineGeometryParameterData;
	
	bool bIsDirty = true;
	TRefCountPtr<FD3D12Buffer> Buffer;
#if DO_CHECK
	bool bWasDefaultMissShaderSet = false;
#endif

	// All reference d3d12 resources from both persistent and dynamic records (rebuild during commit)
	TArray<FD3D12Resource*> ReferencedResources;
	
	// SBTs have their own descriptor heaps
	FD3D12ExplicitDescriptorCache* DescriptorCache = nullptr;

	struct FRecordUpdateUniformBufferListener : public ID3D12UniformBufferUpdateListener
	{
		FD3D12RayTracingShaderBindingTableInternal& ShaderTable;
		FD3D12UniformBuffer* UniformBuffer = nullptr;
		uint32 ShaderTableOffset = 0;
		uint32 RecordIndex = 0;
		uint32 OffsetWithinRootSignature = 0;
#if ENABLE_RESIDENCY_MANAGEMENT
		FD3D12Resource* CurrentResource = nullptr;
#endif // ENABLE_RESIDENCY_MANAGEMENT

		FRecordUpdateUniformBufferListener(FD3D12RayTracingShaderBindingTableInternal& InShaderTable) :	ShaderTable(InShaderTable)
		{
		}

		virtual ~FRecordUpdateUniformBufferListener() = default;

		void Init(uint32 WorkerIndex, FD3D12UniformBuffer* InUniformBuffer, uint32 InShaderTableOffset, uint32 InRecordIndex, uint32 InOffsetWithinRootSignature)
		{
			UniformBuffer = InUniformBuffer;
			ShaderTableOffset = InShaderTableOffset;
			RecordIndex = InRecordIndex;
			OffsetWithinRootSignature = InOffsetWithinRootSignature;
			UniformBuffer->AddUpdateListener(this);

#if ENABLE_RESIDENCY_MANAGEMENT
			CurrentResource = UniformBuffer->ResourceLocation.GetResource();
			if (CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, 1);
			}
#endif // ENABLE_RESIDENCY_MANAGEMENT
		}

		void Reset(uint32 WorkerIndex)
		{
			check(UniformBuffer);
#if ENABLE_RESIDENCY_MANAGEMENT
			if (CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, -1);
				//ShaderTable.AddLastTrackedBaseShaderResource(WorkerIndex, ShaderResource);
			}
			CurrentResource = nullptr;
#endif // ENABLE_RESIDENCY_MANAGEMENT

			UniformBuffer->RemoveUpdateListener(this);
			UniformBuffer = nullptr;
		}

		virtual void RemoveListener(FD3D12UniformBuffer* InUpdatedUniformBuffer) final override
		{
			check(UniformBuffer == InUpdatedUniformBuffer);
			ShaderTable.RemoveUpdateUniformBufferListener(RecordIndex, this);

			// TODO: investigate if current resource be removed from reference resource list as well here?
		}

		virtual void UniformBufferUpdated(const FD3D12ContextArray& Contexts, FD3D12UniformBuffer* InUpdatedUniformBuffer) final override
		{
			check(UniformBuffer == InUpdatedUniformBuffer);
			ShaderTable.SetLocalShaderParameters(ShaderTableOffset, RecordIndex, OffsetWithinRootSignature, InUpdatedUniformBuffer->ResourceLocation.GetGPUVirtualAddress());

#if ENABLE_RESIDENCY_MANAGEMENT
			FD3D12Resource* NewResource = UniformBuffer->ResourceLocation.GetResource();
	
			// only care if actual d3d12 resource has changed
			if (NewResource != CurrentResource)
			{
				// Resource could be streamed in or out and then contain empty d3d12 backing resource
				// Using 0 as consolidated worker index data which gets merged during Commit of the shader binding table
				if (CurrentResource)
				{
					ShaderTable.UpdatePersistentReferencedResource(0, CurrentResource, -1);
				}
				if (NewResource)
				{
					ShaderTable.UpdatePersistentReferencedResource(0, NewResource, 1);
				}
				CurrentResource = NewResource;
			}
#endif // ENABLE_RESIDENCY_MANAGEMENT

			INC_DWORD_STAT(STAT_D3D12RayTracingUBRecordUpdate);
		}
	};

	struct FRecordUpdateRayTracingGeometryListener : public ID3D12RayTracingGeometryUpdateListener
	{
		FD3D12RayTracingShaderBindingTableInternal* ShaderTable = nullptr;
		const FD3D12RayTracingGeometry* Geometry = nullptr;
		uint32 RecordIndex = 0;
		uint32 UserData = 0;
		uint32 SegmentIndex = 0;

		virtual ~FRecordUpdateRayTracingGeometryListener() = default;

		void Init(FD3D12RayTracingShaderBindingTableInternal* InShaderTable, const FD3D12RayTracingGeometry* InGeometry, uint32 InRecordIndex, uint32 InUserData, uint32 InSegmentIndex)
		{
			check(Geometry == nullptr);

			ShaderTable = InShaderTable;
			Geometry = InGeometry;
			RecordIndex = InRecordIndex;
			UserData = InUserData;
			SegmentIndex = InSegmentIndex;

			Geometry->AddUpdateListener(this);
		}

		void Reset()
		{
			ShaderTable = nullptr;
			Geometry = nullptr;
			RecordIndex = 0;
			UserData = 0;
			SegmentIndex = 0;
		}

		virtual void RemoveListener(FD3D12RayTracingGeometry* InGeometry) final override
		{
			check(Geometry == InGeometry);
			ShaderTable->RemoveUpdateRayTracingGeometryListener(RecordIndex, this);
		}

		virtual void HitGroupParametersUpdated(FD3D12RayTracingGeometry* InGeometry) final override
		{
			check(InGeometry == Geometry);

			uint32 GPUIndex = ShaderTable->Device->GetGPUIndex();
			FD3D12HitGroupSystemParameters SystemParameters = Geometry->PerGPUData[GPUIndex].HitGroupSystemParameters[SegmentIndex];
			SystemParameters.RootConstants.UserData = UserData;

			uint32 InlineRecordIndex = ShaderTable->GetInlineRecordIndex(RecordIndex);

			if (EnumHasAnyFlags(ShaderTable->ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
			{
				ShaderTable->SetHitGroupSystemParameters(RecordIndex, SystemParameters);
			}
			if (EnumHasAnyFlags(ShaderTable->ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && InlineRecordIndex != INDEX_NONE)
			{
				ShaderTable->SetInlineGeometryParameters(InlineRecordIndex, SystemParameters);
			}
		}
	};

#if ENABLE_RESIDENCY_MANAGEMENT
	struct FRecordUpdateShaderResourceRenameListener : public FD3D12ShaderResourceRenameListener
	{
		FD3D12RayTracingShaderBindingTableInternal& ShaderTable;
		uint32 RecordIndex = 0;
		TRefCountPtr<FD3D12BaseShaderResource> ShaderResource;
		FD3D12Resource* CurrentResource = nullptr;

		FRecordUpdateShaderResourceRenameListener(FD3D12RayTracingShaderBindingTableInternal& InShaderTable) :	ShaderTable(InShaderTable)
		{
		}
		virtual ~FRecordUpdateShaderResourceRenameListener() = default;

		void Init(uint32 WorkerIndex, uint32 InRecordIndex, FD3D12BaseShaderResource* InShaderResource)
		{
			RecordIndex = InRecordIndex;
			ShaderResource = InShaderResource;
			CurrentResource = InShaderResource->ResourceLocation.GetResource();

			ShaderResource->AddRenameListener(this);
			if (CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, 1);
			}
		}

		void Reset(uint32 WorkerIndex)
		{
			check(ShaderResource);
			if (CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, -1);
				ShaderTable.AddLastTrackedBaseShaderResource(WorkerIndex, ShaderResource);
			}
			ShaderResource->RemoveRenameListener(this);
			ShaderResource = nullptr;
			RecordIndex = 0;
			CurrentResource = 0;
		}

		virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override
		{
			check(InRenamedResource == ShaderResource);
			FD3D12Resource* NewResource = InNewResourceLocation->GetResource();
			
			// only care if actual d3d12 resource has changed
			if (NewResource != CurrentResource)
			{
				// Resource could be streamed in or out and then contain empty d3d12 backing resource
				if (CurrentResource)
				{
					ShaderTable.UpdatePersistentReferencedResource(0, CurrentResource, -1);
				}
				if (NewResource)
				{
					ShaderTable.UpdatePersistentReferencedResource(0, NewResource, 1);
				}
				CurrentResource = NewResource;
			}
		}
	};
	
	struct FRecordUpdateTextureReferenceReplaceListener : public FD3D12TextureReferenceReplaceListener, public FD3D12ShaderResourceRenameListener
	{
		FD3D12RayTracingShaderBindingTableInternal& ShaderTable;
		uint32 RecordIndex = 0;
		TRefCountPtr<FD3D12RHITextureReference> TextureReference;
		FD3D12Resource* CurrentResource = nullptr;

		FRecordUpdateTextureReferenceReplaceListener(FD3D12RayTracingShaderBindingTableInternal& InShaderTable) :	ShaderTable(InShaderTable)
		{
		}
		virtual ~FRecordUpdateTextureReferenceReplaceListener() = default;

		void Init(uint32 WorkerIndex, uint32 InRecordIndex, FD3D12RHITextureReference* InTextureReference)
		{
			RecordIndex = InRecordIndex;
			TextureReference = InTextureReference;
			TextureReference->AddReplaceListener(this);			
			
			// Handle rename events on the references texture as well
			FD3D12Texture* ReferencedTexture = FD3D12DynamicRHI::ResourceCast(TextureReference->GetReferencedTexture());
			ReferencedTexture->AddRenameListener(this);

			CurrentResource = ReferencedTexture->ResourceLocation.GetResource();
			check(CurrentResource);

			ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, 1);
		}

		void Reset(uint32 WorkerIndex)
		{
			check(TextureReference && CurrentResource);
			ShaderTable.UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, -1);
			
			FD3D12Texture* ReferencedTexture = FD3D12DynamicRHI::ResourceCast(TextureReference->GetReferencedTexture());
			ReferencedTexture->RemoveRenameListener(this);
			TextureReference->RemoveReplaceListener(this);

			ShaderTable.AddLastTrackedBaseShaderResource(WorkerIndex, ReferencedTexture);

			TextureReference = nullptr;
			RecordIndex = 0;
			CurrentResource = 0;
		}

		virtual void TextureReplaced(FD3D12ContextArray const& Contexts, FD3D12RHITextureReference* InTextureReference, FD3D12Texture* CurrentTexture, FD3D12Texture* NewTexture) override
		{
			check(TextureReference == InTextureReference);
			FD3D12Resource* NewResource = NewTexture->ResourceLocation.GetResource();
			
			// only care if actual d3d12 resource has changed
			if (NewResource != CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(0, CurrentResource, -1);
				ShaderTable.UpdatePersistentReferencedResource(0, NewResource, 1);
				CurrentResource = NewResource;
			}
			
			// Also update the rename listener
			CurrentTexture->RemoveRenameListener(this);
			NewTexture->AddRenameListener(this);
		}

		virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override
		{
			FD3D12Texture* ReferencedTexture = FD3D12DynamicRHI::ResourceCast(TextureReference->GetReferencedTexture());
			check(InRenamedResource == ReferencedTexture);
			FD3D12Resource* NewResource = InNewResourceLocation->GetResource();
			
			// only care if actual d3d12 resource has changed
			if (NewResource != CurrentResource)
			{
				ShaderTable.UpdatePersistentReferencedResource(0, CurrentResource, -1);
				ShaderTable.UpdatePersistentReferencedResource(0, NewResource, 1);
				CurrentResource = NewResource;
			}
		}
	};
#endif // ENABLE_RESIDENCY_MANAGEMENT

	// Tracked data for each record so stored GPU data can be updated after defrag operation
	struct FRecordData
	{
		enum class EState
		{
			Uninitialized,
			Persistent,
			Transient,
		};

		void Reset()
		{
#if ENABLE_RESIDENCY_MANAGEMENT
			ReferencedShaderResourceListeners.Empty();
			ReferencedTextureReferenceListeners.Empty();
#endif // ENABLE_RESIDENCY_MANAGEMENT
			UpdateGeometryListener.Reset();
			UpdateUniformBufferListeners.Empty();
#if DO_CHECK
			LooseParameterData.Empty();
#endif //DO_CHECK

			State = EState::Uninitialized;
		}

		EState State = EState::Uninitialized;
		TArray<FRecordUpdateUniformBufferListener*> UpdateUniformBufferListeners;
		FRecordUpdateRayTracingGeometryListener UpdateGeometryListener;
		FD3D12ResourceLocation* LooseParameterResourceLocation = nullptr;

#if ENABLE_RESIDENCY_MANAGEMENT				
		TArray<FRecordUpdateShaderResourceRenameListener*> ReferencedShaderResourceListeners;
		TArray<FRecordUpdateTextureReferenceReplaceListener*> ReferencedTextureReferenceListeners;
#endif // ENABLE_RESIDENCY_MANAGEMENT		
#if DO_CHECK
		TArray<uint8> LooseParameterData;
#endif //DO_CHECK
	};
	TArray<FRecordData> HitRecordData;

	uint32 GetInlineRecordIndex(uint32 RecordIndex) const
	{	
		// Only care about shader slot 0 for inline geometry parameters -> remap the record index
		return (RecordIndex % NumShaderSlotsPerGeometrySegment == 0) ? RecordIndex / NumShaderSlotsPerGeometrySegment : INDEX_NONE;
	}

	void SetHitGroupGeometrySystemParameters(uint32 WorkerIndex, ERayTracingLocalShaderBindingType BindingType, uint32 RecordIndex, const FD3D12RayTracingGeometry* Geometry, uint32 GeometrySegmentIndex, uint32 UserData)
	{	
		FD3D12HitGroupSystemParameters SystemParameters;

		if (Geometry != nullptr)
		{
			SystemParameters = Geometry->PerGPUData[Device->GetGPUIndex()].HitGroupSystemParameters[GeometrySegmentIndex];
		}
		else
		{
			FMemory::Memset(&SystemParameters, 0, sizeof(SystemParameters));
		}

		SystemParameters.RootConstants.UserData = UserData;		

		WorkerData[WorkerIndex].MaxUsedHitRecordIndex = FMath::Max(WorkerData[WorkerIndex].MaxUsedHitRecordIndex, (int32)RecordIndex);

		uint32 InlineRecordIndex = GetInlineRecordIndex(RecordIndex);

		if (Lifetime == ERayTracingShaderBindingTableLifetime::Transient)
		{
			if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
			{
				SetHitGroupSystemParameters(RecordIndex, SystemParameters);
			}
			if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && InlineRecordIndex != INDEX_NONE)
			{
				SetInlineGeometryParameters(InlineRecordIndex, SystemParameters);
			}
		}
		else
		{
			FRecordData& RecordData = HitRecordData[RecordIndex];
			bool bSetData = BindingType != ERayTracingLocalShaderBindingType::Validation;

			// Special use case when trying to set transient binding on already persistent stored binding (validate content only then)
			// Can happen we go from persistent -> transient -> persistent when RTPSO is still compiling for example
			if (RecordData.State == FRecordData::EState::Persistent && BindingType == ERayTracingLocalShaderBindingType::Transient)
			{
				bSetData = false;
			}

			if (bSetData)
			{
				check(RecordData.State != FRecordData::EState::Persistent || BindingType == ERayTracingLocalShaderBindingType::Transient);

				if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
				{
					SetHitGroupSystemParameters(RecordIndex, SystemParameters);
				}
				if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && InlineRecordIndex != INDEX_NONE)
				{
					SetInlineGeometryParameters(InlineRecordIndex, SystemParameters);
				}

				// If persistently stored then also listen to hit record updates for stream in/out events of the ray tracing geometry data
				if (BindingType == ERayTracingLocalShaderBindingType::Persistent)
				{
					if (Geometry != nullptr)
					{
						HitRecordData[RecordIndex].UpdateGeometryListener.Init(this, Geometry, RecordIndex, UserData, GeometrySegmentIndex);
					}
					RecordData.State = FRecordData::EState::Persistent;
				}				
				else // if (RecordData.State == FRecordData::EState::Uninitialized)
				{
					RecordData.State = FRecordData::EState::Transient;
				}
			}
			else
			{
				check(RecordData.State == FRecordData::EState::Persistent);
				if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
				{
					CompareHitGroupSystemParameters(RecordIndex, SystemParameters);
				}
				if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline) && InlineRecordIndex != INDEX_NONE)
				{
					CompareInlineGeometryParameters(InlineRecordIndex, SystemParameters);
				}
			}
		}
	}
		
#if ENABLE_RESIDENCY_MANAGEMENT
	void AddLastTrackedBaseShaderResource(uint32 WorkerIndex, FD3D12BaseShaderResource* BaseShaderResource)
	{
		WorkerData[WorkerIndex].LastTrackedBaseShaderResources.Add(BaseShaderResource);
	}

	void UpdatePersistentReferencedResource(uint32 WorkerIndex, FD3D12Resource* Resource, int32 Count)
	{
		check(Resource);

		// Update the worker thread reference count of the resource
		int32& ReferenceCount = WorkerData[WorkerIndex].PersistentReferencedResources.FindOrAdd(Resource);
		ReferenceCount += Count;
		
		if (WorkerIndex == 0)
		{
			check(ReferenceCount >= 0);
			if (ReferenceCount == 0)
			{
				WorkerData[WorkerIndex].PersistentReferencedResources.Remove(Resource);
			}
		}
	}

	void AddDynamicReferencedResource(uint32 WorkerIndex, FD3D12Resource* Resource)
	{
		bool bIsAlreadyInSet = false;
		WorkerData[WorkerIndex].DynamicReferencedResourceSet.Add(Resource, &bIsAlreadyInSet);
		if (!bIsAlreadyInSet)
		{
			WorkerData[WorkerIndex].DynamicReferencedResources.Add(Resource);
		}
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT

	void AddReferencedShaderResource(uint32 WorkerIndex, uint32 RecordIndex, ERayTracingLocalShaderBindingType BindingType, FD3D12BaseShaderResource* ShaderResource)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (BindingType == ERayTracingLocalShaderBindingType::Transient || Lifetime != ERayTracingShaderBindingTableLifetime::Persistent)
		{
			AddDynamicReferencedResource(WorkerIndex, ShaderResource->GetResource());
		}
		else if (BindingType == ERayTracingLocalShaderBindingType::Persistent)
		{
			// Same base shader resource can be bound multiple times to the same record - only track once
			FRecordData& RecordData = HitRecordData[RecordIndex];
			for (FRecordUpdateShaderResourceRenameListener* Listener : RecordData.ReferencedShaderResourceListeners)
			{
				if (Listener->ShaderResource == ShaderResource)
				{
					return;
				}
			}

			// Update the tracked record data so it can be updated in case of defrag operation
			FRecordUpdateShaderResourceRenameListener* Listener = new FRecordUpdateShaderResourceRenameListener(*this);
			Listener->Init(WorkerIndex, RecordIndex, ShaderResource);
			RecordData.ReferencedShaderResourceListeners.Add(Listener);
		}
#endif // ENABLE_RESIDENCY_MANAGEMENT
	}

	void AddReferencedTexture(uint32 WorkerIndex, uint32 RecordIndex, ERayTracingLocalShaderBindingType BindingType, FRHITexture* RHITexture)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (BindingType == ERayTracingLocalShaderBindingType::Transient || Lifetime != ERayTracingShaderBindingTableLifetime::Persistent)
		{
			FD3D12Texture* Texture = FD3D12CommandContext::RetrieveTexture(RHITexture, Device->GetGPUIndex());			
			AddDynamicReferencedResource(WorkerIndex, Texture->ResourceLocation.GetResource());
		}
		else if (BindingType == ERayTracingLocalShaderBindingType::Persistent)
		{			
			// Collect all texture references independent of the shader resources because the internal shader resource can be changed on swap
			if (FRHITextureReference* TextureReferenceRHI = RHITexture->GetTextureReference())
			{
				FD3D12RHITextureReference* TextureReference = FD3D12CommandContext::RetrieveObject<FD3D12RHITextureReference>(TextureReferenceRHI, Device->GetGPUIndex());
								
				// Same texture reference can be bound multiple times to the same record - only track once
				FRecordData& RecordData = HitRecordData[RecordIndex];
				for (FRecordUpdateTextureReferenceReplaceListener* Listener : RecordData.ReferencedTextureReferenceListeners)
				{
					if (Listener->TextureReference == TextureReference)
					{
						return;
					}
				}

				// Update the tracked record data so it can be updated in case of texture reference swap
				FRecordUpdateTextureReferenceReplaceListener* Listener = new FRecordUpdateTextureReferenceReplaceListener(*this);
				Listener->Init(WorkerIndex, RecordIndex, TextureReference);
				RecordData.ReferencedTextureReferenceListeners.Add(Listener);
			}
			else
			{
				// Fall back to regular base shader resource code path
				FD3D12Texture* Texture = FD3D12CommandContext::RetrieveTexture(RHITexture, Device->GetGPUIndex());
				AddReferencedShaderResource(WorkerIndex, RecordIndex, BindingType, Texture);
			}
		}
#endif // ENABLE_RESIDENCY_MANAGEMENT
	}

	void AddReferencedUniformBuffer(uint32 WorkerIndex, uint32 RecordIndex, uint32 ShaderTableOffset, uint32 InOffsetWithinRootSignature, ERayTracingLocalShaderBindingType BindingType, FD3D12UniformBuffer* UniformBuffer)
	{
		if (Lifetime == ERayTracingShaderBindingTableLifetime::Persistent && BindingType == ERayTracingLocalShaderBindingType::Persistent)
		{
			FRecordData& RecordData = HitRecordData[RecordIndex];

			for (FRecordUpdateUniformBufferListener* Listener : RecordData.UpdateUniformBufferListeners)
			{
				check(Listener->UniformBuffer != UniformBuffer);
			}
			
			FRecordUpdateUniformBufferListener* Listener = new FRecordUpdateUniformBufferListener(*this);
			Listener->Init(WorkerIndex, UniformBuffer, ShaderTableOffset, RecordIndex, InOffsetWithinRootSignature);
			RecordData.UpdateUniformBufferListeners.Add(Listener);
		}
	}

	FD3D12ConstantBufferView* SetLooseParameterData(
		uint32 WorkerIndex, 
		uint32 RecordIndex, 
		ERayTracingLocalShaderBindingType BindingType, 
		const void* LooseParameterData, 
		uint32 LooseParameterDataSize, 
		D3D12_GPU_VIRTUAL_ADDRESS& OutGPUVirtualAddress)
	{
		// If we see a significant number of LooseParameter allocations coming through this path, we should consider
		// caching constant buffer blocks inside ShaderTable and linearly sub-allocate from them.
		// If the amount of data is relatively small, it may also be possible to use root constants and avoid extra allocations entirely.

		FD3D12ConstantBufferView* ConstantBufferView = nullptr;

		if (BindingType == ERayTracingLocalShaderBindingType::Validation)
		{
#if DO_CHECK
			FRecordData& RecordData = HitRecordData[RecordIndex];
			check(RecordData.LooseParameterData.Num() == LooseParameterDataSize);
			check(FMemory::Memcmp(RecordData.LooseParameterData.GetData(), LooseParameterData, LooseParameterDataSize) == 0);
			check(RecordData.LooseParameterResourceLocation);
			OutGPUVirtualAddress = RecordData.LooseParameterResourceLocation->GetGPUVirtualAddress();
#endif // DO_CHECK
		}
		else
		{
			// Always allocate using the upload heap allocator and not the TransientUniformBufferAllocator (can cause GPU crashes with transient data somehow)
			FD3D12ResourceLocation ResourceLocation(Device);
			FD3D12UploadHeapAllocator& Allocator = Device->GetParentAdapter()->GetUploadHeapAllocator(Device->GetGPUIndex());
			void* MappedData = Allocator.AllocUploadResource(LooseParameterDataSize, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT, ResourceLocation);
			FMemory::Memcpy(MappedData, LooseParameterData, LooseParameterDataSize);
			OutGPUVirtualAddress = ResourceLocation.GetGPUVirtualAddress();

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			// Not implemented for persistent SBT yet (CBV needs to be managed per record then instead of at SBT level)
			// 
			// CBV also has pointer to stack local ResourceLocation which it shouldn't touch anymore after creation but
			// will still have dangling pointer - might be better to always allocate ResourceLocation as well on heap next to CBV (like persistent path)
			check(Lifetime == ERayTracingShaderBindingTableLifetime::Transient);
			ConstantBufferView = new FD3D12ConstantBufferView(Device, nullptr);
			ConstantBufferView->CreateView(&ResourceLocation, 0, Align(LooseParameterDataSize, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT));
			WorkerData[WorkerIndex].LooseParameterCBVs.Add(ConstantBufferView);			
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS

			if (Lifetime == ERayTracingShaderBindingTableLifetime::Persistent)
			{
				// Move ownership of the upload memory to the record data when persistent (otherwise auto freed next 'frame')
				FRecordData& RecordData = HitRecordData[RecordIndex];
				if (BindingType == ERayTracingLocalShaderBindingType::Persistent)
				{
					check(RecordData.LooseParameterResourceLocation == nullptr);
					RecordData.LooseParameterResourceLocation = new FD3D12ResourceLocation(Device);
					FD3D12ResourceLocation::TransferOwnership(*RecordData.LooseParameterResourceLocation, ResourceLocation);

				#if ENABLE_RESIDENCY_MANAGEMENT
					UpdatePersistentReferencedResource(WorkerIndex, RecordData.LooseParameterResourceLocation->GetResource(), 1);
				#endif // ENABLE_RESIDENCY_MANAGEMENT
				
				#if DO_CHECK
					RecordData.LooseParameterData.SetNum(LooseParameterDataSize);
					FMemory::Memcpy(RecordData.LooseParameterData.GetData(), LooseParameterData, LooseParameterDataSize);
				#endif // DO_CHECK
				}
				// If we had loose parameter resource location allocated then it means it was persistent and now temporarily transient
				// Update the cached loose parameter GPU allocation to make sure the SBT doesn't contain any dangling upload memory allocations
				else if (RecordData.LooseParameterResourceLocation)
				{
				#if DO_CHECK
					check(RecordData.LooseParameterData.Num() == LooseParameterDataSize);
					check(FMemory::Memcmp(RecordData.LooseParameterData.GetData(), LooseParameterData, LooseParameterDataSize) == 0);
				#endif // DO_CHECK

					FD3D12Resource* CurrentResource = RecordData.LooseParameterResourceLocation->GetResource();

					RecordData.LooseParameterResourceLocation->Clear();
					FD3D12ResourceLocation::TransferOwnership(*RecordData.LooseParameterResourceLocation, ResourceLocation);

				#if ENABLE_RESIDENCY_MANAGEMENT
					FD3D12Resource* NewResource = RecordData.LooseParameterResourceLocation->GetResource();
					if (CurrentResource != NewResource)
					{
						UpdatePersistentReferencedResource(WorkerIndex, CurrentResource, -1);
						UpdatePersistentReferencedResource(WorkerIndex, NewResource, 1);
					}
				#endif // ENABLE_RESIDENCY_MANAGEMENT
				}
			}
#if ENABLE_RESIDENCY_MANAGEMENT
			else 
			{			
				AddDynamicReferencedResource(WorkerIndex, ResourceLocation.GetResource());
			}
#endif // ENABLE_RESIDENCY_MANAGEMENT
		}

		return ConstantBufferView;
	}

	void ClearHitRecordData(uint32 WorkerIndex, uint32 RecordIndex)
	{
		if (Lifetime != ERayTracingShaderBindingTableLifetime::Persistent)
		{
			return;
		}

		FRecordData& RecordData = HitRecordData[RecordIndex];
		
		for (FRecordUpdateUniformBufferListener* Listener : RecordData.UpdateUniformBufferListeners)
		{
			Listener->Reset(WorkerIndex);
			delete Listener;
		}

		if (RecordData.LooseParameterResourceLocation)
		{
#if ENABLE_RESIDENCY_MANAGEMENT
			UpdatePersistentReferencedResource(WorkerIndex, RecordData.LooseParameterResourceLocation->GetResource(), -1);
#endif // ENABLE_RESIDENCY_MANAGEMENT
			delete RecordData.LooseParameterResourceLocation;
			RecordData.LooseParameterResourceLocation = nullptr;
		}

		if (RecordData.UpdateGeometryListener.Geometry)
		{
			RecordData.UpdateGeometryListener.Geometry->RemoveUpdateListener(&RecordData.UpdateGeometryListener);
		}
		
#if ENABLE_RESIDENCY_MANAGEMENT
		for (FRecordUpdateShaderResourceRenameListener* Listener : RecordData.ReferencedShaderResourceListeners)
		{
			Listener->Reset(WorkerIndex);
			delete Listener;
		}

		for (FRecordUpdateTextureReferenceReplaceListener* Listener : RecordData.ReferencedTextureReferenceListeners)
		{
			Listener->Reset(WorkerIndex);
			delete Listener;
		}
#endif // ENABLE_RESIDENCY_MANAGEMENT

		// Reset record to default state
		RecordData.Reset();
	}

	void RemoveUpdateUniformBufferListener(uint32 RecordIndex, FRecordUpdateUniformBufferListener* Listener)
	{
		bool bFound = false;
		for (int32 Index = 0; Index < HitRecordData[RecordIndex].UpdateUniformBufferListeners.Num(); ++Index)
		{
			if (HitRecordData[RecordIndex].UpdateUniformBufferListeners[Index] == Listener)
			{
				HitRecordData[RecordIndex].UpdateUniformBufferListeners.RemoveAt(Index);
				
				// Index is 0 since the work has been consolidated
				Listener->Reset(0);
				delete Listener;

				bFound = true;
				break;
			}
		}
		check(bFound);
	}

	void RemoveUpdateRayTracingGeometryListener(uint32 RecordIndex, FRecordUpdateRayTracingGeometryListener* Listener)
	{
		FRecordData& RecordData = HitRecordData[RecordIndex];
		check(&RecordData.UpdateGeometryListener == Listener && RecordData.UpdateGeometryListener.Geometry != nullptr);
		RecordData.UpdateGeometryListener.Reset();
	}

	struct FShaderRecordCacheKey
	{
		static constexpr uint32 MaxUniformBuffers = 6;
		FRHIUniformBuffer* const* UniformBuffers[MaxUniformBuffers];
		uint64 Hash = 0;
		uint32 NumUniformBuffers = 0;
		uint32 ShaderIndex = 0;

		FShaderRecordCacheKey() = default;
		FShaderRecordCacheKey(uint32 InNumUniformBuffers, FRHIUniformBuffer* const* InUniformBuffers, uint32 InShaderIndex)
		{
			ShaderIndex = InShaderIndex;

			check(InNumUniformBuffers <= MaxUniformBuffers);
			NumUniformBuffers = FMath::Min(MaxUniformBuffers, InNumUniformBuffers);

			const uint64 DataSizeInBytes = sizeof(FRHIUniformBuffer*) * NumUniformBuffers;
			FMemory::Memcpy(UniformBuffers, InUniformBuffers, DataSizeInBytes);
			Hash = FShaderHash::HashBuffer(UniformBuffers, DataSizeInBytes).Hash;
		}

		bool operator == (const FShaderRecordCacheKey& Other) const
		{
			if (Hash != Other.Hash) return false;
			if (ShaderIndex != Other.ShaderIndex) return false;
			if (NumUniformBuffers != Other.NumUniformBuffers) return false;

			for (uint32 BufferIndex = 0; BufferIndex < NumUniformBuffers; ++BufferIndex)
			{
				if (UniformBuffers[BufferIndex] != Other.UniformBuffers[BufferIndex]) return false;
			}

			return true;
		}

		friend uint32 GetTypeHash(const FShaderRecordCacheKey& Key)
		{
			return uint32(Key.Hash);
		}
	};
	
	void UpdateResidency(FD3D12CommandContext& CommandContext) const
	{
		// Skip redundant resource residency updates when a shader table is repeatedly used on the same command list
		bool bWasAlreadyInSet = false;
		CommandContext.RayTracingShaderTables.FindOrAdd(UniqueId, bWasAlreadyInSet);
		if (bWasAlreadyInSet)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(ShaderTableUpdateResidency);

		for (FD3D12Resource* Resource : ReferencedResources)
		{
			CommandContext.UpdateResidency(Resource);
		}

		CommandContext.UpdateResidency(Buffer->GetResource());
	}

	struct alignas(PLATFORM_CACHE_LINE_SIZE) FWorkerThreadData
	{
		Experimental::TSherwoodMap<FShaderRecordCacheKey, uint32> ShaderRecordCache;

		int32 MaxUsedHitRecordIndex = -1;

#if ENABLE_RESIDENCY_MANAGEMENT
		// Track all persistent resources with number of refs accross all records (when ref reaches 0 then it doesn't need be tracked anymore)
		TMap<TRefCountPtr<FD3D12Resource>, int32> PersistentReferencedResources;

		// Track last reference to base shader resources because PersistentReferencedResources could have the last reference to the d3d12 resource
		// and we can't delete d3d12baseshaderresource with committed resource when it doesn't hold the last reference to it
		TSet<TRefCountPtr<FD3D12BaseShaderResource>> LastTrackedBaseShaderResources;

		// A set of all resources referenced by this shader table for the purpose of updating residency before ray tracing work dispatch.
		Experimental::TSherwoodSet<void*> DynamicReferencedResourceSet;
		TArray<TRefCountPtr<FD3D12Resource>> DynamicReferencedResources;
#endif // ENABLE_RESIDENCY_MANAGEMENT

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		TArray<FD3D12ConstantBufferView*> LooseParameterCBVs;
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	};

	FWorkerThreadData WorkerData[MaxBindingWorkers];

	const uint64 UniqueId;
	UE::FMutex DispatchMutex;

private:
	static std::atomic_uint64_t NextUniqueId;
};

std::atomic_uint64_t FD3D12RayTracingShaderBindingTableInternal::NextUniqueId = 0;

static void CreateSpecializedStateObjects(
	ID3D12Device5* RayTracingDevice,
	ID3D12RootSignature* GlobalRootSignature,
	uint32 MaxAttributeSizeInBytes,
	uint32 MaxPayloadSizeInBytes,
	const FD3D12RayTracingShaderLibrary& RayGenShaders,
	const TArray<FD3D12RayTracingPipelineCache::FEntry*>& UniqueShaderCollections,
	const TMap<FShaderHash, int32>& RayGenShaderIndexByHash,
	TArray<TRefCountPtr<ID3D12StateObject>>& OutSpecializedStateObjects,
	TArray<int32>& OutSpecializationIndices)
{
	static constexpr uint32 MaxSpecializationBuckets = FD3D12RayTracingPipelineInfo::MaxPerformanceGroups;

	if (RayGenShaders.Shaders.Num() <= 1)
	{
		// No specializations needed
		return;
	}

	// Initialize raygen shader PSO specialization map to default values
	OutSpecializationIndices.Reserve(RayGenShaders.Shaders.Num());
	for (int32 It = 0; It < RayGenShaders.Shaders.Num(); ++It)
	{
		OutSpecializationIndices.Add(INDEX_NONE);
	}

	struct FRayGenShaderSpecialization
	{
		D3D12_EXISTING_COLLECTION_DESC Desc = {};
		int32 ShaderIndex = INDEX_NONE;
	};
	TArray<FRayGenShaderSpecialization> RayGenShaderCollectionBuckets[MaxSpecializationBuckets];
	TArray<D3D12_EXISTING_COLLECTION_DESC> ShaderCollectionDescs;

	// Find useful performance group range for non-raygen shaders.
	// It is not necessary to create PSO specializations for high-occupancy RGS if overall PSO will be limited by low-occupancy hit shaders.
	// Also not necessary to create specializations if all raygen shaders are already in the same group.
	uint32 MaxPerformanceGroupRGS = 0;
	uint32 MinPerformanceGroupRGS = MaxSpecializationBuckets - 1;
	uint32 MaxPerformanceGroupOther = 0;
	uint32 MinPerformanceGroupOther = MaxSpecializationBuckets - 1;
	int32 LastRayGenShaderCollectionIndex = INDEX_NONE;

	for (int32 EntryIndex = 0; EntryIndex < UniqueShaderCollections.Num(); ++EntryIndex)
	{
		FD3D12RayTracingPipelineCache::FEntry* Entry = UniqueShaderCollections[EntryIndex];

		const uint32 Group = FMath::Min<uint32>(Entry->PipelineInfo.PerformanceGroup, MaxSpecializationBuckets);

		if (Entry->CollectionType == FD3D12RayTracingPipelineCache::ECollectionType::RayGen)
		{
			MaxPerformanceGroupRGS = FMath::Max<uint32>(MaxPerformanceGroupRGS, Group);
			MinPerformanceGroupRGS = FMath::Min<uint32>(MinPerformanceGroupRGS, Group);
			LastRayGenShaderCollectionIndex = EntryIndex;
		}
		else
		{
			checkf(EntryIndex > LastRayGenShaderCollectionIndex, TEXT("Ray generation shaders are expected to be first in the UniqueShaderCollections list."));

			MaxPerformanceGroupOther = FMath::Max<uint32>(MaxPerformanceGroupOther, Group);
			MinPerformanceGroupOther = FMath::Min<uint32>(MinPerformanceGroupOther, Group);

			// This is a hit/miss/callable shader which will be common for all specialized RTPSOs.
			ShaderCollectionDescs.Add(Entry->GetCollectionDesc());
		}
	}

	if (MinPerformanceGroupRGS == MaxPerformanceGroupRGS)
	{
		// No need to create a specialized PSO if all raygen shaders are already in the same group
		return;
	}

	// Split RGS collections into a separate lists, organized by performance group
	for (int32 EntryIndex = 0; EntryIndex <= LastRayGenShaderCollectionIndex; ++EntryIndex)
	{
		FD3D12RayTracingPipelineCache::FEntry* Entry = UniqueShaderCollections[EntryIndex];

		check(Entry->CollectionType == FD3D12RayTracingPipelineCache::ECollectionType::RayGen);

		// Don't create specializations for raygen shaders that have better occupancy than worst non-raygen shader
		const uint32 SpecializationBucket = FMath::Min<uint32>(Entry->PipelineInfo.PerformanceGroup, MinPerformanceGroupOther);

		// Don't create extra specialized pipelines for group 0 (worst-performing) and just use the default RTPSO.
		if (SpecializationBucket > 0)
		{
			FRayGenShaderSpecialization Specialization;
			Specialization.Desc = Entry->GetCollectionDesc();
			Specialization.ShaderIndex = RayGenShaderIndexByHash.FindChecked(Entry->Shader->GetHash());
			RayGenShaderCollectionBuckets[SpecializationBucket].Add(Specialization);
		}
	}

	OutSpecializedStateObjects.Reserve(MaxSpecializationBuckets);

	const uint32 ShaderCollectionDescsSize = ShaderCollectionDescs.Num();

	for (const TArray<FRayGenShaderSpecialization>& SpecializationBucket : RayGenShaderCollectionBuckets)
	{
		if (SpecializationBucket.IsEmpty())
		{
			continue;
		}

		const int32 SpecializationIndex = OutSpecializedStateObjects.Num();

		for (const FRayGenShaderSpecialization& Specialization : SpecializationBucket)
		{
			// Temporarily add the RGSs to complete shader collection
			ShaderCollectionDescs.Add(Specialization.Desc);
			OutSpecializationIndices[Specialization.ShaderIndex] = SpecializationIndex;
		}

		TRefCountPtr<ID3D12StateObject> SpecializedPSO = CreateRayTracingStateObject(
			RayTracingDevice,
			{}, // Libraries,
			{}, // LibraryExports,
			MaxAttributeSizeInBytes,
			MaxPayloadSizeInBytes,
			{}, // HitGroups
			GlobalRootSignature,
			{}, // LocalRootSignatures
			{}, // LocalRootSignatureAssociations,
			ShaderCollectionDescs,
			D3D12_STATE_OBJECT_TYPE_RAYTRACING_PIPELINE);

		OutSpecializedStateObjects.Add(SpecializedPSO);

		// Remove the temporary RGSs
		ShaderCollectionDescs.SetNum(ShaderCollectionDescsSize);
	}
}

FD3D12RayTracingPipelineState::FD3D12RayTracingPipelineState(FD3D12Device* Device, const FRayTracingPipelineStateInitializer& Initializer) :
	FRHIRayTracingPipelineState(Initializer),
	Device(Device)
{
	SCOPE_CYCLE_COUNTER(STAT_RTPSO_CreatePipeline);
	TRACE_CPUPROFILER_EVENT_SCOPE(RTPSO_CreatePipeline);

	checkf(Initializer.GetRayGenTable().Num() > 0 || Initializer.bPartial, TEXT("Ray tracing pipelines must have at leat one ray generation shader."));
	checkf(Initializer.bPartial || Initializer.GetHitGroupTable().Num() > 0, TEXT("Ray tracing pipelines must have at leat one hit shader."));

	uint64 TotalCreationTime = 0;
	uint64 CompileTime = 0;
	uint64 LinkTime = 0;
	uint32 NumCacheHits = 0;

	TotalCreationTime -= FPlatformTime::Cycles64();

	ID3D12Device5* RayTracingDevice = Device->GetParentAdapter()->GetD3DDevice5();

	TArrayView<FRHIRayTracingShader*> InitializerHitGroups = Initializer.GetHitGroupTable();
	TArrayView<FRHIRayTracingShader*> InitializerMissShaders = Initializer.GetMissTable();
	TArrayView<FRHIRayTracingShader*> InitializerRayGenShaders = Initializer.GetRayGenTable();
	TArrayView<FRHIRayTracingShader*> InitializerCallableShaders = Initializer.GetCallableTable();
			
	FRHIShaderBindingLayout ShaderBindingLayout = Initializer.ShaderBindingLayout ? *Initializer.ShaderBindingLayout : FRHIShaderBindingLayout();

	const uint32 MaxTotalShaders = InitializerRayGenShaders.Num() + InitializerMissShaders.Num() + InitializerHitGroups.Num() + InitializerCallableShaders.Num();
	checkf(MaxTotalShaders >= 1, TEXT("Ray tracing pipelines are expected to contain at least one shader"));

	FD3D12RayTracingPipelineCache* PipelineCache = Device->GetRayTracingPipelineCache();

	// All raygen shaders must share the same global root signature (this is validated below)

	GlobalRootSignature = PipelineCache->GetGlobalRootSignature(ShaderBindingLayout);

	const FD3D12RayTracingPipelineState* BasePipeline = GRHISupportsRayTracingPSOAdditions 
		? FD3D12DynamicRHI::ResourceCast(Initializer.BasePipeline.GetReference())
		: nullptr;

	if (BasePipeline)
	{
		PipelineShaderHashes = BasePipeline->PipelineShaderHashes;
	}
	PipelineShaderHashes.Reserve(MaxTotalShaders);

	TArray<FD3D12RayTracingPipelineCache::FEntry*> UniqueShaderCollections;
	UniqueShaderCollections.Reserve(MaxTotalShaders);

	FGraphEventArray CompileCompletionList;
	CompileCompletionList.Reserve(MaxTotalShaders);

	// Helper function to acquire a D3D12_EXISTING_COLLECTION_DESC for a compiled shader via cache

	auto AddShaderCollection = [Device, ShaderBindingLayoutHash = ShaderBindingLayout.GetHash(), GlobalRootSignature = this->GlobalRootSignature, PipelineCache,
									&UniqueShaderHashes = this->PipelineShaderHashes, &UniqueShaderCollections, &Initializer, &NumCacheHits, &CompileTime,
									&CompileCompletionList]
		(FD3D12RayTracingShader* Shader, FD3D12RayTracingPipelineCache::ECollectionType CollectionType)
	{
		// verify that that the same shader binding layout is used for all shaders in the RTPSO or not sampling any resources
		uint32 TotalResourceCount = Shader->ResourceCounts.NumCBs + Shader->ResourceCounts.NumSRVs + Shader->ResourceCounts.NumUAVs + Shader->ResourceCounts.NumSamplers;
		if (TotalResourceCount != 0 && Shader->ShaderBindingLayoutHash != ShaderBindingLayoutHash)
		{
			UE_LOGF(LogD3D12RHI, Warning, "Raytracing shader with with entry point %ls, name %ls and ShaderBindingLayout hash %u doesn't match the RTPSO ShaderBindingLayout hash %u.",
			       *Shader->EntryPoint, Shader->GetShaderName(), Shader->ShaderBindingLayoutHash, ShaderBindingLayoutHash);
			ensure(false);
		}

		bool bIsAlreadyInSet = false;
		const uint64 ShaderHash = GetShaderHash64(Shader);
		UniqueShaderHashes.Add(ShaderHash, &bIsAlreadyInSet);

		bool bCacheHit = false;

		CompileTime -= FPlatformTime::Cycles64();

		FD3D12RayTracingPipelineCache::FEntry* ShaderCacheEntry = PipelineCache->GetOrCompileShader(
			Device, Shader, GlobalRootSignature,
			Initializer.MaxAttributeSizeInBytes,
			Initializer.MaxPayloadSizeInBytes,
			!Initializer.bPartial, //< partial PSOs are not required
			CollectionType, CompileCompletionList,
			&bCacheHit);

		CompileTime += FPlatformTime::Cycles64();

		if (!bIsAlreadyInSet)
		{
			UniqueShaderCollections.Add(ShaderCacheEntry);

			if (bCacheHit) NumCacheHits++;
		}

		return ShaderCacheEntry;
	};

	// Add ray generation shaders

	TArray<FD3D12RayTracingPipelineCache::FEntry*> RayGenShaderEntries;

	RayGenShaders.Reserve(InitializerRayGenShaders.Num());
	RayGenShaderEntries.Reserve(InitializerRayGenShaders.Num());
	TMap<FShaderHash, int32> RayGenShaderIndexByHash;

	checkf(UniqueShaderCollections.Num() == 0, TEXT("Ray generation shaders are expected to be first in the UniqueShaderCollections list."));

	for (FRHIRayTracingShader* ShaderRHI : InitializerRayGenShaders)
	{
		FD3D12RayTracingShader* Shader = FD3D12DynamicRHI::ResourceCast(ShaderRHI);
		checkf(!Shader->UsesGlobalUniformBuffer(), TEXT("Global uniform buffers are not implemented for ray generation shaders"));

		FD3D12RayTracingPipelineCache::FEntry* ShaderCacheEntry = AddShaderCollection(Shader, FD3D12RayTracingPipelineCache::ECollectionType::RayGen);

		RayGenShaderEntries.Add(ShaderCacheEntry);
		RayGenShaderIndexByHash.Add(Shader->GetHash(), RayGenShaders.Shaders.Num());
		RayGenShaders.Shaders.Add(Shader);
	}

	MaxHitGroupViewDescriptors = 0;
	MaxLocalRootSignatureSize = 0;


	// Add miss shaders

	TArray<FD3D12RayTracingPipelineCache::FEntry*> MissShaderEntries;
	MissShaders.Reserve(InitializerMissShaders.Num());
	MissShaderEntries.Reserve(InitializerMissShaders.Num());

	for (FRHIRayTracingShader* ShaderRHI : InitializerMissShaders)
	{
		FD3D12RayTracingShader* Shader = FD3D12DynamicRHI::ResourceCast(ShaderRHI);

		checkf(Shader, TEXT("A valid ray tracing shader must be provided for all elements in the FRayTracingPipelineStateInitializer miss shader table."));
		checkf(!Shader->UsesGlobalUniformBuffer(), TEXT("Global uniform buffers are not implemented for ray tracing miss shaders"));

		const uint32 ShaderViewDescriptors = Shader->ResourceCounts.NumSRVs + Shader->ResourceCounts.NumUAVs;
		MaxHitGroupViewDescriptors = FMath::Max(MaxHitGroupViewDescriptors, ShaderViewDescriptors);
		MaxLocalRootSignatureSize = FMath::Max(MaxLocalRootSignatureSize, Shader->LocalRootSignature->GetTotalRootSignatureSizeInBytes());

		FD3D12RayTracingPipelineCache::FEntry* ShaderCacheEntry = AddShaderCollection(Shader, FD3D12RayTracingPipelineCache::ECollectionType::Miss);

		MissShaderEntries.Add(ShaderCacheEntry);
		MissShaders.Shaders.Add(Shader);
	}

	// Add hit groups

	TArray<FD3D12RayTracingPipelineCache::FEntry*> HitGroupEntries;
	HitGroupShaders.Reserve(InitializerHitGroups.Num());
	HitGroupEntries.Reserve(InitializerHitGroups.Num());

	for (FRHIRayTracingShader* ShaderRHI : InitializerHitGroups)
	{
		FD3D12RayTracingShader* Shader = FD3D12DynamicRHI::ResourceCast(ShaderRHI);

		checkf(Shader, TEXT("A valid ray tracing hit group shader must be provided for all elements in the FRayTracingPipelineStateInitializer hit group table."));

		const uint32 ShaderViewDescriptors = Shader->ResourceCounts.NumSRVs + Shader->ResourceCounts.NumUAVs;
		MaxHitGroupViewDescriptors = FMath::Max(MaxHitGroupViewDescriptors, ShaderViewDescriptors);
		MaxLocalRootSignatureSize = FMath::Max(MaxLocalRootSignatureSize, Shader->LocalRootSignature->GetTotalRootSignatureSizeInBytes());

		FD3D12RayTracingPipelineCache::FEntry* ShaderCacheEntry = AddShaderCollection(Shader, FD3D12RayTracingPipelineCache::ECollectionType::HitGroup);

		HitGroupEntries.Add(ShaderCacheEntry);
		HitGroupShaders.Shaders.Add(Shader);
	}

	// Add callable shaders

	TArray<FD3D12RayTracingPipelineCache::FEntry*> CallableShaderEntries;
	CallableShaders.Reserve(InitializerCallableShaders.Num());
	CallableShaderEntries.Reserve(InitializerCallableShaders.Num());

	for (FRHIRayTracingShader* ShaderRHI : InitializerCallableShaders)
	{
		FD3D12RayTracingShader* Shader = FD3D12DynamicRHI::ResourceCast(ShaderRHI);

		checkf(Shader, TEXT("A valid ray tracing shader must be provided for all elements in the FRayTracingPipelineStateInitializer callable shader table."));
		checkf(!Shader->UsesGlobalUniformBuffer(), TEXT("Global uniform buffers are not implemented for ray tracing callable shaders"));

		const uint32 ShaderViewDescriptors = Shader->ResourceCounts.NumSRVs + Shader->ResourceCounts.NumUAVs;
		MaxHitGroupViewDescriptors = FMath::Max(MaxHitGroupViewDescriptors, ShaderViewDescriptors);
		MaxLocalRootSignatureSize = FMath::Max(MaxLocalRootSignatureSize, Shader->LocalRootSignature->GetTotalRootSignatureSizeInBytes());

		FD3D12RayTracingPipelineCache::FEntry* ShaderCacheEntry = AddShaderCollection(Shader, FD3D12RayTracingPipelineCache::ECollectionType::Callable);

		CallableShaderEntries.Add(ShaderCacheEntry);
		CallableShaders.Shaders.Add(Shader);
	}

	check(Initializer.GetMaxLocalBindingDataSize() >= MaxLocalRootSignatureSize);

	// Wait for all compilation tasks to be complete and then gather the compiled collection descriptors

	CompileTime -= FPlatformTime::Cycles64();

	FTaskGraphInterface::Get().WaitUntilTasksComplete(CompileCompletionList);

	CompileTime += FPlatformTime::Cycles64();
	
	if (Initializer.bPartial)
	{
		// Partial pipelines don't have a linking phase, so exit immediately after compilation tasks are complete.
		return;
	}

	TArray<D3D12_EXISTING_COLLECTION_DESC> UniqueShaderCollectionDescs;
	UniqueShaderCollectionDescs.Reserve(MaxTotalShaders);
	for (FD3D12RayTracingPipelineCache::FEntry* Entry : UniqueShaderCollections)
	{
		UniqueShaderCollectionDescs.Add(Entry->GetCollectionDesc());
	}

	// Link final RTPSO from shader collections

	LinkTime -= FPlatformTime::Cycles64();

	// Extending RTPSOs is currently not compatible with PSO specializations
	if (BasePipeline && GRayTracingSpecializeStateObjects == 0)
	{
		if (UniqueShaderCollectionDescs.Num() == 0)
		{
			// New PSO does not actually have any new shaders that were not in the base
			StateObject = BasePipeline->StateObject.GetReference();
		}
		else
		{

			TArray<D3D12_STATE_SUBOBJECT> Subobjects;

			int32 SubobjectIndex = 0;
			Subobjects.Reserve(UniqueShaderCollectionDescs.Num() + 1);

			D3D12_STATE_OBJECT_CONFIG StateObjectConfig = {};
			StateObjectConfig.Flags = D3D12_STATE_OBJECT_FLAG_ALLOW_STATE_OBJECT_ADDITIONS;
			Subobjects.Add(D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_STATE_OBJECT_CONFIG, &StateObjectConfig });

			for (const D3D12_EXISTING_COLLECTION_DESC& Collection : UniqueShaderCollectionDescs)
			{
				Subobjects.Add(D3D12_STATE_SUBOBJECT{ D3D12_STATE_SUBOBJECT_TYPE_EXISTING_COLLECTION, &Collection });
			}

			D3D12_STATE_OBJECT_DESC Desc = {};
			Desc.Type = D3D12_STATE_OBJECT_TYPE_RAYTRACING_PIPELINE;
			Desc.NumSubobjects = Subobjects.Num();
			Desc.pSubobjects = Subobjects.GetData();

			ID3D12Device7* Device7 = Device->GetParentAdapter()->GetD3DDevice7();

			VERIFYD3D12RESULT(Device7->AddToStateObject(&Desc,
				BasePipeline->StateObject.GetReference(),
				IID_PPV_ARGS(StateObject.GetInitReference())));
		}
	}
	else
	{
		StateObject = CreateRayTracingStateObject(
			RayTracingDevice,
			{}, // Libraries,
			{}, // LibraryExports,
			Initializer.MaxAttributeSizeInBytes,
			Initializer.MaxPayloadSizeInBytes,
			{}, // HitGroups
			GlobalRootSignature,
			{}, // LocalRootSignatures
			{}, // LocalRootSignatureAssociations,
			UniqueShaderCollectionDescs,
			D3D12_STATE_OBJECT_TYPE_RAYTRACING_PIPELINE);

		if (StateObject == nullptr)
		{
			UE_LOGF(LogD3D12RHI, Fatal, "Failed to a create raytracing pipeline state");
		}
	}

	if (GRayTracingSpecializeStateObjects != 0 && Initializer.GetRayGenTable().Num() > 1)
	{
		CreateSpecializedStateObjects(
			RayTracingDevice,
			GlobalRootSignature,
			Initializer.MaxAttributeSizeInBytes,
			Initializer.MaxPayloadSizeInBytes,
			RayGenShaders,
			UniqueShaderCollections,
			RayGenShaderIndexByHash,
			SpecializedStateObjects, // out param
			SpecializationIndices // out param
		);
	}

	LinkTime += FPlatformTime::Cycles64();

	HRESULT QueryInterfaceResult = StateObject->QueryInterface(IID_PPV_ARGS(PipelineProperties.GetInitReference()));
	checkf(SUCCEEDED(QueryInterfaceResult), TEXT("Failed to query pipeline properties from the ray tracing pipeline state object. Result=%08x"), QueryInterfaceResult);

	// Query shader identifiers from the pipeline state object

	check(HitGroupEntries.Num() == InitializerHitGroups.Num());

	auto GetEntryShaderIdentifier = [Properties = PipelineProperties.GetReference()](FD3D12RayTracingPipelineCache::FEntry* Entry) -> FD3D12ShaderIdentifier
	{
		if (Entry->Identifier.IsValid())
		{
			return Entry->Identifier;
		}
		else
		{
			return GetShaderIdentifier(Properties, Entry->GetPrimaryExportNameChars());
		}
	};

	HitGroupShaders.Identifiers.SetNumUninitialized(InitializerHitGroups.Num());
	for (int32 HitGroupIndex = 0; HitGroupIndex < HitGroupEntries.Num(); ++HitGroupIndex)
	{
		HitGroupShaders.Identifiers[HitGroupIndex] = GetEntryShaderIdentifier(HitGroupEntries[HitGroupIndex]);
	}

	RayGenShaders.Identifiers.SetNumUninitialized(RayGenShaderEntries.Num());
	for (int32 ShaderIndex = 0; ShaderIndex < RayGenShaderEntries.Num(); ++ShaderIndex)
	{
		RayGenShaders.Identifiers[ShaderIndex] = GetEntryShaderIdentifier(RayGenShaderEntries[ShaderIndex]);
	}

	MissShaders.Identifiers.SetNumUninitialized(MissShaderEntries.Num());
	for (int32 ShaderIndex = 0; ShaderIndex < MissShaderEntries.Num(); ++ShaderIndex)
	{
		MissShaders.Identifiers[ShaderIndex] = GetEntryShaderIdentifier(MissShaderEntries[ShaderIndex]);
	}

	CallableShaders.Identifiers.SetNumUninitialized(CallableShaderEntries.Num());
	for (int32 ShaderIndex = 0; ShaderIndex < CallableShaderEntries.Num(); ++ShaderIndex)
	{
		CallableShaders.Identifiers[ShaderIndex] = GetEntryShaderIdentifier(CallableShaderEntries[ShaderIndex]);
	}

	PipelineStackSize = PipelineProperties->GetPipelineStackSize();

	TotalCreationTime += FPlatformTime::Cycles64();

	// Report stats for pipelines that take a long time to create

#if !NO_LOGGING

	// Gather PSO stats
	ShaderStats.Reserve(UniqueShaderCollections.Num());
	for (FD3D12RayTracingPipelineCache::FEntry* Entry : UniqueShaderCollections)
	{
		FShaderStats Stats;
		Stats.Name = *(Entry->Shader->EntryPoint);
		Stats.ShaderSize = Entry->Shader->Code.Num();
		Stats.CompileTimeMS = Entry->CompileTimeMS;

	#if PLATFORM_WINDOWS
		if (Entry->Shader->GetFrequency() == SF_RayGen)
		{
			Stats.StackSize = uint32(PipelineProperties->GetShaderStackSize(*(Entry->ExportNames[0])));
		}
	#endif // PLATFORM_WINDOWS

		ShaderStats.Add(Stats);
	}

	ShaderStats.Sort([](const FShaderStats& A, const FShaderStats& B) { return B.CompileTimeMS < A.CompileTimeMS; });

	const double TotalCreationTimeMS = 1000.0 * FPlatformTime::ToSeconds64(TotalCreationTime);
	// log a message if the RTPSO creating took a long time (with a higher threshold if this was a background compilation)
	const double CreationTimeWarningThresholdMS = 10.0;
	const double BackgroundCreatingTimeWarningThresholdMS = 1000.0;
	if (TotalCreationTimeMS > (Initializer.bBackgroundCompilation ? CreationTimeWarningThresholdMS : BackgroundCreatingTimeWarningThresholdMS))
	{
		const double CompileTimeMS = 1000.0 * FPlatformTime::ToSeconds64(CompileTime);
		const double LinkTimeMS = 1000.0 * FPlatformTime::ToSeconds64(LinkTime);
		const uint32 NumUniqueShaders = UniqueShaderCollections.Num();
		UE_LOGF(LogD3D12RHI, Log,
			"Creating RTPSO with %d shaders (%d cached, %d new) took %.2f ms. Compile time %.2f ms, link time %.2f ms.",
			PipelineShaderHashes.Num(), NumCacheHits, NumUniqueShaders - NumCacheHits, (float)TotalCreationTimeMS, (float)CompileTimeMS, (float)LinkTimeMS);
	}

#endif //!NO_LOGGING
}

class FD3D12RayTracingShaderBindingTable : public FRHIShaderBindingTable, public FD3D12AdapterChild
{
public:

	UE_NONCOPYABLE(FD3D12RayTracingShaderBindingTable)

	FD3D12RayTracingShaderBindingTable(FRHICommandListBase& RHICmdList, FD3D12Adapter* Adapter, const FRayTracingShaderBindingTableInitializer& InInitializer)
		: FRHIShaderBindingTable(InInitializer), FD3D12AdapterChild(Adapter)
	{
		INC_DWORD_STAT(STAT_D3D12RayTracingAllocatedSBT);

		checkf(Initializer.NumMissShaderSlots >= 1, TEXT("Need at least 1 miss shader slot."));	

		for (FD3D12Device* Device : Adapter->GetDevices())
		{
			InitForDevice(RHICmdList, Device);
		}
	};

	~FD3D12RayTracingShaderBindingTable()
	{
		for (auto& Table : ShaderTablesPerGPU)
		{
			delete Table;
			Table = nullptr;
		}

		DEC_DWORD_STAT(STAT_D3D12RayTracingAllocatedSBT);
	}

	FD3D12RayTracingShaderBindingTableInternal* GetTableForDevice(FD3D12Device* Device)
	{
		const uint32 GPUIndex = Device->GetGPUIndex();
		return ShaderTablesPerGPU[GPUIndex];
	}
	   
	virtual FRHISizeAndStride GetInlineBindingDataSizeAndStride() const override final
	{
		// Size should be the same for all tables
		if (ShaderTablesPerGPU[0])
		{
			return ShaderTablesPerGPU[0]->GetInlineBindingDataSizeAndStride();
		}
		return FRHISizeAndStride{0,0};
	}

private:

	void InitForDevice(FRHICommandListBase& RHICmdList, FD3D12Device* Device)
	{
	 	TRACE_CPUPROFILER_EVENT_SCOPE(ShaderTableInit);
	 	SCOPE_CYCLE_COUNTER(STAT_D3D12CreateShaderTable);
		
		const uint32 GPUIndex = Device->GetGPUIndex();

		check(ShaderTablesPerGPU[GPUIndex] == nullptr);
		ShaderTablesPerGPU[GPUIndex] = new FD3D12RayTracingShaderBindingTableInternal(RHICmdList, Initializer, Device);
	}
	
	FD3D12RayTracingShaderBindingTableInternal* ShaderTablesPerGPU[MAX_NUM_GPUS] = {};
};

void FD3D12Device::InitRayTracing()
{
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6)
	{
		LLM_SCOPE_BYNAME(TEXT("FD3D12RT"));
		check(RayTracingPipelineCache == nullptr);
		RayTracingPipelineCache = new FD3D12RayTracingPipelineCache(GetParentAdapter());
	}
}

void FD3D12Device::CleanupRayTracing()
{
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6)
	{
		delete RayTracingPipelineCache;
		RayTracingPipelineCache = nullptr;

		for (FD3D12Queue& Queue : Queues)
		{
			delete Queue.RayTracingDispatchRaysDescBuffer;
			Queue.RayTracingDispatchRaysDescBuffer = nullptr;
		}
	}
}

FRayTracingAccelerationStructureSize FD3D12DynamicRHI::RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer)
{
	D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS BuildInputs = {};
	BuildInputs.Type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_TOP_LEVEL;
	BuildInputs.DescsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
	BuildInputs.NumDescs = Initializer.MaxNumInstances;
	BuildInputs.Flags = TranslateRayTracingAccelerationStructureFlags(Initializer.BuildFlags);

	FD3D12Adapter& Adapter = GetAdapter();

	FRayTracingAccelerationStructureSize SizeInfo = {};
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; ++GPUIndex)
	{
		D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO PrebuildInfo = {};
		Adapter.GetDevice(GPUIndex)->GetRaytracingAccelerationStructurePrebuildInfo(&BuildInputs, &PrebuildInfo);

		SizeInfo.ResultSize = FMath::Max(SizeInfo.ResultSize, PrebuildInfo.ResultDataMaxSizeInBytes);
		SizeInfo.BuildScratchSize  = FMath::Max(SizeInfo.BuildScratchSize, PrebuildInfo.ScratchDataSizeInBytes);
		SizeInfo.UpdateScratchSize = FMath::Max(SizeInfo.UpdateScratchSize, PrebuildInfo.UpdateScratchDataSizeInBytes);
	}

	return SizeInfo;
}

static ERayTracingAccelerationStructureFlags GetRayTracingAccelerationStructureBuildFlags(const FRayTracingGeometryInitializer& Initializer)
{
	ERayTracingAccelerationStructureFlags BuildFlags = ERayTracingAccelerationStructureFlags::None;

	if (Initializer.bFastBuild)
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastBuild;
	}
	else
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;
	}

	if (Initializer.bAllowUpdate)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
	}

	if (!Initializer.bFastBuild && !Initializer.bAllowUpdate && Initializer.bAllowCompaction && GD3D12RayTracingAllowCompaction && (uint32(GD3D12RayTracingCompactionMinPrimitiveCount) < Initializer.TotalPrimitiveCount))
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction);
	}

	if (GRayTracingDebugForceBuildMode == 1)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastBuild);
		EnumRemoveFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastTrace);
	}
	else if (GRayTracingDebugForceBuildMode == 2)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastTrace);
		EnumRemoveFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastBuild);
	}

	return BuildFlags;
}

static bool GeometryUsesLinearSweptSpheres(const FRayTracingGeometryInitializer& Initializer)
{
	return Initializer.GeometryType == RTGT_LinearSweptSpheres;
}

static void TranslateRayTracingGeometryDescs(const FRayTracingGeometryInitializer& Initializer, TArrayView<D3D12_RAYTRACING_GEOMETRY_DESC> Output)
{
	check(Output.Num() == Initializer.Segments.Num());
	check(!GeometryUsesLinearSweptSpheres(Initializer));

	uint32 ComputedPrimitiveCountForValidation = 0;

	for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
	{
		const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];

		checkf(Segment.VertexBuffer, TEXT("Position vertex buffer is required for ray tracing geometry."));
		checkf(Segment.VertexBufferStride, TEXT("Non-zero position vertex buffer stride is required."));
		checkf(Segment.VertexBufferStride % 4 == 0, TEXT("Position vertex buffer stride must be aligned to 4 bytes for ByteAddressBuffer loads to work."));

		checkf(Segment.MaxVertices != 0 || Segment.NumPrimitives == 0,
			TEXT("FRayTracingGeometrySegment.MaxVertices for '%s' must contain number of positions in the vertex buffer or maximum index buffer value+1 if index buffer is provided."),
			*Initializer.DebugName.ToString());

		D3D12_RAYTRACING_GEOMETRY_DESC& Desc = Output[SegmentIndex];
		Desc = {}; // zero-initialize

		Desc.Flags = TranslateSegmentToGeometryFlags(Segment);

		switch (Initializer.GeometryType)
		{
		case RTGT_Triangles:
			checkf(Segment.VertexBufferElementType == VET_Float3 ||
				   Segment.VertexBufferElementType == VET_Float4, TEXT("Only float3/4 vertex buffers are currently implemented.")); // #dxr_todo UE-72160: support other vertex buffer formats
			checkf(Segment.VertexBufferStride >= 12, TEXT("Only deinterleaved float3 position vertex buffers are currently implemented.")); // #dxr_todo UE-72160: support interleaved vertex buffers

			Desc.Type = D3D12_RAYTRACING_GEOMETRY_TYPE_TRIANGLES;
			Desc.Triangles.VertexFormat = TranslateRayTracingVertexElementType(Segment.VertexBufferElementType);
			checkf(Desc.Triangles.VertexFormat != DXGI_FORMAT_UNKNOWN, TEXT("Unknown vertex format %d - not currently implemented"), int(Desc.Triangles.VertexFormat));

			if (Initializer.IndexBuffer)
			{
				uint32 IndexStride = Initializer.IndexBuffer->GetStride();
				check(Initializer.IndexBuffer->GetSize() >=
					  (Segment.FirstPrimitive + Segment.NumPrimitives) * FD3D12RayTracingGeometry::IndicesPerTriangle * IndexStride + Initializer.IndexBufferOffset);

				// In some cases the geometry is created with 16 bit index buffer, but it's 32 bit at build time.
				// We conservatively set this to 32 bit to allocate acceleration structure memory.
				Desc.Triangles.IndexFormat = DXGI_FORMAT_R32_UINT;
				Desc.Triangles.IndexCount = Segment.NumPrimitives * FD3D12RayTracingGeometry::IndicesPerTriangle;
				Desc.Triangles.VertexCount = Segment.MaxVertices;
			}
			else
			{
				// Non-indexed geometry
				checkf(Initializer.Segments.Num() == 1, TEXT("Non-indexed geometry with multiple segments is not implemented."));
				Desc.Triangles.IndexFormat = DXGI_FORMAT_UNKNOWN;
				Desc.Triangles.VertexCount = FMath::Min<uint32>(Segment.MaxVertices, Initializer.TotalPrimitiveCount * 3);
			}
			break;

		case RTGT_Procedural:
			checkf(Segment.VertexBufferStride >= (2 * sizeof(FVector3f)), TEXT("Procedural geometry vertex buffer must contain at least 2xFloat3 that defines 3D bounding boxes of primitives."));

			Desc.Type = D3D12_RAYTRACING_GEOMETRY_TYPE_PROCEDURAL_PRIMITIVE_AABBS;
			Desc.AABBs.AABBCount = Segment.NumPrimitives;
			break;
		default:
			checkf(false, TEXT("Unexpected ray tracing geometry type"));
			break;
		}

		ComputedPrimitiveCountForValidation += Segment.NumPrimitives;
	}
	check(ComputedPrimitiveCountForValidation == Initializer.TotalPrimitiveCount);
}
#if WITH_NVAPI
static void TranslateRayTracingGeometryDescsEx(const FRayTracingGeometryInitializer& Initializer, TArrayView<NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX> Output)
{
	check(GeometryUsesLinearSweptSpheres(Initializer)); // This function is only for LSS primitives, otherwise use main D3D path
	check(Output.Num() == Initializer.Segments.Num());

	uint32 ComputedPrimitiveCountForValidation = 0;

	for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
	{
		const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];

		checkf(Segment.VertexBuffer, TEXT("Position vertex buffer is required for ray tracing geometry."));
		checkf(Segment.VertexBufferStride, TEXT("Non-zero position vertex buffer stride is required."));
		checkf(Segment.VertexBufferStride % 4 == 0, TEXT("Position vertex buffer stride must be aligned to 4 bytes for ByteAddressBuffer loads to work."));

		checkf(Segment.MaxVertices != 0 || Segment.NumPrimitives == 0,
			   TEXT("FRayTracingGeometrySegment.MaxVertices for '%s' must contain number of positions in the vertex buffer or maximum index buffer value+1 if index buffer is provided."),
			   *Initializer.DebugName.ToString());

		NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX& Desc = Output[SegmentIndex];
		Desc = {}; // zero-initialize

		Desc.flags = TranslateSegmentToGeometryFlags(Segment);

		// RTGT_LinearSweptSpheres (only case for now)
		{
			Desc.type = NVAPI_D3D12_RAYTRACING_GEOMETRY_TYPE_LSS_EX;

			checkf(Segment.VertexBufferStride >= sizeof(FVector4f), TEXT("LSS vertex buffer must contain at least float4 that defines control point."));

			uint32 IndexStride = Initializer.IndexBuffer->GetStride();
			check(Initializer.IndexBuffer->GetSize() >=
				(Segment.FirstPrimitive + Segment.NumPrimitives) * FD3D12RayTracingGeometry::IndicesPerLinearSweptSphere * IndexStride + Initializer.IndexBufferOffset);

			Desc.lss = {};
			Desc.lss.vertexCount = Segment.MaxVertices;
			Desc.lss.indexCount = Segment.NumPrimitives * FD3D12RayTracingGeometry::IndicesPerLinearSweptSphere;
			Desc.lss.primitiveCount = Segment.NumPrimitives;
			Desc.lss.vertexPositionBuffer.StartAddress = 0;
			Desc.lss.vertexPositionBuffer.StrideInBytes = 4 * sizeof(float);
			Desc.lss.vertexPositionFormat = DXGI_FORMAT_R32G32B32_FLOAT;
			Desc.lss.vertexRadiusBuffer.StartAddress = 3 * sizeof(float);
			Desc.lss.vertexRadiusBuffer.StrideInBytes = 4 * sizeof(float);
			Desc.lss.vertexRadiusFormat = DXGI_FORMAT_R32_FLOAT;
			Desc.lss.indexFormat = DXGI_FORMAT_R32_UINT;
			Desc.lss.indexBuffer.StartAddress = 0;
			Desc.lss.indexBuffer.StrideInBytes = sizeof(uint32);
			Desc.lss.endcapMode = NVAPI_D3D12_RAYTRACING_LSS_ENDCAP_MODE_NONE;
			Desc.lss.primitiveFormat = NVAPI_D3D12_RAYTRACING_LSS_PRIMITIVE_FORMAT_LIST;
		}

		ComputedPrimitiveCountForValidation += Segment.NumPrimitives;
	}

	check(ComputedPrimitiveCountForValidation == Initializer.TotalPrimitiveCount);
}
#endif // WITH_NVAPI


FRayTracingAccelerationStructureSize FD3D12DynamicRHI::RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer)
{
	{
		const bool bHasOfflineMetadata = Initializer.OfflineDataHeader.IsValid();

		if (bHasOfflineMetadata)
		{
			FRayTracingAccelerationStructureOfflineMetadata OfflineBvhMetadata = RHIGetRayTracingGeometryOfflineMetadata(Initializer.OfflineDataHeader);

			FRayTracingAccelerationStructureSize SizeInfo = {};
			SizeInfo.ResultSize = Align(OfflineBvhMetadata.Size, GRHIRayTracingAccelerationStructureAlignment);

			return SizeInfo;
		}
	}

	FRayTracingAccelerationStructureSize SizeInfo = {};
	
	ERayTracingAccelerationStructureFlags BuildFlags = GetRayTracingAccelerationStructureBuildFlags(Initializer);

	D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS PrebuildDescInputs = {};

	TArray<D3D12_RAYTRACING_GEOMETRY_DESC, TInlineAllocator<32>> GeometryDescs;
#if WITH_NVAPI
	TArray<NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX, TInlineAllocator<1>> GeometryDescsEx;
	if (GeometryUsesLinearSweptSpheres(Initializer))
	{
		GeometryDescsEx.SetNumUninitialized(Initializer.Segments.Num());
		TranslateRayTracingGeometryDescsEx(Initializer, GeometryDescsEx);
	}
	else
#endif
	{
		GeometryDescs.SetNumUninitialized(Initializer.Segments.Num());
		TranslateRayTracingGeometryDescs(Initializer, GeometryDescs);
	}

	PrebuildDescInputs.Type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL;
	PrebuildDescInputs.DescsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
	PrebuildDescInputs.NumDescs = GeometryDescs.Num();
	PrebuildDescInputs.pGeometryDescs = GeometryDescs.GetData();
	PrebuildDescInputs.Flags = TranslateRayTracingAccelerationStructureFlags(BuildFlags);

	FD3D12Adapter& Adapter = GetAdapter();

	// We don't know the final index buffer format, so take maximum of 16 and 32 bit.

	static const DXGI_FORMAT ValidIndexBufferFormats[] = { DXGI_FORMAT_R16_UINT, DXGI_FORMAT_R32_UINT };
	static const DXGI_FORMAT NullIndexBufferFormats[] = { DXGI_FORMAT_UNKNOWN };

	TArrayView<const DXGI_FORMAT> IndexFormats = Initializer.IndexBuffer.IsValid() 
		? MakeArrayView(ValidIndexBufferFormats)
		: MakeArrayView(NullIndexBufferFormats);

	for (DXGI_FORMAT IndexFormat : IndexFormats)
	{
		for (D3D12_RAYTRACING_GEOMETRY_DESC& GeometryDesc : GeometryDescs)
		{
			if (GeometryDesc.Type == D3D12_RAYTRACING_GEOMETRY_TYPE_TRIANGLES)
			{
				GeometryDesc.Triangles.IndexFormat = IndexFormat;
			}
		}
#if WITH_NVAPI
		for (NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX& GeometryDescEx : GeometryDescsEx)
		{
			check(GeometryDescEx.type == NVAPI_D3D12_RAYTRACING_GEOMETRY_TYPE_LSS_EX);
			GeometryDescEx.lss.indexFormat = IndexFormat;
		}
#endif

		// Get maximum buffer sizes for all GPUs in the system
		for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; ++GPUIndex)
		{
			D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO PrebuildInfo = {};

#if WITH_NVAPI
			if (GeometryUsesLinearSweptSpheres(Initializer))
			{
				NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS_EX InputsEx = {};
				InputsEx.type = PrebuildDescInputs.Type;
				InputsEx.flags = static_cast<NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS_EX>(PrebuildDescInputs.Flags);
				InputsEx.descsLayout = PrebuildDescInputs.DescsLayout;
				InputsEx.geometryDescStrideInBytes = sizeof(NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX);
				InputsEx.instanceDescs = 0ULL;
				InputsEx.numDescs = GeometryDescsEx.Num();
				InputsEx.pGeometryDescs = GeometryDescsEx.GetData();

				NVAPI_GET_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO_EX_PARAMS PrebuildParams = {};
				PrebuildParams.version = NVAPI_GET_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO_EX_PARAMS_VER;
				PrebuildParams.pDesc = &InputsEx;
				PrebuildParams.pInfo = &PrebuildInfo;

				ID3D12Device5* RayTracingDevice = Adapter.GetD3DDevice5();
				NvAPI_Status status = NvAPI_D3D12_GetRaytracingAccelerationStructurePrebuildInfoEx(RayTracingDevice, &PrebuildParams);
				check(status == NVAPI_OK);
			}
			else
#endif
			{
				Adapter.GetDevice(GPUIndex)->GetRaytracingAccelerationStructurePrebuildInfo(&PrebuildDescInputs, &PrebuildInfo);
			}

			SizeInfo.ResultSize = FMath::Max(SizeInfo.ResultSize, PrebuildInfo.ResultDataMaxSizeInBytes);
			SizeInfo.BuildScratchSize = FMath::Max(SizeInfo.BuildScratchSize, PrebuildInfo.ScratchDataSizeInBytes);
			SizeInfo.UpdateScratchSize = FMath::Max(SizeInfo.UpdateScratchSize, PrebuildInfo.UpdateScratchDataSizeInBytes);
		}
	}

	SizeInfo.ResultSize = Align(SizeInfo.ResultSize, GRHIRayTracingAccelerationStructureAlignment);
	SizeInfo.BuildScratchSize = Align(SizeInfo.BuildScratchSize, GRHIRayTracingScratchBufferAlignment);
	SizeInfo.UpdateScratchSize = Align(FMath::Max(1ULL, SizeInfo.UpdateScratchSize), GRHIRayTracingScratchBufferAlignment);

	return SizeInfo;
}

FRayTracingAccelerationStructureOfflineMetadata FD3D12DynamicRHI::RHIGetRayTracingGeometryOfflineMetadata(const FRayTracingGeometryOfflineDataHeader& OfflineDataHeader)
{
	static_assert(sizeof(FD3D12RayTracingOfflineBvhHeader) <= sizeof(FRayTracingGeometryOfflineDataHeader), "FRayTracingGeometryOfflineDataHeader must be large enough to fit FOfflineBVHHeader");

	FD3D12RayTracingOfflineBvhHeader BvhHeader;
	FMemory::Memcpy(&BvhHeader, &OfflineDataHeader, sizeof(BvhHeader));

	FRayTracingAccelerationStructureOfflineMetadata Metadata;
	Metadata.Size = BvhHeader.Size;
	Metadata.SerializedSize = BvhHeader.SerializedSize;
	Metadata.SerializedOffset = 0;

	return Metadata;
}

FRayTracingPipelineStateRHIRef FD3D12DynamicRHI::RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
{
	FD3D12Device* Device = GetAdapter().GetDevice(0); // All pipelines are created on the first node, as they may be used on any other linked GPU.
	FD3D12RayTracingPipelineState* Result = new FD3D12RayTracingPipelineState(Device, Initializer);

	return Result;
}

FRayTracingGeometryRHIRef FD3D12DynamicRHI::RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer)
{
	FD3D12Adapter& Adapter = GetAdapter();
	return new FD3D12RayTracingGeometry(RHICmdList, &Adapter, Initializer);
}

FRayTracingSceneRHIRef FD3D12DynamicRHI::RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateRayTracingScene);

	FD3D12Adapter& Adapter = GetAdapter();

	return new FD3D12RayTracingScene(&Adapter, MoveTemp(Initializer));
}

FShaderBindingTableRHIRef FD3D12DynamicRHI::RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateRayTracingScene);

	FD3D12Adapter& Adapter = GetAdapter();

	return new FD3D12RayTracingShaderBindingTable(RHICmdList, &Adapter, Initializer);
}

enum class ERayTracingBufferType
{
	AccelerationStructure,
	Scratch
};

static TRefCountPtr<FD3D12Buffer> CreateRayTracingBuffer(FD3D12Adapter* Adapter, uint32 GPUIndex, uint64 Size, ERayTracingBufferType Type, const FDebugName& DebugName)
{
	FString DebugNameString = DebugName.ToString();

	checkf(Size != 0, TEXT("Attempting to create ray tracing %s buffer of zero size. Debug name: %s"),
		Type == ERayTracingBufferType::AccelerationStructure ? TEXT("AccelerationStructure") : TEXT("Scratch"),
		*DebugNameString);

	TRefCountPtr<FD3D12Buffer> Result;

	ID3D12ResourceAllocator* ResourceAllocator = nullptr;
	FRHIGPUMask GPUMask = FRHIGPUMask::FromIndex(GPUIndex);
	bool bHasInitialData = false;

	if (Type == ERayTracingBufferType::AccelerationStructure)
	{
		const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(Size, D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE);
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(*DebugNameString, Size, 0, BUF_AccelerationStructure)
			.SetInitialState(ERHIAccess::BVHWrite)
			.SetGPUMask(GPUMask);

		// TODO: Manage switching between RAS_READ and RAS_WRITE?
		Result = Adapter->CreateRHIBuffer(
			ResourceDesc,
			D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BYTE_ALIGNMENT,
			CreateDesc,
			ED3D12ResourceStateMode::SingleState,
			ED3D12Access::BVHRead | ED3D12Access::BVHWrite
		);
	}
	else if (Type == ERayTracingBufferType::Scratch)
	{
		// 
		// Scratch doesn't need single state anymore because there are only a few scratch allocations left and allocating a 
		// dedicated single state heap for it wastes memory - ideally all scratch allocations should be transient
		const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(Size, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(*DebugNameString, Size, 0, BUF_UnorderedAccess)
			.SetInitialState(ERHIAccess::UAVCompute)
			.SetGPUMask(GPUMask);

		Result = Adapter->CreateRHIBuffer(
			ResourceDesc,
			D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BYTE_ALIGNMENT,
			CreateDesc,
			ED3D12ResourceStateMode::Default,
			ED3D12Access::UAVCompute
		);

		// Elevates the scratch buffer heap priority, which may help performance / stability in low memory conditions 
		// (Acceleration structure already boosted from allocation side)
		ID3D12Pageable* HeapResource = Result->GetResource()->GetPageable();
		Adapter->SetResidencyPriority(HeapResource, D3D12_RESIDENCY_PRIORITY_HIGH, GPUIndex, Size);
	}
	else
	{
		checkNoEntry();
	}

	return Result;
}

FString GetGeometryInitializerDebugString(const FRayTracingGeometryInitializer& Initializer)
{
	TStringBuilder<128> Result;

	Result << "DebugName=" << Initializer.DebugName.ToString();
	Result << " NumSegments=" << Initializer.Segments.Num();
	Result << " NumPrims=" << Initializer.TotalPrimitiveCount;
	if (Initializer.IndexBuffer)
	{
		Result << " IndexStride=" << Initializer.IndexBuffer->GetStride();
	}
	else
	{
		Result << " NonIndexed";
	}

	if (Initializer.OfflineData)
	{
		Result << " HasOfflineData";
	}

	return Result.ToString();
}

FD3D12RayTracingGeometry::FD3D12RayTracingGeometry(FRHICommandListBase& RHICmdList, FD3D12Adapter* Adapter, const FRayTracingGeometryInitializer& InInitializer)
	: FRHIRayTracingGeometry(InInitializer), FD3D12AdapterChild(Adapter)
{
	INC_DWORD_STAT(STAT_D3D12RayTracingAllocatedBLAS);

	static const FName NAME_BLAS("BLAS");

	DebugName = !Initializer.DebugName.IsNone() ? Initializer.DebugName : NAME_BLAS;
	OwnerName = Initializer.OwnerName;
	
	PerGPUData.SetNum(GNumExplicitGPUsForRendering);

	RegisterD3D12RayTracingGeometry(this);

	checkf(Initializer.Segments.Num() > 0, TEXT("Ray tracing geometry must be initialized with at least one segment."));

	SetDirty(FRHIGPUMask::All(), true);

	const bool bHasOfflineMetadata = Initializer.OfflineDataHeader.IsValid();
	FRayTracingAccelerationStructureOfflineMetadata OfflineBvhMetadata;

	if (bHasOfflineMetadata)
	{
		OfflineBvhMetadata = RHIGetRayTracingGeometryOfflineMetadata(InInitializer.OfflineDataHeader);
	}

	const void* SourceData = nullptr;

	if (Initializer.OfflineData != nullptr)
	{
		checkf(bHasOfflineMetadata, TEXT("OfflineData provided in Initializer has data but OfflineDataHeader is not valid."));

		SourceData = Initializer.OfflineData->GetResourceData();

		const uint32 SourceDataSize = Initializer.OfflineData->GetResourceDataSize();
		checkf(SourceDataSize >= OfflineBvhMetadata.SerializedSize, TEXT("OfflineData provided in Initializer has %u bytes but FD3D12RayTracingGeometry expected %u bytes."), SourceDataSize, OfflineBvhMetadata.SerializedSize);
	}

	if (SourceData != nullptr)
	{
		checkf(!InInitializer.bAllowUpdate, TEXT("FD3D12RayTracingGeometry doesn't support updating BVH created using offline data."));
	}

	// Compute the required size of the in-memory BVH buffer
	if (SourceData != nullptr)
	{
		SizeInfo.ResultSize = OfflineBvhMetadata.Size;
		SizeInfo.BuildScratchSize = 0;
		SizeInfo.UpdateScratchSize = 0;

		AccelerationStructureCompactedSize = OfflineBvhMetadata.Size;
#if WITH_NVAPI
		check(!GeometryUsesLinearSweptSpheres(Initializer));
#endif
	}
	else
	{
		// Get maximum buffer sizes for all GPUs in the system
		SizeInfo = RHICalcRayTracingGeometrySize(Initializer);
	}

	checkf(SizeInfo.ResultSize != 0,
		TEXT("Unexpected acceleration structure buffer size (0).\nGeometry initializer details:\n%s"),
		*GetGeometryInitializerDebugString(Initializer));

	// If this RayTracingGeometry going to be used as streaming destination 
	// we don't want to allocate its memory as it will be replaced later by streamed version
	// but we still need correct SizeInfo as it is used to estimate its memory requirements outside of RHI.
	if (Initializer.Type == ERayTracingGeometryInitializerType::StreamingDestination)
	{
		return;
	}

	// Allocate acceleration structure buffer
	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		FPerGPUState& PerGPUState = PerGPUData[GPUIndex];

		PerGPUState.AccelerationStructureBuffer = CreateRayTracingBuffer(Adapter, GPUIndex, SizeInfo.ResultSize, ERayTracingBufferType::AccelerationStructure, DebugName);
		PerGPUState.AccelerationStructureBuffer->SetOwnerName(OwnerName);

		INC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		INC_MEMORY_STAT_BY(STAT_D3D12RayTracingBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		if (Initializer.bAllowUpdate)
		{
			INC_MEMORY_STAT_BY(STAT_D3D12RayTracingDynamicBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		}
		else
		{
			INC_MEMORY_STAT_BY(STAT_D3D12RayTracingStaticBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		}
	});

	switch (Initializer.GeometryType)
	{
		case RTGT_Triangles:          INC_DWORD_STAT_BY(STAT_D3D12RayTracingTrianglesBLAS           , Initializer.TotalPrimitiveCount); break;
		case RTGT_Procedural:         INC_DWORD_STAT_BY(STAT_D3D12RayTracingProceduralPrimitivesBLAS, Initializer.TotalPrimitiveCount); break;
		case RTGT_LinearSweptSpheres: INC_DWORD_STAT_BY(STAT_D3D12RayTracingLinearSweptSpheresBLAS  , Initializer.TotalPrimitiveCount); break;
	}

	const bool bForRendering = Initializer.Type == ERayTracingGeometryInitializerType::Rendering;
	if (SourceData != nullptr)
	{
		FD3D12Device* Device = Adapter->GetDevice(0);

		FD3D12ResourceLocation SrcResourceLoc(Device);
		uint8* DstDataBase = (uint8*)Adapter->GetUploadHeapAllocator(0).AllocUploadResource(OfflineBvhMetadata.SerializedSize, 256, SrcResourceLoc);
		FMemory::Memcpy(DstDataBase, SourceData, OfflineBvhMetadata.SerializedSize);

		RHICmdList.EnqueueLambda([this, SrcResourceLoc = MoveTemp(SrcResourceLoc), bForRendering](FRHICommandListBase& ExecutingCmdList)
		{
			FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
				FPerGPUState& PerGPUState = PerGPUData[GPUIndex];

				FD3D12Buffer* AccelerationStructure = PerGPUState.AccelerationStructureBuffer;

				Context.RayTracingCommandList()->CopyRaytracingAccelerationStructure(
					AccelerationStructure->ResourceLocation.GetGPUVirtualAddress(),
					SrcResourceLoc.GetGPUVirtualAddress(),
					D3D12_RAYTRACING_ACCELERATION_STRUCTURE_COPY_MODE_DESERIALIZE
				);

				Context.UpdateResidency(SrcResourceLoc.GetResource());
				Context.UpdateResidency(AccelerationStructure->GetResource());
				Context.ConditionalSplitCommandList();

				if (bForRendering)
				{
					RegisterAsRenameListener(GPUIndex);
					SetupHitGroupSystemParameters(GPUIndex);
				}
			});

			SetDirty(FRHIGPUMask::All(), false);
		});

		Initializer.OfflineData->Discard();
	}
	else
	{
		// Offline data already registered via FD3D12RHICommandInitializeRayTracingGeometry
		FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
		{
			RegisterAsRenameListener(GPUIndex);
		});
	}
}

void FD3D12RayTracingGeometry::Swap(FD3D12RayTracingGeometry& Other)
{
	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		::Swap(PerGPUData[GPUIndex].AccelerationStructureBuffer, Other.PerGPUData[GPUIndex].AccelerationStructureBuffer);
		::Swap(PerGPUData[GPUIndex].bIsAccelerationStructureDirty, Other.PerGPUData[GPUIndex].bIsAccelerationStructureDirty);
	});
	::Swap(AccelerationStructureCompactedSize, Other.AccelerationStructureCompactedSize);

	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		UnregisterAsRenameListener(GPUIndex);
	});

	Initializer = Other.Initializer;

	DebugName = !Initializer.DebugName.IsNone() ? Initializer.DebugName : FName("BLAS");

	checkf(Initializer.Segments.Num() > 0, TEXT("Ray tracing geometry must be initialized with at least one segment."));

	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		RegisterAsRenameListener(GPUIndex);
		SetupHitGroupSystemParameters(GPUIndex);
	});

#if ENABLE_RESIDENCY_MANAGEMENT
	++ResourceGeneration;
#endif
}

void FD3D12RayTracingGeometry::ReleaseUnderlyingResource()
{
	UnregisterD3D12RayTracingGeometry(this);

	// Remove compaction request if still pending
	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		FPerGPUState& PerGPUState = PerGPUData[GPUIndex];
		if (PerGPUState.bHasPendingCompactionRequests)
		{
			check(PerGPUState.AccelerationStructureBuffer);
			FD3D12Device* Device = PerGPUState.AccelerationStructureBuffer.GetReference()->GetParentDevice();
			bool bRequestFound = Device->GetRayTracingCompactionRequestHandler()->ReleaseRequest(this);
			check(bRequestFound);
			PerGPUState.bHasPendingCompactionRequests = false;
		}
	});

	// Unregister as dependent resource on vertex and index buffers & clear the SRVs
	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		FPerGPUState& PerGPUState = PerGPUData[GPUIndex];
		PerGPUState.HitGroupSystemIndexBufferSRV.Reset();
		PerGPUState.HitGroupSystemSegmentVertexBufferSRVs.Empty();

		UnregisterAsRenameListener(GPUIndex);
	});

	for (FPerGPUState& GPUState : PerGPUData)
	{
		if (GPUState.AccelerationStructureBuffer)
		{
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, GPUState.AccelerationStructureBuffer->GetSize());
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingBLASMemory, GPUState.AccelerationStructureBuffer->GetSize());

			ERayTracingAccelerationStructureFlags BuildFlags = GetRayTracingAccelerationStructureBuildFlags(Initializer);
			if (EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate))
			{
				DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingDynamicBLASMemory, GPUState.AccelerationStructureBuffer->GetSize());
			}
			else
			{
				DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingStaticBLASMemory, GPUState.AccelerationStructureBuffer->GetSize());
			}
		}
	}

	if (Initializer.Type != ERayTracingGeometryInitializerType::StreamingSource)
	{
		switch (Initializer.GeometryType)
		{
			case RTGT_Triangles:          DEC_DWORD_STAT_BY(STAT_D3D12RayTracingTrianglesBLAS           , Initializer.TotalPrimitiveCount); break;
			case RTGT_Procedural:         DEC_DWORD_STAT_BY(STAT_D3D12RayTracingProceduralPrimitivesBLAS, Initializer.TotalPrimitiveCount); break;
			case RTGT_LinearSweptSpheres: DEC_DWORD_STAT_BY(STAT_D3D12RayTracingLinearSweptSpheresBLAS  , Initializer.TotalPrimitiveCount); break;
		}

		DEC_DWORD_STAT(STAT_D3D12RayTracingAllocatedBLAS);
	}

	// Reset members
	for (FPerGPUState& GPUState : PerGPUData)
	{
		GPUState.AccelerationStructureBuffer.SafeRelease();
		GPUState.HitGroupSystemParameters.Empty();
	}

	Initializer = {};

	AccelerationStructureCompactedSize = 0;

#if ENABLE_RESIDENCY_MANAGEMENT
	++ResourceGeneration;
#endif
}

FD3D12RayTracingGeometry::~FD3D12RayTracingGeometry()
{
	// RT geometry can be destroyed before persistent SBT records are cleared from the cached MDCs 
	// because they are still pending removal from the scene primitives
	for (ID3D12RayTracingGeometryUpdateListener* UpdateListener : UpdateListeners)
	{
		UpdateListener->RemoveListener(this);
	}
	UpdateListeners.Empty();

	ReleaseUnderlyingResource();
}

bool FD3D12RayTracingGeometry::AllocateBufferSRVs(uint32 InGPUIndex)
{
	bool bAnyBuffersUpdated = false;

	FPerGPUState& PerGPUState = PerGPUData[InGPUIndex];

	// Procedural doesn't need any SRVs for index buffer
	if (Initializer.IndexBuffer && (Initializer.GeometryType == RTGT_Triangles || Initializer.GeometryType == RTGT_LinearSweptSpheres)
#if !DO_CHECK
		// if checks are not enabled then simply early out when HitGroupSystemIndexBufferSRV is already created because we don't support updating them
		&& PerGPUState.HitGroupSystemIndexBufferSRV == nullptr		
#endif	
		)
	{
		checkf((Initializer.IndexBufferOffset % RHI_RAW_VIEW_ALIGNMENT) == 0, TEXT("The byte offset of raw views must be a multiple of %d (specified offset: %d)."), RHI_RAW_VIEW_ALIGNMENT, Initializer.IndexBufferOffset);
		
		FD3D12Buffer* IndexBuffer = FD3D12DynamicRHI::ResourceCast(Initializer.IndexBuffer.GetReference());

		// Initializer.TotalPrimitiveCount is the accumulated num primitives of the segments
		// The highest indexed entry can be higher due to Segment.FirstPrimitive or it can be lower if segments overlap
		// So here we calculate the highest indexed entry by looping over the segments
		uint32 MaxPrimitiveCount = 0;
		for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
		{
			MaxPrimitiveCount = FMath::Max(MaxPrimitiveCount, Segment.FirstPrimitive + Segment.NumPrimitives);
		}

		const uint32 PerPrimitiveIndexCount = (Initializer.GeometryType == RTGT_Triangles) ? IndicesPerTriangle : IndicesPerLinearSweptSphere;

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
		SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Buffer.FirstElement = (Initializer.IndexBufferOffset + IndexBuffer->ResourceLocation.GetOffsetFromBaseOfResource()) >> 2u;
		SRVDesc.Buffer.NumElements = FMath::Max((uint32)1, ((MaxPrimitiveCount * PerPrimitiveIndexCount * IndexBuffer->GetStride()) + 3) >> 2u);
		SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;
		SRVDesc.Buffer.StructureByteStride = 0;

#if DO_CHECK
		if (PerGPUState.HitGroupSystemIndexBufferSRV)
		{
			// Validate that current index buffer view hasn't changed
			D3D12_SHADER_RESOURCE_VIEW_DESC CurrentSRVDesc = PerGPUState.HitGroupSystemIndexBufferSRV->GetD3DDesc();
			check(SRVDesc.Buffer.FirstElement == CurrentSRVDesc.Buffer.FirstElement &&
				SRVDesc.Buffer.NumElements == CurrentSRVDesc.Buffer.NumElements &&
				IndexBuffer == PerGPUState.HitGroupSystemIndexBufferSRV->GetShaderResource());
		}
		else
#endif // DO_CHECK
		{
			PerGPUState.HitGroupSystemIndexBufferSRV = MakeShared<FD3D12ShaderResourceView>(GetParentAdapter()->GetDevice(InGPUIndex), InGPUIndex > 0 ? PerGPUData[0].HitGroupSystemIndexBufferSRV.Get() : nullptr, ERHIDescriptorType::BufferSRV);
			PerGPUState.HitGroupSystemIndexBufferSRV->CreateView(IndexBuffer, SRVDesc, FD3D12ShaderResourceView::EFlags::None);
		
			bAnyBuffersUpdated = true;
		}
	}

	if (PerGPUState.HitGroupSystemSegmentVertexBufferSRVs.Num() != Initializer.Segments.Num())
	{		
		PerGPUState.HitGroupSystemSegmentVertexBufferSRVs.SetNum(Initializer.Segments.Num());
	}

	for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
	{
		const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];
		checkf((Segment.VertexBufferOffset % RHI_RAW_VIEW_ALIGNMENT) == 0, TEXT("The byte offset of raw views must be a multiple of %d (specified offset: %d)."), RHI_RAW_VIEW_ALIGNMENT, Segment.VertexBufferOffset);

		FD3D12Buffer* VertexBuffer = FD3D12DynamicRHI::ResourceCast(Segment.VertexBuffer.GetReference());

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
		SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Buffer.FirstElement = (Segment.VertexBufferOffset + VertexBuffer->ResourceLocation.GetOffsetFromBaseOfResource()) >> 2u;
		if (Initializer.GeometryType == RTGT_Procedural)
		{
			SRVDesc.Buffer.NumElements = ((Segment.NumPrimitives * Segment.VertexBufferStride) + 3) / 4; //< NumElements in R32 size
		}
		else
		{
			SRVDesc.Buffer.NumElements = FMath::Max((uint32)1, ((Segment.MaxVertices * Segment.VertexBufferStride) + 3) / 4); //< NumElements in R32 size
		}
		SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;
		SRVDesc.Buffer.StructureByteStride = 0;

		bool bCreateNewSRV = true;
		FD3D12ShaderResourceView* CurrentVertexBufferSRV = SegmentIndex < PerGPUState.HitGroupSystemSegmentVertexBufferSRVs.Num() ? PerGPUState.HitGroupSystemSegmentVertexBufferSRVs[SegmentIndex].Get() : nullptr;
		if (CurrentVertexBufferSRV)
		{
			D3D12_SHADER_RESOURCE_VIEW_DESC CurrentSRVDesc = CurrentVertexBufferSRV->GetD3DDesc();
			if (SRVDesc.Buffer.FirstElement == CurrentSRVDesc.Buffer.FirstElement &&
				SRVDesc.Buffer.NumElements == CurrentSRVDesc.Buffer.NumElements &&
				CurrentVertexBufferSRV->GetShaderResource() == VertexBuffer)
			{
				bCreateNewSRV = false;
			}
		}

		if (bCreateNewSRV)
		{
			FD3D12ShaderResourceView* FirstLinkedObject = nullptr;
			if (InGPUIndex > 0)
			{
				if (PerGPUData[0].HitGroupSystemSegmentVertexBufferSRVs.Num() > SegmentIndex)
				{
					FirstLinkedObject = PerGPUData[0].HitGroupSystemSegmentVertexBufferSRVs[SegmentIndex].Get();
				}
			}
			TSharedPtr<FD3D12ShaderResourceView> VertexBufferSRV = MakeShared<FD3D12ShaderResourceView>(GetParentAdapter()->GetDevice(InGPUIndex), FirstLinkedObject, ERHIDescriptorType::BufferSRV);
			VertexBufferSRV->CreateView(VertexBuffer, SRVDesc, FD3D12ShaderResourceView::EFlags::None);
			PerGPUState.HitGroupSystemSegmentVertexBufferSRVs[SegmentIndex] = VertexBufferSRV;

			bAnyBuffersUpdated = true;
		}
	}

	return bAnyBuffersUpdated;
}

void FD3D12RayTracingGeometry::RegisterAsRenameListener(uint32 InGPUIndex)
{
#if !ENABLE_RESIDENCY_MANAGEMENT
	if (AreBindlessResourcesEnabledForRayTracing(GetParentAdapter()))
	{
		return;
	}
#endif

	FPerGPUState& PerGPUState = PerGPUData[InGPUIndex];
	
	check(!PerGPUState.bRegisteredAsRenameListener);

	FD3D12Buffer* IndexBuffer = FD3D12DynamicRHI::ResourceCast(Initializer.IndexBuffer.GetReference(), InGPUIndex);
	if (IndexBuffer)
	{
		IndexBuffer->AddRenameListener(this);
	}

	TArray<FD3D12Buffer*, TInlineAllocator<1>> UniqueVertexBuffers;
	UniqueVertexBuffers.Reserve(Initializer.Segments.Num());
	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		FD3D12Buffer* VertexBuffer = FD3D12DynamicRHI::ResourceCast(Segment.VertexBuffer.GetReference(), InGPUIndex);
		if (VertexBuffer && !UniqueVertexBuffers.Contains(VertexBuffer))
		{
			VertexBuffer->AddRenameListener(this);
			UniqueVertexBuffers.Add(VertexBuffer);
		}
	}

	PerGPUState.bRegisteredAsRenameListener = true;
}

void FD3D12RayTracingGeometry::UnregisterAsRenameListener(uint32 InGPUIndex)
{
	FPerGPUState& PerGPUState = PerGPUData[InGPUIndex];

	if (!PerGPUState.bRegisteredAsRenameListener)
	{
		return;
	}

#if !ENABLE_RESIDENCY_MANAGEMENT
	check(!AreBindlessResourcesEnabledForRayTracing(GetParentAdapter()));
#endif

	FD3D12Buffer* IndexBuffer = FD3D12DynamicRHI::ResourceCast(Initializer.IndexBuffer.GetReference(), InGPUIndex);
	if (IndexBuffer)
	{
		IndexBuffer->RemoveRenameListener(this);
	}

	TArray<FD3D12Buffer*, TInlineAllocator<1>> UniqueVertexBuffers;
	UniqueVertexBuffers.Reserve(Initializer.Segments.Num());
	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		FD3D12Buffer* VertexBuffer = FD3D12DynamicRHI::ResourceCast(Segment.VertexBuffer.GetReference(), InGPUIndex);
		if (VertexBuffer && !UniqueVertexBuffers.Contains(VertexBuffer))
		{
			VertexBuffer->RemoveRenameListener(this);
			UniqueVertexBuffers.Add(VertexBuffer);
		}
	}

	PerGPUState.bRegisteredAsRenameListener = false;
}

void FD3D12RayTracingGeometry::ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation)
{
	// Empty resource location is used on destruction of the base shader resource but this
	// shouldn't happen for RT Geometries because it keeps smart pointers to it's resources.
	check(InNewResourceLocation != nullptr);

#if !ENABLE_RESIDENCY_MANAGEMENT
	check(!AreBindlessResourcesEnabledForRayTracing(GetParentAdapter()));
#endif

	// Recreate the hit group parameters which cache the address to the index and vertex buffers directly if the geometry is fully valid
	uint32 GPUIndex = InRenamedResource->GetParentDevice()->GetGPUIndex();
	if (!AreBindlessResourcesEnabledForRayTracing(GetParentAdapter()))
	{
		if (BuffersValid(GPUIndex))
		{
			SetupHitGroupSystemParameters(GPUIndex);
		}
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	++ResourceGeneration;
#endif
}

bool FD3D12RayTracingGeometry::BuffersValid(uint32 GPUIndex) const
{
	if (Initializer.IndexBuffer)
	{
		const FD3D12Buffer* IndexBuffer = FD3D12DynamicRHI::ResourceCast(Initializer.IndexBuffer.GetReference(), GPUIndex);
		if (!IndexBuffer->ResourceLocation.IsValid())
		{
			return false;
		}
	}

	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		const FD3D12Buffer* VertexBuffer = FD3D12DynamicRHI::ResourceCast(Segment.VertexBuffer.GetReference(), GPUIndex);
		if (!VertexBuffer->ResourceLocation.IsValid())
		{
			return false;
		}
	}

	return true;
}

void FD3D12RayTracingGeometry::UpdateResidency(FD3D12CommandContext& CommandContext)
{
	if (Initializer.IndexBuffer)
	{
		FD3D12Buffer* IndexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Initializer.IndexBuffer.GetReference());
		CommandContext.UpdateResidency(IndexBuffer->GetResource());
	}

	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		const FBufferRHIRef& RHIVertexBuffer = Segment.VertexBuffer;
		FD3D12Buffer* VertexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(RHIVertexBuffer.GetReference());		
		CommandContext.UpdateResidency(VertexBuffer->ResourceLocation.GetResource());
	}

	const uint32 GPUIndex = CommandContext.GetGPUIndex();
	CommandContext.UpdateResidency(PerGPUData[GPUIndex].AccelerationStructureBuffer->GetResource());
}

void FD3D12RayTracingGeometry::SetupHitGroupSystemParameters(uint32 InGPUIndex)
{
	bool bBindless = AreBindlessResourcesEnabledForRayTracing(GetParentAdapter());

	bool bAnyBuffersUpdated = true;
	check(BuffersValid(InGPUIndex));
	if (bBindless)
	{
		bAnyBuffersUpdated = AllocateBufferSRVs(InGPUIndex);
	}

	if (bAnyBuffersUpdated)
	{
		FPerGPUState& PerGPUState = PerGPUData[InGPUIndex];

		TArray<FD3D12HitGroupSystemParameters>& HitGroupSystemParametersForThisGPU = PerGPUState.HitGroupSystemParameters;
		HitGroupSystemParametersForThisGPU.Reset(Initializer.Segments.Num());

		FD3D12Buffer* IndexBuffer = FD3D12DynamicRHI::ResourceCast(Initializer.IndexBuffer.GetReference(), InGPUIndex);
		const uint32 IndexStride = IndexBuffer ? IndexBuffer->GetStride() : 0;
		const uint32 LocalIndicesPerPrimitive = Initializer.GeometryType == RTGT_LinearSweptSpheres ? IndicesPerLinearSweptSphere : IndicesPerTriangle;
		for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
		{
			const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];
			FD3D12Buffer* VertexBuffer = FD3D12DynamicRHI::ResourceCast(Segment.VertexBuffer.GetReference(), InGPUIndex);

			FD3D12HitGroupSystemParameters SystemParameters = {};
			SystemParameters.RootConstants.SetVertexAndIndexStride(Segment.VertexBufferStride, IndexStride);
			if (bBindless)
			{
				SystemParameters.BindlessHitGroupSystemVertexBuffer = PerGPUState.HitGroupSystemSegmentVertexBufferSRVs[SegmentIndex]->GetBindlessHandle().GetIndex();
			}
			else
			{
				SystemParameters.VertexBuffer = VertexBuffer->ResourceLocation.GetGPUVirtualAddress() + Segment.VertexBufferOffset;
			}

			if (IndexBuffer != nullptr)
			{
				if (bBindless)
				{
					SystemParameters.BindlessHitGroupSystemIndexBuffer = PerGPUState.HitGroupSystemIndexBufferSRV->GetBindlessHandle().GetIndex();
				}
				else
				{
					SystemParameters.IndexBuffer = IndexBuffer->ResourceLocation.GetGPUVirtualAddress();
				}
				SystemParameters.RootConstants.IndexBufferOffsetInBytes = Initializer.IndexBufferOffset + IndexStride * Segment.FirstPrimitive * LocalIndicesPerPrimitive;
				SystemParameters.RootConstants.FirstPrimitive = Segment.FirstPrimitive;
			}

			HitGroupSystemParametersForThisGPU.Add(SystemParameters);
		}
		
		// Notify listeners about changes
		HitGroupParamatersUpdated();
	}
}

void FD3D12RayTracingGeometry::CreateAccelerationStructureBuildDesc(FD3D12CommandContext& CommandContext, EAccelerationStructureBuildMode BuildMode, D3D12_GPU_VIRTUAL_ADDRESS ScratchBufferAddress, D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& OutDesc, TArrayView<D3D12_RAYTRACING_GEOMETRY_DESC>& OutGeometryDescs) const
{
	check(!GeometryUsesLinearSweptSpheres(Initializer));

	if (Initializer.IndexBuffer)
	{
		checkf(Initializer.IndexBuffer->GetStride() == 2 || Initializer.IndexBuffer->GetStride() == 4, TEXT("Index buffer must be 16 or 32 bit."));
	}

	const uint32 GPUIndex = CommandContext.GetGPUIndex();
	const uint32 IndexStride = Initializer.IndexBuffer ? Initializer.IndexBuffer->GetStride() : 0;
	const bool bIsUpdate = BuildMode == EAccelerationStructureBuildMode::Update;

	// Make sure we have the right amount of space reserved
	check(OutGeometryDescs.Num() == Initializer.Segments.Num());
	checkf(BuffersValid(GPUIndex), TEXT("Index & vertex buffers should be valid (not streamed out) when building the acceleration structure '%s' (owner '%s')"),
		DebugName.IsNone() ? TEXT("UNKNOWN") : *DebugName.ToString(),
		OwnerName.IsNone() ? TEXT("UNKNOWN") : *OwnerName.ToString());

	FD3D12Buffer* IndexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Initializer.IndexBuffer.GetReference());
		
	const TArray<FD3D12HitGroupSystemParameters>& HitGroupSystemParametersForThisGPU = PerGPUData[GPUIndex].HitGroupSystemParameters;
	check(HitGroupSystemParametersForThisGPU.Num() == Initializer.Segments.Num());

	ERayTracingAccelerationStructureFlags BuildFlags = GetRayTracingAccelerationStructureBuildFlags(Initializer);

	// Fill in the GeometryDesc
	TranslateRayTracingGeometryDescs(Initializer, OutGeometryDescs);

	// second pass to actually set the current VB/IB buffer addresses
	for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
	{
		D3D12_RAYTRACING_GEOMETRY_DESC& Desc = OutGeometryDescs[SegmentIndex];

		const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];
		const FD3D12HitGroupSystemParameters& SystemParameters = HitGroupSystemParametersForThisGPU[SegmentIndex];
		
		FD3D12Buffer* VertexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Segment.VertexBuffer.GetReference());

		switch (Initializer.GeometryType)
		{
		case RTGT_Triangles:
			check(Desc.Triangles.VertexFormat == TranslateRayTracingVertexElementType(Segment.VertexBufferElementType));
			// #dxr_todo UE-72160: support various vertex buffer layouts (fetch/decode based on vertex stride and format)
			checkf(Segment.VertexBufferElementType == VET_Float3 ||
				   Segment.VertexBufferElementType == VET_Float4, TEXT("Only VET_Float3 and Float4 are currently implemented and tested. Other formats will be supported in the future."));

			if (!Segment.bEnabled)
			{
				Desc.Triangles.IndexCount = 0;
			}

			checkf(Desc.Triangles.Transform3x4 == D3D12_GPU_VIRTUAL_ADDRESS(0), TEXT("BLAS geometry transforms are not supported!"));

			if (IndexBuffer)
			{
				check(Desc.Triangles.IndexCount <= Segment.NumPrimitives * FD3D12RayTracingGeometry::IndicesPerTriangle);

				Desc.Triangles.IndexFormat = (IndexStride == 4 ? DXGI_FORMAT_R32_UINT : DXGI_FORMAT_R16_UINT);
				Desc.Triangles.IndexBuffer = IndexBuffer->ResourceLocation.GetGPUVirtualAddress() + SystemParameters.RootConstants.IndexBufferOffsetInBytes;
			}
			else
			{
				// Non-indexed geometry
				checkf(Initializer.Segments.Num() == 1, TEXT("Non-indexed geometry with multiple segments is not implemented."));
				check(Desc.Triangles.IndexFormat == DXGI_FORMAT_UNKNOWN);
				check(Desc.Triangles.IndexCount == 0);
				check(Desc.Triangles.IndexBuffer == D3D12_GPU_VIRTUAL_ADDRESS(0));
			}

			Desc.Triangles.VertexBuffer.StartAddress = VertexBuffer->ResourceLocation.GetGPUVirtualAddress() + Segment.VertexBufferOffset;
			Desc.Triangles.VertexBuffer.StrideInBytes = Segment.VertexBufferStride;
			break;

		case RTGT_Procedural:
			Desc.AABBs.AABBCount = Segment.NumPrimitives;
			Desc.AABBs.AABBs.StartAddress = VertexBuffer->ResourceLocation.GetGPUVirtualAddress() + Segment.VertexBufferOffset;
			Desc.AABBs.AABBs.StrideInBytes = Segment.VertexBufferStride;
			break;

		default:
			checkf(false, TEXT("Unexpected ray tracing geometry type"));
			break;
		}
	}

	D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS LocalBuildFlags = TranslateRayTracingAccelerationStructureFlags(BuildFlags);

	if (bIsUpdate)
	{
		checkf(EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate),
			TEXT("Acceleration structure must be created with FRayTracingGeometryInitializer::bAllowUpdate=true to perform refit / update."));

		LocalBuildFlags |= D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_PERFORM_UPDATE;
	}

	D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS PrebuildDescInputs = {};

	PrebuildDescInputs.Type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL;
	PrebuildDescInputs.DescsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
	PrebuildDescInputs.NumDescs = OutGeometryDescs.Num();
	PrebuildDescInputs.pGeometryDescs = OutGeometryDescs.GetData();
	PrebuildDescInputs.Flags = LocalBuildFlags;

#if DO_CHECK
	D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO PrebuildInfo = {};

	CommandContext.GetParentDevice()->GetRaytracingAccelerationStructurePrebuildInfo(&PrebuildDescInputs, &PrebuildInfo);

	// Must make sure that values computed in the constructor are valid.
	check(PrebuildInfo.ResultDataMaxSizeInBytes <= SizeInfo.ResultSize);

	if (bIsUpdate)
	{
		check(PrebuildInfo.UpdateScratchDataSizeInBytes <= SizeInfo.UpdateScratchSize);
	}
	else
	{
		check(PrebuildInfo.ScratchDataSizeInBytes <= SizeInfo.BuildScratchSize);
	}
#endif // DO_CHECK

	D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC BuildDesc = {};
	BuildDesc.Inputs = PrebuildDescInputs;
	BuildDesc.DestAccelerationStructureData = PerGPUData[GPUIndex].AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress();
	BuildDesc.ScratchAccelerationStructureData = ScratchBufferAddress;
	BuildDesc.SourceAccelerationStructureData = bIsUpdate
		? PerGPUData[GPUIndex].AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress()
		: D3D12_GPU_VIRTUAL_ADDRESS(0);

	OutDesc = BuildDesc;
}

#if WITH_NVAPI
void FD3D12RayTracingGeometry::CreateAccelerationStructureBuildDescEx(FD3D12CommandContext& CommandContext, EAccelerationStructureBuildMode BuildMode, D3D12_GPU_VIRTUAL_ADDRESS ScratchBufferAddress, NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX& OutDescEx, TArrayView<NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX>& OutGeometryDescsEx) const
{
	check(GeometryUsesLinearSweptSpheres(GetInitializer()));
	if (Initializer.IndexBuffer)
	{
		checkf(Initializer.IndexBuffer->GetStride() == 2 || Initializer.IndexBuffer->GetStride() == 4, TEXT("Index buffer must be 16 or 32 bit."));
	}

	const uint32 GPUIndex = CommandContext.GetGPUIndex();
	const uint32 IndexStride = Initializer.IndexBuffer ? Initializer.IndexBuffer->GetStride() : 0;
	const bool bIsUpdate = BuildMode == EAccelerationStructureBuildMode::Update;

	// Make sure we've reserved the right amount of space
	check(OutGeometryDescsEx.Num() == Initializer.Segments.Num());
	checkf(BuffersValid(GPUIndex), TEXT("Index & vertex buffers should be valid (not streamed out) when building the acceleration structure '%s' (owner '%s')"),
		DebugName.IsNone() ? TEXT("UNKNOWN") : *DebugName.ToString(),
		OwnerName.IsNone() ? TEXT("UNKNOWN") : *OwnerName.ToString());

	FD3D12Buffer* IndexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Initializer.IndexBuffer.GetReference());
		
	const TArray<FD3D12HitGroupSystemParameters>& HitGroupSystemParametersForThisGPU = PerGPUData[GPUIndex].HitGroupSystemParameters;
	check(HitGroupSystemParametersForThisGPU.Num() == Initializer.Segments.Num());

	TranslateRayTracingGeometryDescsEx(Initializer, OutGeometryDescsEx);

	ERayTracingAccelerationStructureFlags BuildFlags = GetRayTracingAccelerationStructureBuildFlags(Initializer);
	for (int32 SegmentIndex = 0; SegmentIndex < Initializer.Segments.Num(); ++SegmentIndex)
	{
		NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX& Desc = OutGeometryDescsEx[SegmentIndex];
		const FRayTracingGeometrySegment& Segment = Initializer.Segments[SegmentIndex];
		const FD3D12HitGroupSystemParameters& SystemParameters = HitGroupSystemParametersForThisGPU[SegmentIndex];

		FD3D12Buffer* VertexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Segment.VertexBuffer.GetReference());

		check(Desc.type == NVAPI_D3D12_RAYTRACING_GEOMETRY_TYPE_LSS_EX);

		Desc.lss.indexCount = Segment.bEnabled ? Segment.NumPrimitives * IndicesPerLinearSweptSphere : 0;

		check(IndexBuffer != nullptr);
		
		Desc.lss.indexFormat = (IndexStride == 4 ? DXGI_FORMAT_R32_UINT : DXGI_FORMAT_R16_UINT);
		Desc.lss.indexBuffer.StartAddress = IndexBuffer->ResourceLocation.GetGPUVirtualAddress() + SystemParameters.RootConstants.IndexBufferOffsetInBytes;
		Desc.lss.indexBuffer.StrideInBytes = sizeof(uint32);
		Desc.lss.vertexPositionBuffer.StartAddress = VertexBuffer->ResourceLocation.GetGPUVirtualAddress() + Segment.VertexBufferOffset;
		Desc.lss.vertexPositionBuffer.StrideInBytes = Segment.VertexBufferStride;
		Desc.lss.vertexRadiusBuffer.StartAddress = Desc.lss.vertexPositionBuffer.StartAddress + 3 * sizeof(float);
		Desc.lss.vertexRadiusBuffer.StrideInBytes = Segment.VertexBufferStride;
	}

	NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS_EX LocalBuildFlagsEx = static_cast<NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAGS_EX>(TranslateRayTracingAccelerationStructureFlags(BuildFlags));

	if (bIsUpdate)
	{
		checkf(EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate),
			   TEXT("Acceleration structure must be created with FRayTracingGeometryInitializer::bAllowUpdate=true to perform refit / update."));

		LocalBuildFlagsEx |= NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_PERFORM_UPDATE_EX;
	}

#if DO_CHECK
	D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO PrebuildInfo = {};

	NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS_EX InputsEx = {};
	InputsEx.type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL;
	InputsEx.flags = LocalBuildFlagsEx;
	InputsEx.descsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
	InputsEx.geometryDescStrideInBytes = sizeof(NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX);
	InputsEx.instanceDescs = 0ULL;
	InputsEx.numDescs = OutGeometryDescsEx.Num();
	InputsEx.pGeometryDescs = OutGeometryDescsEx.GetData();

	NVAPI_GET_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO_EX_PARAMS PrebuildParams = {};
	PrebuildParams.version = NVAPI_GET_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO_EX_PARAMS_VER;
	PrebuildParams.pDesc = &InputsEx;
	PrebuildParams.pInfo = &PrebuildInfo;

	ID3D12Device5* RayTracingDevice = CommandContext.GetParentDevice()->GetParentAdapter()->GetD3DDevice5();
	NvAPI_Status status = NvAPI_D3D12_GetRaytracingAccelerationStructurePrebuildInfoEx(RayTracingDevice, &PrebuildParams);
	check(status == NVAPI_OK);

	// Must make sure that values computed in the constructor are valid.
	check(PrebuildInfo.ResultDataMaxSizeInBytes <= SizeInfo.ResultSize);

	if (bIsUpdate)
	{
		check(PrebuildInfo.UpdateScratchDataSizeInBytes <= SizeInfo.UpdateScratchSize);
	}
	else
	{
		check(PrebuildInfo.ScratchDataSizeInBytes <= SizeInfo.BuildScratchSize);
	}
#endif // DO_CHECK

	OutDescEx.inputs.flags = LocalBuildFlagsEx;
	OutDescEx.inputs.type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_BOTTOM_LEVEL;
	OutDescEx.inputs.numDescs = OutGeometryDescsEx.Num();
	OutDescEx.inputs.descsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
	OutDescEx.inputs.geometryDescStrideInBytes = sizeof(NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX);
	OutDescEx.inputs.pGeometryDescs = OutGeometryDescsEx.GetData();

	OutDescEx.destAccelerationStructureData = PerGPUData[GPUIndex].AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress();
	OutDescEx.scratchAccelerationStructureData = ScratchBufferAddress;
	OutDescEx.sourceAccelerationStructureData = bIsUpdate
		? PerGPUData[GPUIndex].AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress()
		: D3D12_GPU_VIRTUAL_ADDRESS(0);
}
#endif

static bool ShouldCompactAfterBuild(ERayTracingAccelerationStructureFlags BuildFlags)
{
	return EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction | ERayTracingAccelerationStructureFlags::FastTrace)
		&& !EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
}

void FD3D12RayTracingGeometry::CompactAccelerationStructure(FD3D12CommandContext& CommandContext, uint32 InGPUIndex, uint64 InSizeAfterCompaction)
{
	LLM_SCOPE_BYNAME(TEXT("FD3D12RT/CompactBLAS"));

	FPerGPUState& PerGPUState = PerGPUData[InGPUIndex];

	// Should have a pending request
	check(PerGPUState.bHasPendingCompactionRequests);
	PerGPUState.bHasPendingCompactionRequests = false;

	ensureMsgf(InSizeAfterCompaction > 0, TEXT("Compacted acceleration structure size is expected to be non-zero. This error suggests that GPU readback synchronization is broken."));
	if (InSizeAfterCompaction == 0)
	{
		return;
	}

	DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
	DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
	DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingStaticBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());

	UnregisterD3D12RayTracingGeometry(this);

	// Move old AS into this temporary variable which gets released when this function returns
	TRefCountPtr<FD3D12Buffer> OldAccelerationStructure = PerGPUState.AccelerationStructureBuffer;

	PerGPUState.AccelerationStructureBuffer = CreateRayTracingBuffer(CommandContext.GetParentAdapter(), InGPUIndex, InSizeAfterCompaction, ERayTracingBufferType::AccelerationStructure, DebugName);
	PerGPUState.AccelerationStructureBuffer->SetOwnerName(OwnerName);

	INC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
	INC_MEMORY_STAT_BY(STAT_D3D12RayTracingBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
	INC_MEMORY_STAT_BY(STAT_D3D12RayTracingStaticBLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());

	CommandContext.UpdateResidency(OldAccelerationStructure->GetResource());
	CommandContext.UpdateResidency(PerGPUState.AccelerationStructureBuffer->GetResource());

	CommandContext.RayTracingCommandList()->CopyRaytracingAccelerationStructure(
		PerGPUState.AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress(),
		OldAccelerationStructure->ResourceLocation.GetGPUVirtualAddress(),
		D3D12_RAYTRACING_ACCELERATION_STRUCTURE_COPY_MODE_COMPACT
	);

	AccelerationStructureCompactedSize = InSizeAfterCompaction;

	RegisterD3D12RayTracingGeometry(this);

#if ENABLE_RESIDENCY_MANAGEMENT
	++ResourceGeneration;
#endif
}

FD3D12RayTracingScene::FD3D12RayTracingScene(FD3D12Adapter* Adapter, FRayTracingSceneInitializer InInitializer)
	: FD3D12AdapterChild(Adapter), Initializer(MoveTemp(InInitializer))
{
	PerGPUData.SetNum(GNumExplicitGPUsForRendering);

	INC_DWORD_STAT(STAT_D3D12RayTracingAllocatedTLAS);

	checkf(Initializer.Lifetime == RTSL_SingleFrame, TEXT("Only single-frame ray tracing scenes are currently implemented."));

	// Get maximum buffer sizes for all GPUs in the system
	SizeInfo = RHICalcRayTracingSceneSize(Initializer);
};

FD3D12RayTracingScene::~FD3D12RayTracingScene()
{
	ReleaseBuffer();

	DEC_DWORD_STAT(STAT_D3D12RayTracingAllocatedTLAS);
}

void FD3D12RayTracingScene::ReleaseBuffer()
{
	for (FPerGPUState& GPUState : PerGPUData)
	{
		if (GPUState.AccelerationStructureBuffer)
		{
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, GPUState.AccelerationStructureBuffer->GetSize());
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingTLASMemory, GPUState.AccelerationStructureBuffer->GetSize());
		}

		GPUState.AccelerationStructureBuffer = nullptr;
	}
}

void FD3D12RayTracingScene::BindBuffer(FRHIBuffer* InBuffer, uint32 InBufferOffset)
{
	check(SizeInfo.ResultSize + InBufferOffset <= InBuffer->GetSize());

	FOREACH_GPU(GPUIndex < (uint32)PerGPUData.Num(),
	{
		FPerGPUState& PerGPUState = PerGPUData[GPUIndex];

		if (PerGPUState.AccelerationStructureBuffer)
		{
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
			DEC_MEMORY_STAT_BY(STAT_D3D12RayTracingTLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		}

		PerGPUState.AccelerationStructureBuffer = FD3D12CommandContext::RetrieveObject<FD3D12Buffer>(InBuffer, GPUIndex);

		INC_MEMORY_STAT_BY(STAT_D3D12RayTracingUsedVideoMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
		INC_MEMORY_STAT_BY(STAT_D3D12RayTracingTLASMemory, PerGPUState.AccelerationStructureBuffer->GetSize());
	});

	BufferOffset = InBufferOffset;
}

#if ENABLE_RESIDENCY_MANAGEMENT
void FD3D12RayTracingScene::UpdateResidencyTracking(FD3D12CommandContext& CommandContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateTLASResidency);

	const uint32 GPUIndex = CommandContext.GetGPUIndex();

	FPerGPUState& GPUState = PerGPUData[GPUIndex];

	// Build lookup set of current geometries
	TSet<FRHIRayTracingGeometry*> CurrentGeometrySet;
	CurrentGeometrySet.Reserve(ReferencedGeometries.Num());
	for (const TRefCountPtr<FRHIRayTracingGeometry>& GeomRef : ReferencedGeometries)
	{
		CurrentGeometrySet.Add(GeomRef.GetReference());
	}

	// Helpers: increment/decrement ref counts for a resource array
	auto IncrementRefCounts = [](TMap<const FD3D12Resource*, int32>& RefCounts, TConstArrayView<const FD3D12Resource*> Resources)
		{
			for (const FD3D12Resource* Resource : Resources)
			{
				++RefCounts.FindOrAdd(Resource, 0);
			}
		};

	auto DecrementRefCounts = [](TMap<const FD3D12Resource*, int32>& RefCounts, TConstArrayView<const FD3D12Resource*> Resources)
		{
			for (const FD3D12Resource* Resource : Resources)
			{
				int32& RefCount = RefCounts.FindChecked(Resource);
				if (--RefCount == 0)
				{
					RefCounts.Remove(Resource);
				}
			}
		};

	// Remove stale entries from tracked geometries
	for (auto It = GPUState.TrackedGeometries.CreateIterator(); It; ++It)
	{
		if (!CurrentGeometrySet.Contains(It.Key()))
		{
			DecrementRefCounts(GPUState.ResourceRefCounts, MakeConstArrayView(&It.Value().CachedASResource, 1));
			DecrementRefCounts(GPUState.DispatchResourceRefCounts, It.Value().CachedDispatchResources);
			It.RemoveCurrent();
		}
	}

	// Helper: resolve a geometry's D3D12 resources for this GPU
	auto ResolveGeometryResources = [&CommandContext, GPUIndex](FD3D12RayTracingGeometry* Geometry,
			FGeometryResidencyInfo& OutResidencyInfo)
		{
			OutResidencyInfo.CachedASResource = Geometry->PerGPUData[GPUIndex].AccelerationStructureBuffer->GetResource();
			
			OutResidencyInfo.CachedDispatchResources.Reset();

			if (GRHIGlobals.RayTracing.SupportsShaders || GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT)
			{
				if (Geometry->Initializer.IndexBuffer)
				{
					FD3D12Buffer* IndexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Geometry->Initializer.IndexBuffer.GetReference());
					OutResidencyInfo.CachedDispatchResources.Add(IndexBuffer->GetResource());
				}

				for (const FRayTracingGeometrySegment& Segment : Geometry->Initializer.Segments)
				{
					if (Segment.VertexBuffer)
					{
						FD3D12Buffer* VertexBuffer = CommandContext.RetrieveObject<FD3D12Buffer>(Segment.VertexBuffer.GetReference());
						OutResidencyInfo.CachedDispatchResources.Add(VertexBuffer->GetResource());
					}
				}
			}
		};

	// Process additions and mutations
	for (int32 Index = 0; Index < ReferencedGeometries.Num(); ++Index)
	{
		FRHIRayTracingGeometry* GeometryRHI = ReferencedGeometries[Index];
		FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(GeometryRHI);

		checkf(!Geometry->IsDirty(GPUIndex),
			TEXT("Acceleration structures for all geometries must be built before building the top level acceleration structure for the scene."));

		if (GRHIGlobals.RayTracing.SupportsShaders || GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT)
		{
			checkf(Geometry->BuffersValid(GPUIndex),
				TEXT("Index and vertex buffers for all geometries must be valid (streamed in) when adding geometry to the top level acceleration structure for the scene"));
		}

		FGeometryResidencyInfo* ExistingInfo = GPUState.TrackedGeometries.Find(GeometryRHI);

		if (!ExistingInfo)
		{
			// Added geometry - resolve and track
			FGeometryResidencyInfo Info;
			Info.ResourceGeneration = Geometry->ResourceGeneration;
			ResolveGeometryResources(Geometry, Info);

			IncrementRefCounts(GPUState.ResourceRefCounts, MakeConstArrayView(&Info.CachedASResource, 1));
			IncrementRefCounts(GPUState.DispatchResourceRefCounts, Info.CachedDispatchResources);

			GPUState.TrackedGeometries.Add(GeometryRHI, MoveTemp(Info));
		}
		else if (ExistingInfo->ResourceGeneration != Geometry->ResourceGeneration)
		{
			// Mutated geometry - swap resources
			DecrementRefCounts(GPUState.ResourceRefCounts, MakeConstArrayView(&ExistingInfo->CachedASResource, 1));
			DecrementRefCounts(GPUState.DispatchResourceRefCounts, ExistingInfo->CachedDispatchResources);

			ExistingInfo->ResourceGeneration = Geometry->ResourceGeneration;
			ResolveGeometryResources(Geometry, *ExistingInfo);

			IncrementRefCounts(GPUState.ResourceRefCounts, MakeConstArrayView(&ExistingInfo->CachedASResource, 1));
			IncrementRefCounts(GPUState.DispatchResourceRefCounts, ExistingInfo->CachedDispatchResources);
		}
		// else: unchanged - skip entirely
	}

	// Rebuild flat arrays from tracked resources + transient referenced buffers, deduplicating by residency handle to minimize UpdateResidency calls.
	{
		Experimental::TSherwoodSet<FD3D12ResidencyHandle*> UniqueResidencyHandles;

		auto AddUniqueResource = [&UniqueResidencyHandles](const FD3D12Resource* Resource, TArray<const FD3D12Resource*>& OutResources)
			{
				bool bShouldTrackResidency = false;

				if (Resource->NeedsDeferredResidencyUpdate())
				{
					// Resources whose residency handles might change dynamically must always be tracked
					bShouldTrackResidency = true;
				}
				else
				{
					// Resources that share *all* residency handles with what's already tracked don't need to be added to be tracked separately
					for (FD3D12ResidencyHandle* ResidencyHandle : Resource->GetResidencyHandles())
					{
						if (D3DX12Residency::IsInitialized(ResidencyHandle))
						{
							bool bIsAlreadyInSet = false;
							UniqueResidencyHandles.Add(ResidencyHandle, &bIsAlreadyInSet);
							if (!bIsAlreadyInSet)
							{
								bShouldTrackResidency = true;
							}
						}
					}
				}

				if (bShouldTrackResidency)
				{
					OutResources.Add(Resource);
				}
			};

		GPUState.ResourcesToMakeResident.Reset(GPUState.ResourceRefCounts.Num() + ReferencedBuffers.Num());
		for (const auto& Pair : GPUState.ResourceRefCounts)
		{
			AddUniqueResource(Pair.Key, GPUState.ResourcesToMakeResident);
		}

		for (FRHIBuffer* ReferencedBuffer : ReferencedBuffers)
		{
			FD3D12Buffer* Buffer = CommandContext.RetrieveObject<FD3D12Buffer>(ReferencedBuffer);
			AddUniqueResource(Buffer->GetResource(), GPUState.ResourcesToMakeResident);
		}

		GPUState.DispatchResourcesToMakeResident.Reset(GPUState.DispatchResourceRefCounts.Num());
		for (const auto& Pair : GPUState.DispatchResourceRefCounts)
		{
			AddUniqueResource(Pair.Key, GPUState.DispatchResourcesToMakeResident);
		}
	}
}
#endif // ENABLE_RESIDENCY_MANAGEMENT

void FD3D12RayTracingScene::UpdateResidency(FD3D12CommandContext& CommandContext, bool bIncludeDispatchResources) const
{
#if ENABLE_RESIDENCY_MANAGEMENT
	const uint32 GPUIndex = CommandContext.GetGPUIndex();
	const FPerGPUState& PerGPUState = PerGPUData[GPUIndex];
	CommandContext.UpdateResidency(PerGPUState.AccelerationStructureBuffer->GetResource());
	for (const FD3D12Resource* Resource : PerGPUState.ResourcesToMakeResident)
	{
		CommandContext.UpdateResidency(Resource);
	}
	if (bIncludeDispatchResources)
	{
		for (const FD3D12Resource* Resource : PerGPUState.DispatchResourcesToMakeResident)
		{
			CommandContext.UpdateResidency(Resource);
		}
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

void PrepareAccelerationStructureBuild(
	FD3D12CommandContext& CommandContext,
	FD3D12RayTracingScene& Scene,
	FD3D12Buffer* ScratchBuffer, uint32 ScratchBufferOffset,
	FD3D12Buffer* InstanceBuffer, uint32 InstanceBufferOffset,
	uint32 NumInstances,
	EAccelerationStructureBuildMode BuildMode,
	D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& OutBuildDesc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PrepareAccelerationStructureBuild_TopLevel);

	check(InstanceBuffer != nullptr);
	checkf(NumInstances <= Scene.Initializer.MaxNumInstances, TEXT("NumInstances must be less or equal to MaxNumInstances"));

	const bool bIsUpdate = BuildMode == EAccelerationStructureBuildMode::Update;

	if (bIsUpdate)
	{
		checkf(NumInstances == Scene.NumInstances, TEXT("Number of instances used to update TLAS must match the number used to build."));
	}
	else
	{
		Scene.NumInstances = NumInstances;
	}

	const uint32 GPUIndex = CommandContext.GetGPUIndex();
	FD3D12Adapter* Adapter = CommandContext.GetParentAdapter();

	TRefCountPtr<FD3D12Buffer> AutoScratchBuffer;
	if (ScratchBuffer == nullptr)
	{
		const uint64 ScratchBufferSize = bIsUpdate ? Scene.SizeInfo.UpdateScratchSize : Scene.SizeInfo.BuildScratchSize;

		static const FName ScratchBufferName("AutoBuildScratchTLAS");
		AutoScratchBuffer = CreateRayTracingBuffer(Adapter, GPUIndex, ScratchBufferSize, ERayTracingBufferType::Scratch, ScratchBufferName);
		ScratchBuffer = AutoScratchBuffer.GetReference();
		ScratchBufferOffset = 0;
	}

	if (bIsUpdate)
	{
		checkf(ScratchBuffer, TEXT("TLAS update requires scratch buffer of at least %lld bytes."), Scene.SizeInfo.UpdateScratchSize);
	}
	else
	{
		checkf(ScratchBuffer, TEXT("TLAS build requires scratch buffer of at least %lld bytes."), Scene.SizeInfo.BuildScratchSize);
	}

	{
		D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS BuildInputs;
		BuildInputs = {};
		BuildInputs.Type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_TOP_LEVEL;
		BuildInputs.DescsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
		BuildInputs.NumDescs = NumInstances;
		BuildInputs.Flags = TranslateRayTracingAccelerationStructureFlags(Scene.Initializer.BuildFlags);

		D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO PrebuildInfo = {};

		CommandContext.GetParentDevice()->GetRaytracingAccelerationStructurePrebuildInfo(&BuildInputs, &PrebuildInfo);

		checkf(PrebuildInfo.ResultDataMaxSizeInBytes <= Scene.SizeInfo.ResultSize,
			TEXT("TLAS build result buffer now requires %lld bytes, but only %lld was calculated in the constructor."),
			PrebuildInfo.ResultDataMaxSizeInBytes, Scene.SizeInfo.ResultSize);

		checkf(PrebuildInfo.ScratchDataSizeInBytes <= Scene.SizeInfo.BuildScratchSize,
			TEXT("TLAS build scratch buffer now requires %lld bytes, but only %lld was calculated in the constructor."),
			PrebuildInfo.ScratchDataSizeInBytes, Scene.SizeInfo.BuildScratchSize);

		if (bIsUpdate)
		{
			checkf(PrebuildInfo.UpdateScratchDataSizeInBytes <= Scene.SizeInfo.UpdateScratchSize,
				TEXT("TLAS update scratch buffer now requires %lld bytes, but only %lld was calculated in the constructor."),
				PrebuildInfo.UpdateScratchDataSizeInBytes, Scene.SizeInfo.UpdateScratchSize);

			checkf(ScratchBufferOffset + PrebuildInfo.UpdateScratchDataSizeInBytes <= ScratchBuffer->GetSize(),
				TEXT("TLAS scratch buffer size is %d bytes with offset %d (%d bytes available), but the update requires %lld bytes (NumInstances = %d)."),
				ScratchBuffer->GetSize(), ScratchBufferOffset, ScratchBuffer->GetSize() - ScratchBufferOffset,
				PrebuildInfo.UpdateScratchDataSizeInBytes, NumInstances);
		}
		else
		{
			checkf(ScratchBufferOffset + PrebuildInfo.ScratchDataSizeInBytes <= ScratchBuffer->GetSize(),
				TEXT("TLAS scratch buffer size is %d bytes with offset %d (%d bytes available), but the build requires %lld bytes (NumInstances = %d)."),
				ScratchBuffer->GetSize(), ScratchBufferOffset, ScratchBuffer->GetSize() - ScratchBufferOffset,
				PrebuildInfo.ScratchDataSizeInBytes, NumInstances);
		}
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	Scene.UpdateResidencyTracking(CommandContext);
#endif // ENABLE_RESIDENCY_MANAGEMENT

	// Make necessary resources resident

	TRefCountPtr<FD3D12Buffer>& AccelerationStructureBuffer = Scene.PerGPUData[GPUIndex].AccelerationStructureBuffer;
	checkf(AccelerationStructureBuffer.IsValid(),
		TEXT("Acceleration structure buffer must be set for this scene using RHIBindAccelerationStructureMemory() before build command is issued."));

	CommandContext.UpdateResidency(InstanceBuffer->GetResource());
	CommandContext.UpdateResidency(ScratchBuffer->GetResource());

	Scene.UpdateResidency(CommandContext, false);

	if (ShouldRunRayTracingGPUValidation())
	{
		RHI_BREADCRUMB_EVENT(CommandContext, "RTSceneValidation");

		TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(&CommandContext);
		uint32 InstanceBufferStride = GRHIRayTracingInstanceDescriptorSize;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// TODO: Validation related to SBT needs to be done somewhere else since SBT is not known when in BuildAccelerationStructure
		uint32 TotalHitGroupSlots = Scene.Initializer.NumTotalSegments;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FRayTracingValidateSceneBuildParamsCS::Dispatch(RHICmdList,
			TotalHitGroupSlots, NumInstances,
			InstanceBuffer, InstanceBufferOffset, InstanceBufferStride);
	}

	{
		const D3D12_GPU_VIRTUAL_ADDRESS BufferAddress = AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress() + Scene.BufferOffset;
		D3D12_GPU_VIRTUAL_ADDRESS ScratchAddress = ScratchBuffer->ResourceLocation.GetGPUVirtualAddress() + ScratchBufferOffset;

		checkf(BufferAddress % GRHIRayTracingAccelerationStructureAlignment == 0,
			TEXT("TLAS buffer (plus offset) must be aligned to %d bytes."),
			GRHIRayTracingAccelerationStructureAlignment);

		checkf(ScratchAddress % GRHIRayTracingScratchBufferAlignment == 0,
			TEXT("TLAS scratch buffer (plus offset) must be aligned to %d bytes."),
			GRHIRayTracingScratchBufferAlignment);

		OutBuildDesc.Inputs = {};
		OutBuildDesc.Inputs.Type = D3D12_RAYTRACING_ACCELERATION_STRUCTURE_TYPE_TOP_LEVEL;
		OutBuildDesc.Inputs.DescsLayout = D3D12_ELEMENTS_LAYOUT_ARRAY;
		OutBuildDesc.Inputs.NumDescs = NumInstances;
		OutBuildDesc.Inputs.InstanceDescs = InstanceBuffer->ResourceLocation.GetGPUVirtualAddress() + InstanceBufferOffset;
		OutBuildDesc.Inputs.Flags = TranslateRayTracingAccelerationStructureFlags(Scene.Initializer.BuildFlags);

		if (bIsUpdate)
		{
			checkf(EnumHasAllFlags(Scene.Initializer.BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate),
				TEXT("Acceleration structure must be created with FRayTracingGeometryInitializer::bAllowUpdate=true to perform refit / update."));

			OutBuildDesc.Inputs.Flags |= D3D12_RAYTRACING_ACCELERATION_STRUCTURE_BUILD_FLAG_PERFORM_UPDATE;
		}

		OutBuildDesc.DestAccelerationStructureData = BufferAddress;
		OutBuildDesc.ScratchAccelerationStructureData = ScratchAddress;
		OutBuildDesc.SourceAccelerationStructureData = bIsUpdate ? BufferAddress : D3D12_GPU_VIRTUAL_ADDRESS(0);

		if (bIsUpdate)
		{
			INC_DWORD_STAT(STAT_D3D12RayTracingUpdatedTLAS);
		}
		else
		{
			INC_DWORD_STAT(STAT_D3D12RayTracingBuiltTLAS);
		}
	}
}

void FD3D12CommandContext::BuildAccelerationStructuresInternal(TConstArrayView<D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC> BuildDescs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12::BuildRaytracingAccelerationStructure);
	for (const D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& Desc : BuildDescs)
	{
		GraphicsCommandList4()->BuildRaytracingAccelerationStructure(&Desc, 0, nullptr);
	}
}

#if WITH_NVAPI
void FD3D12CommandContext::BuildAccelerationStructuresInternalEx(const TArrayView<NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX> DescArray)
{
	for (const NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX& Desc : DescArray)
	{
		NVAPI_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_EX_PARAMS Params = { 0 };
		Params.version = NVAPI_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_EX_PARAMS_VER;
		Params.pDesc = &Desc;

		NvAPI_Status status = NvAPI_D3D12_BuildRaytracingAccelerationStructureEx(GraphicsCommandList4().Get(), &Params);
		check(status == NVAPI_OK);
	}
}
#endif

#if WITH_MGPU
void FD3D12CommandContext::UnregisterAccelerationStructuresInternalMGPU(TConstArrayView<FRayTracingGeometryBuildParams> Params, FRHIGPUMask GPUMask)
{
	// We need to unregister rename listeners for all GPUs in a separate pass before running "RHIBuildAccelerationStructures", as the build process
	// may modify the buffer references in the ray tracing geometry.  This leads to an assert where the code attempts to unregister the newer buffer
	// references on the additional GPUs, rather than the original buffer references.  It's OK to unregister redundantly, as a flag is set to track
	// whether a buffer is registered, and additional unregister calls do nothing.
	for (uint32 GPUIndex : GPUMask)
	{
		for (const FRayTracingGeometryBuildParams& P : Params)
		{
			FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(P.Geometry.GetReference());
			Geometry->UnregisterAsRenameListener(GPUIndex);
		}
	}
}
#endif  // WITH_MGPU

void FD3D12CommandContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildAccelerationStructure_BottomLevel);
	SCOPE_CYCLE_COUNTER(STAT_D3D12BuildBLAS);
	LLM_SCOPE_BYNAME(TEXT("FD3D12RT/BLAS"));

	checkf(ScratchBufferRange.Buffer != nullptr, TEXT("BuildAccelerationStructures requires valid scratch buffer"));

	// Update geometry vertex buffers
	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(P.Geometry.GetReference());
		Geometry->UnregisterAsRenameListener(GetGPUIndex());

		if (P.Segments.Num())
		{
			checkf(P.Segments.Num() == Geometry->Initializer.Segments.Num(),
				TEXT("If updated segments are provided, they must exactly match existing geometry segments. Only vertex buffer bindings may change."));

			for (int32 i = 0; i < P.Segments.Num(); ++i)
			{
				checkf(P.Segments[i].VertexBuffer != nullptr, TEXT("Segments used to build/update ray tracing geometry must have a valid VertexBuffer."));

				checkf(P.Segments[i].MaxVertices <= Geometry->Initializer.Segments[i].MaxVertices,
					TEXT("Maximum number of vertices in a segment (%u) must not be larger than what was declared during FRHIRayTracingGeometry creation (%u), as this controls BLAS memory allocation."),
					P.Segments[i].MaxVertices, Geometry->Initializer.Segments[i].MaxVertices
				);

				Geometry->Initializer.Segments[i].VertexBuffer            = P.Segments[i].VertexBuffer;
				Geometry->Initializer.Segments[i].VertexBufferElementType = P.Segments[i].VertexBufferElementType;
				Geometry->Initializer.Segments[i].VertexBufferStride      = P.Segments[i].VertexBufferStride;
				Geometry->Initializer.Segments[i].VertexBufferOffset      = P.Segments[i].VertexBufferOffset;
			}

#if ENABLE_RESIDENCY_MANAGEMENT
			++Geometry->ResourceGeneration;
#endif
		}
	}

	FlushResourceBarriers();
		
	const uint32 GPUIndex = GetGPUIndex();

	// Then do all work
	TArray<D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC, TInlineAllocator<32>> BuildDescs;
	BuildDescs.Reserve(Params.Num());

#if WITH_NVAPI
	TArray<NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX, TInlineAllocator<32>> BuildDescsEx;
	BuildDescsEx.Reserve(Params.Num());
#endif

	uint32 ScratchBufferSize = ScratchBufferRange.Size ? ScratchBufferRange.Size : ScratchBufferRange.Buffer->GetSize();

	checkf(ScratchBufferSize + ScratchBufferRange.Offset <= ScratchBufferRange.Buffer->GetSize(),
		TEXT("BLAS scratch buffer range size is %lld bytes with offset %lld, but the buffer only has %d bytes. "),
		ScratchBufferRange.Size, ScratchBufferRange.Offset, ScratchBufferRange.Buffer->GetSize());


	const uint64 ScratchAlignment = GRHIRayTracingAccelerationStructureAlignment;
	FD3D12Buffer* ScratchBuffer = FD3D12DynamicRHI::ResourceCast(ScratchBufferRange.Buffer, GPUIndex);
	uint32 ScratchBufferOffset = ScratchBufferRange.Offset;

	UpdateResidency(ScratchBuffer->GetResource());

	FMemMark Mark(FMemStack::Get());

	for (int32 i = 0; i < Params.Num(); i++)
	{
		const FRayTracingGeometryBuildParams& P = Params[i];

		FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(P.Geometry.GetReference());
		Geometry->SetDirty(GetGPUMask(), true);

		// Register as rename listener to index/vertex buffers
		Geometry->UnregisterAsRenameListener(GPUIndex);
		Geometry->RegisterAsRenameListener(GPUIndex);

		// Recreate the hit group system parameters and use them during setup of the descs
		Geometry->SetupHitGroupSystemParameters(GPUIndex);

		if (Geometry->IsDirty(GPUIndex))
		{
			uint64 ScratchBufferRequiredSize = P.BuildMode == EAccelerationStructureBuildMode::Update ? Geometry->SizeInfo.UpdateScratchSize : Geometry->SizeInfo.BuildScratchSize;
			checkf(ScratchBufferRequiredSize + ScratchBufferOffset <= ScratchBufferSize,
				TEXT("BLAS scratch buffer size is %d bytes with offset %d (%d bytes available), but the build requires %lld bytes. "),
				ScratchBufferSize, ScratchBufferOffset, ScratchBufferSize - ScratchBufferOffset, ScratchBufferRequiredSize);

			D3D12_GPU_VIRTUAL_ADDRESS ScratchBufferAddress = ScratchBuffer->ResourceLocation.GetGPUVirtualAddress() + ScratchBufferOffset;

			ScratchBufferOffset = Align(ScratchBufferOffset + ScratchBufferRequiredSize, ScratchAlignment);

			checkf(ScratchBufferAddress % GRHIRayTracingAccelerationStructureAlignment == 0,
				TEXT("BLAS scratch buffer (plus offset) must be aligned to %d bytes."),
				GRHIRayTracingAccelerationStructureAlignment);

#if WITH_NVAPI
			if (GeometryUsesLinearSweptSpheres(Geometry->GetInitializer()))
			{
				const uint32 NumGeometryDescs = Geometry->GetInitializer().Segments.Num();
				NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX* LocalGeometryDescsMemoryEx = (NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX*)FMemStack::Get().Alloc(NumGeometryDescs * sizeof(NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX), alignof(NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX));
				TArrayView<NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX> LocalGeometryDescsEx = MakeArrayView(LocalGeometryDescsMemoryEx, NumGeometryDescs);

				NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX& BuildDescEx = BuildDescsEx.AddZeroed_GetRef();
				Geometry->CreateAccelerationStructureBuildDescEx(*this, P.BuildMode, ScratchBufferAddress, BuildDescEx, LocalGeometryDescsEx);
			}
			else
#endif
			{
				// We need to keep D3D12_RAYTRACING_GEOMETRY_DESCs that are used in D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC alive.
			const uint32 NumGeometryDescs = Geometry->GetInitializer().Segments.Num();
				D3D12_RAYTRACING_GEOMETRY_DESC* LocalGeometryDescsMemory = (D3D12_RAYTRACING_GEOMETRY_DESC*)FMemStack::Get().Alloc(NumGeometryDescs * sizeof(D3D12_RAYTRACING_GEOMETRY_DESC), alignof(D3D12_RAYTRACING_GEOMETRY_DESC));
				TArrayView<D3D12_RAYTRACING_GEOMETRY_DESC> LocalGeometryDescs = MakeArrayView(LocalGeometryDescsMemory, NumGeometryDescs);

				D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& BuildDesc = BuildDescs.AddZeroed_GetRef();			
				Geometry->CreateAccelerationStructureBuildDesc(*this, P.BuildMode, ScratchBufferAddress, BuildDesc, LocalGeometryDescs);
			}

			Geometry->UpdateResidency(*this);

			if (P.BuildMode == EAccelerationStructureBuildMode::Update)
			{
				INC_DWORD_STAT(STAT_D3D12RayTracingUpdatedBLAS);
			}
			else
			{
				INC_DWORD_STAT(STAT_D3D12RayTracingBuiltBLAS);
			}
		}
	}

	if (ShouldRunRayTracingGPUValidation())
	{
		RHI_BREADCRUMB_EVENT(*this, "RTGeometryValidation");

		TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
		for (const FRayTracingGeometryBuildParams& P : Params)
		{
			FRayTracingValidateGeometryBuildParamsCS::Dispatch(RHICmdList, P);
		}
	}


	BuildAccelerationStructuresInternal(BuildDescs);
#if WITH_NVAPI
	BuildAccelerationStructuresInternalEx(BuildDescsEx);
#endif

	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(P.Geometry.GetReference());

		if (Geometry->IsDirty(GPUIndex))
		{
			ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(Geometry->Initializer);
			if (ShouldCompactAfterBuild(GeometryBuildFlags))
			{
				GetParentDevice()->GetRayTracingCompactionRequestHandler()->RequestCompact(Geometry);
				Geometry->PerGPUData[GPUIndex].bHasPendingCompactionRequests = true;
			}

			Geometry->SetDirty(GetGPUMask(), false);
		}
	}

	// Add a UAV barrier after each acceleration structure build batch.
	// This is required because there are currently no explicit read/write barriers
	// for acceleration structures, but we need to ensure that all commands
	// are complete before BLAS is used again on the GPU.
	//
	// @TODO - EBs let us express those R/W barriers so we should do that
	//		   Also could use the postbuild flags here if added (see CreateAccelerationStructureBuildDesc())

	AddGlobalBarrier(ED3D12Access::BVHWrite, ED3D12Access::BVHRead);
}

void FD3D12CommandContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildAccelerationStructures_TopLevel);
	SCOPE_CYCLE_COUNTER(STAT_D3D12BuildTLAS);

	TArray<D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC, TInlineAllocator<8>> BuildDescs;
	BuildDescs.Reserve(Params.Num());

	for (const FRayTracingSceneBuildParams& SceneBuildParams : Params)
	{
		FD3D12RayTracingScene* Scene = FD3D12DynamicRHI::ResourceCast(SceneBuildParams.Scene);
		FD3D12Buffer* ScratchBuffer = RetrieveObject<FD3D12Buffer>(SceneBuildParams.ScratchBuffer);
		FD3D12Buffer* InstanceBuffer = RetrieveObject<FD3D12Buffer>(SceneBuildParams.InstanceBuffer);

		Scene->SetReferencedResources(SceneBuildParams.ReferencedGeometries, SceneBuildParams.ReferencedBuffers);

		D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& BuildDesc = BuildDescs.AddDefaulted_GetRef();

		PrepareAccelerationStructureBuild(
			*this,
			*Scene,
			ScratchBuffer, SceneBuildParams.ScratchBufferOffset,
			InstanceBuffer, SceneBuildParams.InstanceBufferOffset,
			SceneBuildParams.NumInstances,
			SceneBuildParams.BuildMode,
			BuildDesc
		);
	}

	// Barrier is used here to ensure that all bottom level acceleration structures are built
	AddGlobalBarrier(ED3D12Access::BVHWrite, ED3D12Access::BVHRead);

	FlushResourceBarriers();

	BuildAccelerationStructuresInternal(BuildDescs);

	// Barrier is used here to ensure that the acceleration structure build is complete before any rays are traced
	// #dxr_todo: these barriers should ideally be inserted by the high level code to allow more overlapped execution
	AddGlobalBarrier(ED3D12Access::BVHWrite, ED3D12Access::BVHRead);

	for (const FRayTracingSceneBuildParams& SceneBuildParams : Params)
	{
		FD3D12RayTracingScene* Scene = FD3D12DynamicRHI::ResourceCast(SceneBuildParams.Scene);
		FD3D12Buffer* ScratchBuffer = RetrieveObject<FD3D12Buffer>(SceneBuildParams.ScratchBuffer);
		FD3D12Buffer* InstanceBuffer = RetrieveObject<FD3D12Buffer>(SceneBuildParams.InstanceBuffer);

		Scene->bBuilt = true;

#if D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING
		D3D12RayTracingSceneDebugUpdate(*Scene, InstanceBuffer, SceneBuildParams.InstanceBufferOffset, *this);
#endif // D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING
	}
}

void FD3D12CommandContext::RHIBindAccelerationStructureMemory(FRHIRayTracingScene* InScene, FRHIBuffer* InBuffer, uint32 InBufferOffset)
{
	FD3D12RayTracingScene* Scene = FD3D12DynamicRHI::ResourceCast(InScene);
	Scene->BindBuffer(InBuffer, InBufferOffset);
}

void FD3D12CommandContext::RHICommitShaderBindingTable(FRHIShaderBindingTable* InSBT, FRHIBuffer* InlineBindingDataBuffer)
{
	FD3D12RayTracingShaderBindingTable* SBT = FD3D12DynamicRHI::ResourceCast(InSBT);
	check(SBT);

	FD3D12RayTracingShaderBindingTableInternal* ShaderTableForDevice = SBT->GetTableForDevice(GetParentDevice());
	if (ShaderTableForDevice->bIsDirty)
	{
		ShaderTableForDevice->Commit(*this, InlineBindingDataBuffer);
	}
}

static constexpr uint32 LooseParameterCBVIndex = 0; // Global uniform buffer is always assumed to be in slot 0

struct FD3D12RayTracingGlobalResourceBinder
{
	FD3D12RayTracingGlobalResourceBinder(FD3D12CommandContext& InCommandContext, FD3D12ExplicitDescriptorCache& InDescriptorCache)
		: CommandContext(InCommandContext)
		, DescriptorCache(InDescriptorCache)
	{
	}

	void SetUniformBuffer(uint32 BaseSlotIndex, uint32 DescriptorIndex, FD3D12UniformBuffer* UniformBuffer)
	{
		// don't have to do anything
	}

	void SetRootCBV(uint32 BaseSlotIndex, uint32 DescriptorIndex, D3D12_GPU_VIRTUAL_ADDRESS Address)
	{
		CommandContext.GraphicsCommandList()->SetComputeRootConstantBufferView(BaseSlotIndex + DescriptorIndex, Address);
	}

	void SetRootSRV(uint32 BaseSlotIndex, uint32 DescriptorIndex, D3D12_GPU_VIRTUAL_ADDRESS Address)
	{
		CommandContext.GraphicsCommandList()->SetComputeRootShaderResourceView(BaseSlotIndex + DescriptorIndex, Address);
	}

	void SetRootDescriptorTable(uint32 SlotIndex, D3D12_GPU_DESCRIPTOR_HANDLE DescriptorTable)
	{
		CommandContext.GraphicsCommandList()->SetComputeRootDescriptorTable(SlotIndex, DescriptorTable);
	}

	FD3D12ConstantBufferView* SetLooseParameterData(const void* Data, uint32 DataSize, D3D12_GPU_VIRTUAL_ADDRESS& OutGPUVirtualAddress)
	{
		checkf(0, TEXT("Loose parameters are not implemented for global ray tracing shaders (raygen, miss, callable)"));
		return nullptr;
	}

	void AddReferencedShaderResource(FD3D12BaseShaderResource* ShaderResource)
	{
		CommandContext.UpdateResidency(ShaderResource->GetResource());
	}

	void AddReferencedTexture(FRHITexture* RHITexture)
	{
		FD3D12Texture* Texture = FD3D12CommandContext::RetrieveTexture(RHITexture, CommandContext.GetGPUIndex());
		CommandContext.UpdateResidency(Texture->ResourceLocation.GetResource());
	}
	
	void AddReferencedUniformBuffer(uint32 BaseSlotIndex, uint32 DescriptorIndex, FD3D12UniformBuffer* UniformBuffer)
	{
		CommandContext.UpdateResidency(UniformBuffer->ResourceLocation.GetResource());
	}

	void AddRayTracingSceneReference(FD3D12RayTracingScene* D3D12RayTracingScene)
	{
		D3D12RayTracingScene->UpdateResidency(CommandContext, true);
	}

	FD3D12Device* GetDevice()
	{
		return CommandContext.GetParentDevice();
	}

#if ENABLE_RHI_VALIDATION
	RHIValidation::FTracker* GetValidationTracker()
	{
		return CommandContext.Tracker;
	}
#endif

	FD3D12CommandContext& CommandContext;
	FD3D12ExplicitDescriptorCache& DescriptorCache;
	static constexpr uint32 WorkerIndex = 0;
};

struct FD3D12RayTracingLocalResourceBinder
{
	FD3D12RayTracingLocalResourceBinder(
		FD3D12Device& InDevice, 
		FD3D12RayTracingShaderBindingTableInternal& InShaderTable, 
		const FD3D12RootSignature& InRootSignature, 
		ERayTracingLocalShaderBindingType InBindingType,
		uint32 InRecordIndex, 
		uint32 InWorkerIndex, 
		ERayTracingBindingType RTBindingType)
		: Device(InDevice)
		, ShaderTable(InShaderTable)
		, DescriptorCache(*InShaderTable.DescriptorCache)
		, RootSignature(InRootSignature)
		, BindingType(InBindingType)
		, RecordIndex(InRecordIndex)
		, WorkerIndex(InWorkerIndex)
	{
		check(InShaderTable.DescriptorCache != nullptr);
		check(WorkerIndex < InShaderTable.MaxBindingWorkers);
		check(WorkerIndex < uint32(DescriptorCache.WorkerData.Num()));
		check(RecordIndex != ~0u);

		switch (RTBindingType)
		{
		case ERayTracingBindingType::CallableShader:
			ShaderTableOffset = InShaderTable.CallableShaderTableOffset;
			break;
		case ERayTracingBindingType::HitGroup:
			ShaderTableOffset = InShaderTable.HitGroupShaderTableOffset;
			break;
		case ERayTracingBindingType::MissShader:
			ShaderTableOffset = InShaderTable.MissShaderTableOffset;
			break;
		default:
			checkNoEntry();
		}
	}

	uint32 ComputeOffsetWithinRootSignature(uint32 BaseSlotIndex, uint32 DescriptorIndex)
	{
		const uint32 BindOffsetBase = RootSignature.GetBindSlotOffsetInBytes(BaseSlotIndex);
		const uint32 DescriptorSize = uint32(sizeof(D3D12_GPU_VIRTUAL_ADDRESS));
		return BindOffsetBase + DescriptorIndex * DescriptorSize;
	}

	void SetRootDescriptor(uint32 BaseSlotIndex, uint32 DescriptorIndex, D3D12_GPU_VIRTUAL_ADDRESS Address)
	{
		const uint32 OffsetWithinRootSignature = ComputeOffsetWithinRootSignature(BaseSlotIndex, DescriptorIndex);
		if (BindingType == ERayTracingLocalShaderBindingType::Validation)
		{
			ShaderTable.CompareLocalShaderParameters(ShaderTableOffset, RecordIndex, OffsetWithinRootSignature, Address);
		}
		else
		{
			ShaderTable.SetLocalShaderParameters(ShaderTableOffset, RecordIndex, OffsetWithinRootSignature, Address);
		}
	}

	void SetRootCBV(uint32 BaseSlotIndex, uint32 DescriptorIndex, D3D12_GPU_VIRTUAL_ADDRESS Address)
	{
		SetRootDescriptor(BaseSlotIndex, DescriptorIndex, Address);
	}

	void SetRootSRV(uint32 BaseSlotIndex, uint32 DescriptorIndex, D3D12_GPU_VIRTUAL_ADDRESS Address)
	{
		SetRootDescriptor(BaseSlotIndex, DescriptorIndex, Address);
	}

	void SetRootDescriptorTable(uint32 SlotIndex, D3D12_GPU_DESCRIPTOR_HANDLE DescriptorTable)
	{
		const uint32 BindOffset = RootSignature.GetBindSlotOffsetInBytes(SlotIndex);
		if (BindingType == ERayTracingLocalShaderBindingType::Validation)
		{
			ShaderTable.CompareLocalShaderParameters(ShaderTableOffset, RecordIndex, BindOffset, DescriptorTable);
		}
		else
		{
			ShaderTable.SetLocalShaderParameters(ShaderTableOffset, RecordIndex, BindOffset, DescriptorTable);
		}
	}

	FD3D12ConstantBufferView* SetLooseParameterData(const void* Data, uint32 DataSize, D3D12_GPU_VIRTUAL_ADDRESS& OutGPUVirtualAddress)
	{
		bHasLooseParameterData = true;
		return ShaderTable.SetLooseParameterData(WorkerIndex, RecordIndex, BindingType, Data, DataSize, OutGPUVirtualAddress);
	}

	void AddReferencedShaderResource(FD3D12BaseShaderResource* ShaderResource)
	{
		ShaderTable.AddReferencedShaderResource(WorkerIndex, RecordIndex, BindingType, ShaderResource);
	}

	void AddReferencedTexture(FRHITexture* RHITexture)
	{
		ShaderTable.AddReferencedTexture(WorkerIndex, RecordIndex, BindingType, RHITexture);
	}

	void AddReferencedUniformBuffer(uint32 BaseSlotIndex, uint32 DescriptorIndex, FD3D12UniformBuffer* UniformBuffer)
	{
		const uint32 OffsetWithinRootSignature = ComputeOffsetWithinRootSignature(BaseSlotIndex, DescriptorIndex);		
		ShaderTable.AddReferencedUniformBuffer(WorkerIndex, RecordIndex, ShaderTableOffset, OffsetWithinRootSignature, BindingType, UniformBuffer);
	}

	void AddRayTracingSceneReference(FD3D12RayTracingScene* D3D12RayTracingScene)
	{
		checkf(false, TEXT("Unexpected RayTracingScene reference in local shader bindings"));
	}

	FD3D12Device* GetDevice()
	{
		return &Device;
	}

#if ENABLE_RHI_VALIDATION
	RHIValidation::FTracker* GetValidationTracker()
	{
		// We can't validate resource states in RHISetBindingsOnShaderBindingTable because there's no command context at that point, and because the states will
		// change before the raytracing command is dispatched anyway.
		return nullptr;
	}
#endif

	FD3D12Device& Device;
	FD3D12RayTracingShaderBindingTableInternal& ShaderTable;
	FD3D12ExplicitDescriptorCache& DescriptorCache;
	const FD3D12RootSignature& RootSignature;
	ERayTracingLocalShaderBindingType BindingType;
	uint32 ShaderTableOffset = 0;
	uint32 RecordIndex = ~0u;
	uint32 WorkerIndex = 0;
	bool bHasLooseParameterData = false;
	
	TArray<FD3D12View*, TInlineAllocator<MAX_SRVS + MAX_UAVS>> ReferencedViews;
	TArray<FD3D12Resource*, TInlineAllocator<MAX_CBS>> ReferencedCBResources;
};

template <typename ResourceBinderType>
static bool SetRayTracingShaderResources(
	const FD3D12RayTracingShader* Shader,
	const FD3D12RootSignature* RootSignature,
	uint32 InNumBindlessParameters, FRHIShaderParameterResource const* BindlessParameters,
	uint32 InNumTextures, FRHITexture* const* Textures,
	uint32 InNumSRVs, FRHIShaderResourceView* const* SRVs,
	uint32 InNumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 InNumSamplers, FRHISamplerState* const* Samplers,
	uint32 InNumUAVs, FRHIUnorderedAccessView* const* UAVs,
	uint32 InLooseParameterDataSize, const void* InLooseParameterData,
	ResourceBinderType& Binder)
{
	check(Shader && RootSignature);

	struct FBindings
	{
		FBindings(ResourceBinderType& InBinder, uint32 InGPUIndex, const FD3D12ShaderData* ShaderData)
			: Binder(InBinder)
			, GPUIndex(InGPUIndex)
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			, bBindlessResources(EnumHasAnyFlags(ShaderData->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources))
			, bBindlessSamplers(EnumHasAnyFlags(ShaderData->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessSamplers))
#endif
		{
		}

		ResourceBinderType& Binder;
		uint32 GPUIndex;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		const bool bBindlessResources;
		const bool bBindlessSamplers;
#endif

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		D3D12_CPU_DESCRIPTOR_HANDLE LocalCBVs[MAX_CBS];
#endif
		D3D12_GPU_VIRTUAL_ADDRESS RemoteCBVs[MAX_CBS];
		FD3D12UniformBuffer* UniformBuffers[MAX_CBS] = {};

		D3D12_CPU_DESCRIPTOR_HANDLE LocalSRVs[MAX_SRVS];
		D3D12_CPU_DESCRIPTOR_HANDLE LocalUAVs[MAX_UAVS];
		D3D12_CPU_DESCRIPTOR_HANDLE LocalSamplers[MAX_SAMPLERS];

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		uint32 CBVVersions[MAX_CBS];
#endif
		uint32 SRVVersions[MAX_SRVS];
		uint32 UAVVersions[MAX_SRVS];
		uint32 SamplerVersions[MAX_SRVS];

		TArray<FD3D12RayTracingScene*, TInlineAllocator<1>> ReferencedRayTracingScenes;

		uint64 BoundSRVMask = 0;
		uint64 BoundCBVMask = 0;
		uint64 BoundUAVMask = 0;
		uint64 BoundSamplerMask = 0;

		void SetUAV(FRHIUnorderedAccessView* RHIUAV, uint8 Index)
		{
			FD3D12UnorderedAccessView* UAV = FD3D12CommandContext::RetrieveObject<FD3D12UnorderedAccessView_RHI>(RHIUAV, GPUIndex);
			check(UAV != nullptr);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if (!bBindlessResources)
#endif
			{
				FD3D12OfflineDescriptor Descriptor = UAV->GetOfflineCpuHandle();
				LocalUAVs[Index] = Descriptor;
				UAVVersions[Index] = Descriptor.GetVersion();
				BoundUAVMask |= 1ull << Index;
			}

			Binder.AddReferencedShaderResource(UAV->GetShaderResource());
		}

		void SetSRV(FRHIShaderResourceView* RHISRV, uint8 Index)
		{
			FD3D12ShaderResourceView* SRV = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(RHISRV, GPUIndex);
			check(SRV != nullptr);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if (!bBindlessResources)
#endif
			{
				FD3D12OfflineDescriptor Descriptor = SRV->GetOfflineCpuHandle();
				LocalSRVs[Index] = Descriptor;
				SRVVersions[Index] = Descriptor.GetVersion();
				BoundSRVMask |= 1ull << Index;
			}

			Binder.AddReferencedShaderResource(SRV->GetShaderResource());

			FD3D12RayTracingScene* ReferencedRayTracingScene = SRV->GetRayTracingScene();
			if (ReferencedRayTracingScene)
			{
				ReferencedRayTracingScenes.Add(ReferencedRayTracingScene);
			}
		}

		void SetTexture(FRHITexture* RHITexture, uint8 Index)
		{
			FD3D12ShaderResourceView* SRV = FD3D12CommandContext::RetrieveTexture(RHITexture, GPUIndex)->GetShaderResourceView();
			if (!ensure(SRV))
			{
				SRV = FD3D12CommandContext::RetrieveTexture(GBlackTexture->TextureRHI, GPUIndex)->GetShaderResourceView();
			}
			check(SRV != nullptr);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if (!bBindlessResources)
#endif
			{
				FD3D12OfflineDescriptor Descriptor = SRV->GetOfflineCpuHandle();
				LocalSRVs[Index] = Descriptor;
				SRVVersions[Index] = Descriptor.GetVersion();
				BoundSRVMask |= 1ull << Index;
			}

			Binder.AddReferencedTexture(RHITexture);
		}

		void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint8 Index)
		{
			FD3D12ResourceCollection* D3D12ResourceCollection = FD3D12CommandContext::RetrieveObject<FD3D12ResourceCollection>(ResourceCollection, GPUIndex);
			FD3D12ShaderResourceView* SRV = D3D12ResourceCollection ? D3D12ResourceCollection->GetShaderResourceView() : nullptr;

			check(SRV != nullptr);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if (!bBindlessResources)
#endif
			{
				FD3D12OfflineDescriptor Descriptor = SRV->GetOfflineCpuHandle();
				LocalSRVs[Index] = Descriptor;
				SRVVersions[Index] = Descriptor.GetVersion();
			}

			BoundSRVMask |= 1ull << Index;

			Binder.AddReferencedShaderResource(SRV->GetShaderResource());

			// Ensure all member SRVs referenced by the collection buffer are resident.
			for (FD3D12ShaderResourceView* MemberSRV : D3D12ResourceCollection->AllSrvs)
			{
				Binder.AddReferencedShaderResource(MemberSRV->GetShaderResource());

				if (FD3D12RayTracingScene* ReferencedRayTracingScene = MemberSRV->GetRayTracingScene())
				{
					ReferencedRayTracingScenes.Add(ReferencedRayTracingScene);
				}
			}

			// Resolve texture references to their current textures and ensure residency.
			for (FD3D12RHITextureReference* TextureReference : D3D12ResourceCollection->AllTextureReferences)
			{
				if (TextureReference)
				{
					Binder.AddReferencedTexture(TextureReference);
				}
			}
		}

		void SetSampler(FRHISamplerState* RHISampler, uint8 Index)
		{
			FD3D12SamplerState* Sampler = FD3D12CommandContext::RetrieveObject<FD3D12SamplerState>(RHISampler, GPUIndex);
			check(Sampler != nullptr);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if (!bBindlessSamplers)
#endif
			{
				FD3D12OfflineDescriptor Descriptor = Sampler->OfflineDescriptor;
				LocalSamplers[Index] = Descriptor;
				SamplerVersions[Index] = Descriptor.GetVersion();
				BoundSamplerMask |= 1ull << Index;
			}
		}
	};
	FBindings Bindings(Binder, Binder.GetDevice()->GetGPUIndex(), Shader);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (uint32 BindlessParameterIndex = 0; BindlessParameterIndex < InNumBindlessParameters; ++BindlessParameterIndex)
	{
		const FRHIShaderParameterResource& ShaderParameterResource = BindlessParameters[BindlessParameterIndex];
		if (FRHIResource* Resource = ShaderParameterResource.Resource)
		{
			switch (ShaderParameterResource.Type)
			{
				case FRHIShaderParameterResource::EType::Texture:
					Bindings.SetTexture(static_cast<FRHITexture*>(Resource), BindlessParameterIndex);
					break;
				case FRHIShaderParameterResource::EType::ResourceView:
					Bindings.SetSRV(static_cast<FRHIShaderResourceView*>(Resource), BindlessParameterIndex);
					break;
				case FRHIShaderParameterResource::EType::UnorderedAccessView:
					Bindings.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource), BindlessParameterIndex);
					break;
				case FRHIShaderParameterResource::EType::Sampler:
					Bindings.SetSampler(static_cast<FRHISamplerState*>(Resource), BindlessParameterIndex);
					break;
				case FRHIShaderParameterResource::EType::ResourceCollection:
					Bindings.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource), BindlessParameterIndex);
					break;
			}
		}
	}
#endif

	for (uint32 TextureIndex = 0; TextureIndex < InNumTextures; ++TextureIndex)
	{
		FRHITexture* Resource = Textures[TextureIndex];
		if (Resource)
		{
			Bindings.SetTexture(Resource, TextureIndex);
		}
	}

	for (uint32 SRVIndex = 0; SRVIndex < InNumSRVs; ++SRVIndex)
	{
		FRHIShaderResourceView* Resource = SRVs[SRVIndex];
		if (Resource)
		{
			Bindings.SetSRV(Resource, SRVIndex);
		}
	}

	for (uint32 CBVIndex = 0; CBVIndex < InNumUniformBuffers; ++CBVIndex)
	{
		FRHIUniformBuffer* Resource = UniformBuffers[CBVIndex];
		if (Resource)
		{
			FD3D12UniformBuffer* CBV = FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(Resource, Bindings.GPUIndex);
		#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			FD3D12OfflineDescriptor Descriptor = CBV->View->GetOfflineCpuHandle();
			Bindings.LocalCBVs[CBVIndex] = Descriptor;
			Bindings.CBVVersions[CBVIndex] = Descriptor.GetVersion();
		#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			Bindings.RemoteCBVs[CBVIndex] = CBV->ResourceLocation.GetGPUVirtualAddress();
			Bindings.BoundCBVMask |= 1ull << CBVIndex;

			Bindings.UniformBuffers[CBVIndex] = CBV;
		}
	}

	for (uint32 SamplerIndex = 0; SamplerIndex < InNumSamplers; ++SamplerIndex)
	{
		FRHISamplerState* Resource = Samplers[SamplerIndex];
		if (Resource)
		{
			Bindings.SetSampler(Resource, SamplerIndex);
		}
	}

	for (uint32 UAVIndex = 0; UAVIndex < InNumUAVs; ++UAVIndex)
	{
		FRHIUnorderedAccessView* Resource = UAVs[UAVIndex];
		if (Resource)
		{
			Bindings.SetUAV(Resource, UAVIndex);
		}
	}

	{
		uint32 DirtyUniformBuffers = ~(0u);
		UE::RHI::Private::SetUniformBufferResourcesFromTables(
			  Bindings
			, *Shader
			, DirtyUniformBuffers
			, UniformBuffers
#if ENABLE_RHI_VALIDATION
			, Binder.GetValidationTracker()
#endif
		);
	}

	// Bind loose parameters

	if (Shader->UsesGlobalUniformBuffer())
	{
		checkf(InLooseParameterDataSize && InLooseParameterData, TEXT("Shader uses global uniform buffer, but the required loose parameter data is not provided."));
	}

	if (InLooseParameterData && Shader->UsesGlobalUniformBuffer())
	{
		D3D12_GPU_VIRTUAL_ADDRESS LooseParameterGPUVirtualAddress = 0;
		FD3D12ConstantBufferView* ConstantBufferView = Binder.SetLooseParameterData(InLooseParameterData, InLooseParameterDataSize, LooseParameterGPUVirtualAddress);

	#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		Bindings.LocalCBVs[LooseParameterCBVIndex] = ConstantBufferView->GetOfflineCpuHandle();
	#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		Bindings.RemoteCBVs[LooseParameterCBVIndex] = LooseParameterGPUVirtualAddress;

		Bindings.BoundCBVMask |= 1ull << LooseParameterCBVIndex;
	}

	// Validate that all resources required by the shader are set

	auto IsCompleteBinding = [](uint32 ExpectedCount, uint64 BoundMask)
	{
		if (ExpectedCount > 64) return false; // Bound resource mask can't be represented by uint64

		// All bits of the mask [0..ExpectedCount) are expected to be set
		uint64 ExpectedMask = ExpectedCount == 64 ? ~0ull : ((1ull << ExpectedCount) - 1);
		return (ExpectedMask & BoundMask) == ExpectedMask;
	};
	check(IsCompleteBinding(Shader->ResourceCounts.NumSRVs    , Bindings.BoundSRVMask));
	check(IsCompleteBinding(Shader->ResourceCounts.NumUAVs    , Bindings.BoundUAVMask));
	check(IsCompleteBinding(Shader->ResourceCounts.NumCBs     , Bindings.BoundCBVMask));
	check(IsCompleteBinding(Shader->ResourceCounts.NumSamplers, Bindings.BoundSamplerMask));

	FD3D12ExplicitDescriptorCache& DescriptorCache = Binder.DescriptorCache;
	const uint32 WorkerIndex = Binder.WorkerIndex;

	const uint32 NumSRVs = Shader->ResourceCounts.NumSRVs;
	if (NumSRVs)
	{
		const int32 DescriptorTableBaseIndex = DescriptorCache.AllocateDeduplicated(Bindings.SRVVersions, Bindings.LocalSRVs, NumSRVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
		if (DescriptorTableBaseIndex < 0)
		{
			return false;
		}

		const uint32 BindSlot = RootSignature->SRVRDTBindSlot(SF_Compute);
		check(BindSlot != 0xFF);

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = DescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		Binder.SetRootDescriptorTable(BindSlot, ResourceDescriptorTableBaseGPU);
	}

	const uint32 NumUAVs = Shader->ResourceCounts.NumUAVs;
	if (NumUAVs)
	{
		const int32 DescriptorTableBaseIndex = DescriptorCache.AllocateDeduplicated(Bindings.UAVVersions, Bindings.LocalUAVs, NumUAVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
		if (DescriptorTableBaseIndex < 0)
		{
			return false;
		}

		const uint32 BindSlot = RootSignature->UAVRDTBindSlot(SF_Compute);
		check(BindSlot != 0xFF);

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = DescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		Binder.SetRootDescriptorTable(BindSlot, ResourceDescriptorTableBaseGPU);
	}

	const uint32 NumCBVs = Shader->ResourceCounts.NumCBs;
	if (Shader->ResourceCounts.NumCBs)
	{
	#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		if (!EnumHasAllFlags(Shader->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources))
		{
			const uint32 DescriptorTableBaseIndex = DescriptorCache.AllocateDeduplicated(Bindings.CBVVersions, Bindings.LocalCBVs, NumCBVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
			const uint32 BindSlot = RootSignature->CBVRDTBindSlot(SF_Compute);
			check(BindSlot != 0xFF);

			const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = DescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
			Binder.SetRootDescriptorTable(BindSlot, ResourceDescriptorTableBaseGPU);
		}
		else
	#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		{
			checkf(RootSignature->CBVRDTBindSlot(SF_Compute) == 0xFF, TEXT("Root CBV descriptor tables are not implemented for ray tracing shaders."));

			const uint32 BindSlot = RootSignature->CBVRDBaseBindSlot(SF_Compute);
			check(BindSlot != 0xFF);

			for (uint32 i = 0; i < Shader->ResourceCounts.NumCBs; ++i)
			{
				const uint64 SlotMask = (1ull << i);
				D3D12_GPU_VIRTUAL_ADDRESS BufferAddress = (Bindings.BoundCBVMask & SlotMask) ? Bindings.RemoteCBVs[i] : 0;
				Binder.SetRootCBV(BindSlot, i, BufferAddress);
				
				// Also set the uniform buffer in case it's valid so it can be notified about changes when it's a persistent binding
				if (Bindings.UniformBuffers[i])
				{
					Binder.AddReferencedUniformBuffer(BindSlot, i, Bindings.UniformBuffers[i]);					
				}
			}
		}
	}

	// Bind samplers

	const uint32 NumSamplers = Shader->ResourceCounts.NumSamplers;
	if (NumSamplers)
	{
		const int32 DescriptorTableBaseIndex = DescriptorCache.AllocateDeduplicated(Bindings.SamplerVersions, Bindings.LocalSamplers, NumSamplers, D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER, WorkerIndex);
		if (DescriptorTableBaseIndex < 0)
		{
			return false;
		}

		const uint32 BindSlot = RootSignature->SamplerRDTBindSlot(SF_Compute);
		check(BindSlot != 0xFF);

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = DescriptorCache.SamplerHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		Binder.SetRootDescriptorTable(BindSlot, ResourceDescriptorTableBaseGPU);
	}

	for (FD3D12RayTracingScene* RayTracingScene : Bindings.ReferencedRayTracingScenes)
	{
		Binder.AddRayTracingSceneReference(RayTracingScene);
	}

	return true;
}

template <typename ResourceBinderType>
static bool SetRayTracingShaderResources(
	const FD3D12RayTracingShader* Shader,
	const FD3D12RootSignature* RootSignature,
	const FRayTracingShaderBindings& ResourceBindings,
	ResourceBinderType& Binder)
{
	static_assert(
		sizeof(ResourceBindings.SRVs) / sizeof(*ResourceBindings.SRVs) == MAX_SRVS,
		"Ray Tracing Shader Bindings SRV array size must match D3D12 RHI Limit");
	static_assert(
		sizeof(ResourceBindings.UniformBuffers) / sizeof(*ResourceBindings.UniformBuffers) == MAX_CBS,
		"Ray Tracing Shader Bindings Uniform Buffer array size must match D3D12 RHI Limit");
	static_assert(
		sizeof(ResourceBindings.Samplers) / sizeof(*ResourceBindings.Samplers) == MAX_SAMPLERS,
		"Ray Tracing Shader Bindings Sampler array size must match D3D12 RHI Limit");
	static_assert(
		sizeof(ResourceBindings.UAVs) / sizeof(*ResourceBindings.UAVs) == MAX_UAVS,
		"Ray Tracing Shader Bindings UAV array size must match D3D12 RHI Limit");

	return SetRayTracingShaderResources(
		Shader,
		RootSignature,
		ResourceBindings.BindlessParameters.Num(), ResourceBindings.BindlessParameters.GetData(),
		UE_ARRAY_COUNT(ResourceBindings.Textures), ResourceBindings.Textures,
		UE_ARRAY_COUNT(ResourceBindings.SRVs), ResourceBindings.SRVs,
		UE_ARRAY_COUNT(ResourceBindings.UniformBuffers), ResourceBindings.UniformBuffers,
		UE_ARRAY_COUNT(ResourceBindings.Samplers), ResourceBindings.Samplers,
		UE_ARRAY_COUNT(ResourceBindings.UAVs), ResourceBindings.UAVs,
		0, nullptr, // loose parameters
		Binder);
}

static void DispatchRays(FD3D12CommandContext& CommandContext,
	const FRayTracingShaderBindings& GlobalBindings,
	const FD3D12RayTracingPipelineState* Pipeline,
	uint32 RayGenShaderIndex,
	FD3D12RayTracingShaderBindingTableInternal* OptShaderTable,
	const D3D12_DISPATCH_RAYS_DESC& DispatchDesc,
	ED3D12QueueType QueueType,
	FD3D12Resource* RayGenUploadResource = nullptr,
	FD3D12Buffer* ArgumentBuffer = nullptr, uint32 ArgumentOffset = 0)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12DispatchRays);

	// TODO: add optional validation that all (used/valid) shader identifiers used in the SBT are also available in the RTPSO

	FD3D12Device* Device = CommandContext.GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	FD3D12Buffer* DispatchRaysDescBuffer = nullptr;

	if (ArgumentBuffer)
	{
		// Source indirect argument buffer only contains the dispatch dimensions, however D3D12 requires a full D3D12_DISPATCH_RAYS_DESC structure.
		// We create a new buffer, fill the SBT pointers on CPU and copy the dispatch dimensions into the right place.

		DispatchRaysDescBuffer = Device->GetRayTracingDispatchRaysDescBuffer(QueueType);
		FD3D12Resource* DispatchRaysDescBufferResource = DispatchRaysDescBuffer->GetResource();

		CommandContext.AddBarrier(
			DispatchRaysDescBufferResource,
			ED3D12Access::IndirectArgs,
			ED3D12Access::CopyDest,
			0);

		CommandContext.AddBarrier(
			ArgumentBuffer->GetResource(),
			ED3D12Access::SRVGraphicsNonPixel | ED3D12Access::IndirectArgs,
			ED3D12Access::CopySrc,
			0);

		CommandContext.FlushResourceBarriers();

		// Compute the allocation & copy sizes
		uint32 DispatchRayDescSize = sizeof(D3D12_DISPATCH_RAYS_DESC);
		uint32 SBTPartSize = offsetof(D3D12_DISPATCH_RAYS_DESC, Width);
		uint32 IndirectDimensionSize = DispatchRayDescSize - SBTPartSize;
		static_assert((sizeof(D3D12_DISPATCH_RAYS_DESC) - offsetof(D3D12_DISPATCH_RAYS_DESC, Width)) == sizeof(uint32) * 4, "Assume 4 uints at the end of the struct to store the dimension + alignment overhead");

		uint32 BaseRayDescBufferOffset = DispatchRaysDescBuffer->ResourceLocation.GetOffsetFromBaseOfResource();

		// Copy SBT data part of the dispatch desc to upload memory
		FD3D12ResourceLocation UploadResourceLocation(Device);
		void* Data = Device->GetDefaultFastAllocator().Allocate(DispatchRayDescSize, 256, &UploadResourceLocation);
		FMemory::Memcpy(Data, &DispatchDesc, SBTPartSize);

		CommandContext.UpdateResidency(DispatchRaysDescBufferResource);
		CommandContext.UpdateResidency(UploadResourceLocation.GetResource());
		CommandContext.UpdateResidency(ArgumentBuffer->GetResource());

		// Copy SBT data part to resource
		CommandContext.CopyBufferRegionChecked(
			DispatchRaysDescBufferResource->GetResource(), DispatchRaysDescBufferResource->GetName(),
			BaseRayDescBufferOffset,
			UploadResourceLocation.GetResource()->GetResource(), UploadResourceLocation.GetResource()->GetName(),
			UploadResourceLocation.GetOffsetFromBaseOfResource(),
			SBTPartSize
		);

		// Copy GPU computed indirect args to resource
		CommandContext.CopyBufferRegionChecked(
			DispatchRaysDescBufferResource->GetResource(), DispatchRaysDescBufferResource->GetName(),
			BaseRayDescBufferOffset + SBTPartSize,
			ArgumentBuffer->GetResource()->GetResource(), ArgumentBuffer->GetResource()->GetName(),
			ArgumentBuffer->ResourceLocation.GetOffsetFromBaseOfResource() + ArgumentOffset,
			IndirectDimensionSize
		);

		CommandContext.AddBarrier(
			DispatchRaysDescBufferResource,
			ED3D12Access::CopyDest,
			ED3D12Access::IndirectArgs,
			0);

		CommandContext.AddBarrier(
			ArgumentBuffer->GetResource(),
			ED3D12Access::CopySrc,
			ED3D12Access::SRVGraphicsNonPixel | ED3D12Access::IndirectArgs,
			0);

		CommandContext.FlushResourceBarriers();
	}

	// Setup state for RT dispatch

	// Invalidate state cache to ensure all root parameters for regular shaders are reset when non-RT work is dispatched later.
	CommandContext.StateCache.TransitionComputeState(ED3D12PipelineType::RayTracing);

	CommandContext.GraphicsCommandList();

	FD3D12RayTracingShader* RayGenShader = Pipeline->RayGenShaders.Shaders[RayGenShaderIndex];

	const FRHIShaderBindingLayout& ShaderBindingLayout = CommandContext.GetShaderBindingLayout();
	check(RayGenShader->ShaderBindingLayoutHash == ShaderBindingLayout.GetHash());

	const TArray<FRHIUniformBuffer*>& StaticUniformBuffers = CommandContext.GetStaticUniformBuffers();

	const FD3D12RootSignature* GlobalRTRootSignature = Adapter->GetGlobalRayTracingRootSignature(ShaderBindingLayout);

	bool bResourcesBound = false;
	if (OptShaderTable && OptShaderTable->DescriptorCache)
	{
		FD3D12ExplicitDescriptorCache* DescriptorCache = OptShaderTable->DescriptorCache;
		check(DescriptorCache != nullptr);

		UE::TScopeLock Lock(OptShaderTable->DispatchMutex);
		TRACE_CPUPROFILER_EVENT_SCOPE(SetRayTracingShaderResources);

		CommandContext.SetExplicitDescriptorCache(*DescriptorCache);
		CommandContext.GraphicsCommandList()->SetComputeRootSignature(Pipeline->GlobalRootSignature);

		FD3D12RayTracingGlobalResourceBinder ResourceBinder(CommandContext, *DescriptorCache);
		bResourcesBound = SetRayTracingShaderResources(RayGenShader, GlobalRTRootSignature, GlobalBindings, ResourceBinder);

		OptShaderTable->UpdateResidency(CommandContext);
	}
	else
	{
		FD3D12ExplicitDescriptorCache TransientDescriptorCache(CommandContext.GetParentDevice(), FD3D12RayTracingShaderBindingTableInternal::MaxBindingWorkers);
		TransientDescriptorCache.Init(0, MAX_SRVS + MAX_UAVS, MAX_SAMPLERS, ERHIBindlessConfiguration::RayTracing);

		CommandContext.SetExplicitDescriptorCache(TransientDescriptorCache);
		CommandContext.GraphicsCommandList()->SetComputeRootSignature(Pipeline->GlobalRootSignature);

		FD3D12RayTracingGlobalResourceBinder ResourceBinder(CommandContext, TransientDescriptorCache);
		bResourcesBound = SetRayTracingShaderResources(RayGenShader, GlobalRTRootSignature, GlobalBindings, ResourceBinder);
	}

	// Bind diagnostic buffer to allow asserts in ray generation shaders
	CommandContext.BindDiagnosticBuffer(GlobalRTRootSignature, ED3D12PipelineType::Compute);

	int8 StaticShaderBindingSlot = GlobalRTRootSignature->GetStaticShaderBindingSlot();
	if (StaticShaderBindingSlot >= 0)
	{
		for (uint32 Index = 0; Index < ShaderBindingLayout.GetNumUniformBufferEntries(); ++Index)
		{
			const FRHIUniformBufferShaderBindingLayout& LayoutEntry = ShaderBindingLayout.GetUniformBufferEntry(Index);
			const uint32 RootParameterSlotIndex = uint32(StaticShaderBindingSlot) + LayoutEntry.CBVResourceIndex;

			FRHIUniformBuffer* UniformBuffer = StaticUniformBuffers[Index];
			checkf(UniformBuffer, TEXT("Static uniform buffer at index %d is referenced in the shader binding layout but not provided in the last RHISetStaticUniformBuffers() command"), Index);

			FD3D12UniformBuffer* D3D12UniformBuffer = FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(UniformBuffer, Device->GetGPUIndex());
			if (D3D12UniformBuffer->ResourceLocation.GetGPUVirtualAddress())
			{
				const FD3D12ResourceLocation& ResourceLocation = D3D12UniformBuffer->ResourceLocation;
				CommandContext.GraphicsCommandList()->SetComputeRootConstantBufferView(RootParameterSlotIndex, ResourceLocation.GetGPUVirtualAddress());
			}
		}
	}

	if (bResourcesBound)
	{
		// Ensure residency for the ray gen
		if (RayGenUploadResource)
		{
			CommandContext.UpdateResidency(RayGenUploadResource);
		}

		CommandContext.FlushResourceBarriers();

		ID3D12StateObject* RayTracingStateObject = nullptr;

		// Select a specialized RTPSO, if one is available
		if (GRayTracingAllowSpecializedStateObjects
			&& !Pipeline->SpecializedStateObjects.IsEmpty() 
			&& !Pipeline->SpecializationIndices.IsEmpty())
		{
			int32 SpecializationIndex = Pipeline->SpecializationIndices[RayGenShaderIndex];
			if (SpecializationIndex != INDEX_NONE)
			{
				RayTracingStateObject = Pipeline->SpecializedStateObjects[SpecializationIndex];
			}
		}

		// Fall back to default full RTPSO if specialization is not available
		if (!RayTracingStateObject)
		{
			RayTracingStateObject = Pipeline->StateObject.GetReference();
		}

		Pipeline->FrameCounter.Set(CommandContext.GetFrameFenceCounter());
		
		CommandContext.RayTracingCommandList()->SetPipelineState1(RayTracingStateObject);

		if (DispatchRaysDescBuffer)
		{
			ID3D12CommandSignature* CommandSignature = Adapter->GetDispatchRaysIndirectCommandSignature();
			CommandContext.RayTracingCommandList()->ExecuteIndirect(
				CommandSignature,
				1,
				DispatchRaysDescBuffer->ResourceLocation.GetResource()->GetResource(),
				DispatchRaysDescBuffer->ResourceLocation.GetOffsetFromBaseOfResource(),
				nullptr,
				0
			);
		}
		else
		{
			CommandContext.RayTracingCommandList()->DispatchRays(&DispatchDesc);
		}
	}

	// Restore old global descriptor heaps
	CommandContext.UnsetExplicitDescriptorCache();
}

void FD3D12CommandContext::RHIRayTraceDispatch(FRHIRayTracingPipelineState* InRayTracingPipelineState, FRHIRayTracingShader* RayGenShaderRHI,
	FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
	uint32 Width, uint32 Height)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHIRayTraceDispatch);

	const FD3D12RayTracingPipelineState* Pipeline = FD3D12DynamicRHI::ResourceCast(InRayTracingPipelineState);
	FD3D12RayTracingShaderBindingTable* SBT = FD3D12DynamicRHI::ResourceCast(InSBT);
	
	FD3D12RayTracingShaderBindingTableInternal* ShaderTableForDevice = SBT->GetTableForDevice(GetParentDevice());
	checkf(!ShaderTableForDevice->bIsDirty, TEXT("The shader table contains pending modifications. CommitRayTracingBindings must be called after SetRayTracingBindings"));

	FD3D12RayTracingShader* RayGenShader = FD3D12DynamicRHI::ResourceCast(RayGenShaderRHI);
	const int32 RayGenShaderIndex = Pipeline->RayGenShaders.Find(RayGenShader->GetHash());
	checkf(RayGenShaderIndex != INDEX_NONE,
		TEXT("RayGen shader '%s' is not present in the given ray tracing pipeline. ")
		TEXT("All RayGen shaders must be declared when creating RTPSO."),
			*(RayGenShader->EntryPoint));

	const FD3D12ShaderIdentifier& RayGenShaderIdentifier = Pipeline->RayGenShaders.Identifiers[RayGenShaderIndex];
	FD3D12Resource* RayGenUploadResource = nullptr;
	D3D12_DISPATCH_RAYS_DESC DispatchDesc = ShaderTableForDevice->GetDispatchRaysDesc(GetParentDevice(), RayGenShaderIdentifier, RayGenUploadResource);

	DispatchDesc.Width = Width;
	DispatchDesc.Height = Height;
	DispatchDesc.Depth = 1;

	DispatchRays(*this, GlobalResourceBindings, Pipeline, RayGenShaderIndex, ShaderTableForDevice, DispatchDesc, QueueType, RayGenUploadResource);
}

void FD3D12CommandContext::RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* InRayTracingPipelineState, FRHIRayTracingShader* RayGenShaderRHI,
	FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
	FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHIRayTraceDispatchIndirect);
	checkf(GRHISupportsRayTracingDispatchIndirect, TEXT("RHIRayTraceDispatchIndirect may not be used because DXR 1.1 is not supported on this machine."));

	const FD3D12RayTracingPipelineState* Pipeline = FD3D12DynamicRHI::ResourceCast(InRayTracingPipelineState);
	FD3D12RayTracingShaderBindingTable* SBT = FD3D12DynamicRHI::ResourceCast(InSBT);
	
	FD3D12RayTracingShaderBindingTableInternal* ShaderTableForDevice = SBT->GetTableForDevice(GetParentDevice());
	checkf(!ShaderTableForDevice->bIsDirty, TEXT("The shader table contains pending modifications. CommitRayTracingBindings must be called after SetRayTracingBindings"));

	FD3D12RayTracingShader* RayGenShader = FD3D12DynamicRHI::ResourceCast(RayGenShaderRHI);
	const int32 RayGenShaderIndex = Pipeline->RayGenShaders.Find(RayGenShader->GetHash());
	checkf(RayGenShaderIndex != INDEX_NONE, TEXT("RayGen shader is not present in the given ray tracing pipeline. All RayGen shaders must be declared when creating RTPSO."));

	const FD3D12ShaderIdentifier& RayGenShaderIdentifier = Pipeline->RayGenShaders.Identifiers[RayGenShaderIndex];
	FD3D12Resource* RayGenUploadResource = nullptr;
	D3D12_DISPATCH_RAYS_DESC DispatchDesc = ShaderTableForDevice->GetDispatchRaysDesc(GetParentDevice(), RayGenShaderIdentifier, RayGenUploadResource);
	DispatchRays(*this, GlobalResourceBindings, Pipeline, RayGenShaderIndex, ShaderTableForDevice, DispatchDesc, QueueType, RayGenUploadResource, RetrieveObject<FD3D12Buffer>(ArgumentBuffer), ArgumentOffset);
}

static void SetRayTracingHitGroup(
	FD3D12Device* Device,
	FD3D12RayTracingShaderBindingTableInternal* ShaderTable, uint32 RecordIndex,
	FD3D12RayTracingPipelineState* Pipeline, uint32 HitGroupIndex,
	const FD3D12RayTracingGeometry* Geometry, uint32 GeometrySegmentIndex,
	uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize, const void* LooseParameterData,
	uint32 UserData,
	ERayTracingLocalShaderBindingType BindingType,
	uint32 WorkerIndex)
{
	const uint32 GPUIndex = Device->GetGPUIndex();

	// If Shader table doesn't support hit group indexing then only set the hit group identifier and it should be first record index
	if (ShaderTable->HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Disallow)
	{
		check(RecordIndex == 0 && Pipeline);
		if (Pipeline)
		{
			ShaderTable->SetHitGroupIdentifier(RecordIndex, Pipeline->HitGroupShaders.Identifiers[HitGroupIndex]);
		}
		return;
	}
	
	checkf(RecordIndex < ShaderTable->NumHitRecords, TEXT("Hit group record index is invalid. Make sure that NumGeometrySegments and NumShaderSlotsPerGeometrySegment is correct in FRayTracingShaderBindingTableInitializer."));

#if DO_CHECK
	if (Geometry != nullptr)
	{
		const uint32 NumGeometrySegments = Geometry->GetNumSegments();
		checkf(GeometrySegmentIndex < NumGeometrySegments, TEXT("Segment %d is out of range for ray tracing geometry '%s' that contains %d segments"),
			GeometrySegmentIndex, Geometry->DebugName.IsNone() ? TEXT("UNKNOWN") : *Geometry->DebugName.ToString(), NumGeometrySegments);

		// If SBT and binding are persistent then all uniform buffers need to be allocated multiframe
		if (ShaderTable->Lifetime == ERayTracingShaderBindingTableLifetime::Persistent && BindingType == ERayTracingLocalShaderBindingType::Persistent)
		{
			for (uint32 UBIndex = 0; UBIndex < NumUniformBuffers; ++UBIndex)
			{
				FRHIUniformBuffer* Resource = UniformBuffers[UBIndex];
				if (Resource)
				{
					FD3D12UniformBuffer* UB = FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(Resource, 0);
					checkf(UB->UniformBufferUsage == UniformBuffer_MultiFrame, TEXT("Trying to bind non multiframe uniform buffer to persistent SBT: %s"), *Resource->GetLayout().GetDebugName());
				}
			}
		}
	}
#endif // DO_CHECK

	ShaderTable->SetHitGroupGeometrySystemParameters(WorkerIndex, BindingType, RecordIndex, Geometry, GeometrySegmentIndex, UserData);

	if (EnumHasAnyFlags(ShaderTable->ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO) && Pipeline)
	{
		const FD3D12RayTracingShader* Shader = Pipeline->HitGroupShaders.Shaders[HitGroupIndex];

		FD3D12RayTracingShaderBindingTableInternal::FShaderRecordCacheKey CacheKey;

		// TODO: disable RecordCache when using persistent SBT
		const bool bCanUseRecordCache = GRayTracingCacheShaderRecords
			&& LooseParameterDataSize == 0 // loose parameters end up in unique constant buffers, so SBT records can't be shared
			&& NumUniformBuffers > 0 // there is no benefit from cache if no resources are being bound
			&& NumUniformBuffers <= CacheKey.MaxUniformBuffers
			&& (BindingType == ERayTracingLocalShaderBindingType::Transient || ShaderTable->Lifetime == ERayTracingShaderBindingTableLifetime::Transient); // Only transient SBTs or Transient records can use the record cache because otherwise the per record listeners need to tracked and copied as well

		ERayTracingLocalShaderBindingType RTPSOBindingType = BindingType;
		if (bCanUseRecordCache)
		{
			CacheKey = FD3D12RayTracingShaderBindingTableInternal::FShaderRecordCacheKey(NumUniformBuffers, UniformBuffers, HitGroupIndex);

			uint32* ExistingRecordIndex = ShaderTable->WorkerData[WorkerIndex].ShaderRecordCache.Find(CacheKey);
			if (ExistingRecordIndex)
			{
				// Simply copy local shader parameters from existing SBT record and set the shader identifier, skipping resource binding work.
				const uint32 OffsetFromRootSignatureStart = sizeof(FD3D12HitGroupSystemParameters);
				ShaderTable->SetHitGroupIdentifier(RecordIndex, Pipeline->HitGroupShaders.Identifiers[HitGroupIndex]);
				ShaderTable->CopyHitGroupParameters(RecordIndex, *ExistingRecordIndex, OffsetFromRootSignatureStart);

				#if DO_CHECK
				// Validate all other data against copied data
				RTPSOBindingType = ERayTracingLocalShaderBindingType::Validation;
				#else
				return;
				#endif // DO_CHECK
			}
		}

		FD3D12RayTracingLocalResourceBinder ResourceBinder(*Device, *ShaderTable, *(Shader->LocalRootSignature), RTPSOBindingType, RecordIndex, WorkerIndex, ERayTracingBindingType::HitGroup);
		const bool bResourcesBound = SetRayTracingShaderResources(Shader, Shader->LocalRootSignature,
			0, nullptr,	// BindlessParameters
			0, nullptr, // Textures
			0, nullptr, // SRVs
			NumUniformBuffers, UniformBuffers,
			0, nullptr, // Samplers
			0, nullptr, // UAVs
			LooseParameterDataSize, LooseParameterData,
			ResourceBinder);

		if (bCanUseRecordCache && bResourcesBound)
		{
			ShaderTable->WorkerData[WorkerIndex].ShaderRecordCache.FindOrAdd(CacheKey, RecordIndex);
		}

		FD3D12ShaderIdentifier ShaderIdentifier = bResourcesBound ? Pipeline->HitGroupShaders.Identifiers[HitGroupIndex] : FD3D12ShaderIdentifier::Null;
		if (RTPSOBindingType != ERayTracingLocalShaderBindingType::Validation)
		{
			ShaderTable->SetHitGroupIdentifier(RecordIndex, ShaderIdentifier);
		}
		else
		{
			ShaderTable->CompareHitGroupIdentifier(RecordIndex, ShaderIdentifier);
		}
	}
}

static void SetRayTracingCallableShader(
	FD3D12Device* Device,
	FD3D12RayTracingShaderBindingTableInternal* ShaderTable, uint32 RecordIndex,
	FD3D12RayTracingPipelineState* Pipeline, uint32 ShaderIndexInPipeline,
	uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize, const void* LooseParameterData,
	uint32 UserData,
	uint32 WorkerIndex)
{
	checkf(RecordIndex < ShaderTable->NumCallableRecords, TEXT("Callable shader record index is invalid. Make sure that NumCallableShaderSlots is correct in FRayTracingShaderBindingTableInitializer."));

	const uint32 UserDataOffset = offsetof(FD3D12HitGroupSystemParameters, RootConstants) + offsetof(FHitGroupSystemRootConstants, UserData);
	ShaderTable->SetCallableShaderParameters(RecordIndex, UserDataOffset, UserData);

	const FD3D12ShaderIdentifier* ShaderIdentifier = &FD3D12ShaderIdentifier::Null;

	if (ShaderIndexInPipeline != INDEX_NONE)
	{
		const FD3D12RayTracingShader* Shader = Pipeline->CallableShaders.Shaders[ShaderIndexInPipeline];

		FD3D12RayTracingLocalResourceBinder ResourceBinder(*Device, *ShaderTable, *(Shader->LocalRootSignature), ERayTracingLocalShaderBindingType::Transient, RecordIndex, WorkerIndex, ERayTracingBindingType::CallableShader);
		const bool bResourcesBound = SetRayTracingShaderResources(Shader, Shader->LocalRootSignature,
			0, nullptr,	// BindlessParameters
			0, nullptr, // Textures
			0, nullptr, // SRVs
			NumUniformBuffers, UniformBuffers,
			0, nullptr, // Samplers
			0, nullptr, // UAVs
			LooseParameterDataSize, LooseParameterData, // Loose parameters
			ResourceBinder);

		if (bResourcesBound)
		{
			ShaderIdentifier = &Pipeline->CallableShaders.Identifiers[ShaderIndexInPipeline];
		}
	}

	ShaderTable->SetCallableIdentifier(RecordIndex, *ShaderIdentifier);
}

static void SetRayTracingMissShader(
	FD3D12Device* Device,
	FD3D12RayTracingShaderBindingTableInternal* ShaderTable, uint32 RecordIndex,
	FD3D12RayTracingPipelineState* Pipeline, uint32 ShaderIndexInPipeline,
	uint32 NumUniformBuffers, FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize, const void* LooseParameterData,
	uint32 UserData,
	uint32 WorkerIndex)
{
	checkf(RecordIndex < ShaderTable->NumMissRecords, TEXT("Miss shader record index is invalid. Make sure that NumMissShaderSlots is correct in FRayTracingShaderBindingTableInitializer."));

	const uint32 UserDataOffset = offsetof(FD3D12HitGroupSystemParameters, RootConstants) + offsetof(FHitGroupSystemRootConstants, UserData);
	ShaderTable->SetMissShaderParameters(RecordIndex, UserDataOffset, UserData);

	const FD3D12RayTracingShader* Shader = Pipeline->MissShaders.Shaders[ShaderIndexInPipeline];

	FD3D12RayTracingLocalResourceBinder ResourceBinder(*Device, *ShaderTable, *(Shader->LocalRootSignature), ERayTracingLocalShaderBindingType::Transient, RecordIndex, WorkerIndex, ERayTracingBindingType::MissShader);
	const bool bResourcesBound = SetRayTracingShaderResources(Shader, Shader->LocalRootSignature,
		0, nullptr,	// BindlessParameters
		0, nullptr, // Textures
		0, nullptr, // SRVs
		NumUniformBuffers, UniformBuffers,
		0, nullptr, // Samplers
		0, nullptr, // UAVs
		LooseParameterDataSize, LooseParameterData, // Loose parameters
		ResourceBinder);

	ShaderTable->SetMissIdentifier(RecordIndex,
		bResourcesBound
		? Pipeline->MissShaders.Identifiers[ShaderIndexInPipeline]
		: FD3D12ShaderIdentifier::Null);
}

void FD3D12CommandContext::RHISetBindingsOnShaderBindingTable(
	FRHIShaderBindingTable* InSBT, FRHIRayTracingPipelineState* InPipeline,
	uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
	ERayTracingBindingType BindingType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RHISetBindingsOnShaderBindingTable);
	SCOPE_CYCLE_COUNTER(STAT_D3D12SetBindingsOnShaderBindingTable);

	FD3D12RayTracingShaderBindingTable* SBT = FD3D12DynamicRHI::ResourceCast(InSBT);
	FD3D12RayTracingPipelineState* Pipeline = FD3D12DynamicRHI::ResourceCast(InPipeline);

	// Pipeline shouldn't contain any shaders which have bigger local data size then currently set in the initializer
	// (Otherwise changing of local binding data size would need to supported)
	check(Pipeline == nullptr || Pipeline->MaxLocalRootSignatureSize <= SBT->GetInitializer().LocalBindingDataSize);

	FD3D12RayTracingShaderBindingTableInternal* ShaderTableForDevice = SBT->GetTableForDevice(GetParentDevice());

	FGraphEventArray TaskList;

	const uint32 NumWorkerThreads = FMath::Max<uint32>(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
	const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() 
		? FMath::Min<uint32>(NumWorkerThreads, FD3D12RayTracingShaderBindingTableInternal::MaxBindingWorkers)
		: 1;

	struct FTaskContext
	{
		uint32 WorkerIndex = 0;
	};

	TArray<FTaskContext, TInlineAllocator<FD3D12RayTracingShaderBindingTableInternal::MaxBindingWorkers>> TaskContexts;
	for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
	{
		TaskContexts.Add(FTaskContext{WorkerIndex});
	}

	auto BindingTask = [Bindings, Device = Device, ShaderTableForDevice, Pipeline, BindingType](const FTaskContext& Context, int32 CurrentIndex)
	{
		const FRayTracingLocalShaderBindings& Binding = Bindings[CurrentIndex];

		if (BindingType == ERayTracingBindingType::HitGroup)
		{
			if (Binding.BindingType != ERayTracingLocalShaderBindingType::Clear)
			{
				//UE_LOGF(LogD3D12RHI, Log, "Set hit record data for RecordIndex %d on SBT %#016llx with mode: %d", Binding.RecordIndex, ShaderTableForDevice, Binding.BindingType);
								
				const FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(Binding.Geometry);
				SetRayTracingHitGroup(Device,
				    ShaderTableForDevice, Binding.RecordIndex,
				    Pipeline, Binding.ShaderIndexInPipeline,
				    Geometry, Binding.SegmentIndex,
				    Binding.NumUniformBuffers,
				    Binding.UniformBuffers,
				    Binding.LooseParameterDataSize,
				    Binding.LooseParameterData,
				    Binding.UserData,
				    Binding.BindingType,
				    Context.WorkerIndex);
			}
			else
			{
				//UE_LOGF(LogD3D12RHI, Log, "Clear hit record data for RecordIndex %d on SBT %#016llx", Binding.RecordIndex, ShaderTableForDevice);

				ShaderTableForDevice->ClearHitRecordData(Context.WorkerIndex, Binding.RecordIndex);
			}
		}
		else if (BindingType == ERayTracingBindingType::CallableShader)
		{
			check(Binding.BindingType == ERayTracingLocalShaderBindingType::Transient);
			SetRayTracingCallableShader(Device,
				ShaderTableForDevice, Binding.RecordIndex,
				Pipeline, Binding.ShaderIndexInPipeline,
				Binding.NumUniformBuffers,
				Binding.UniformBuffers,
				Binding.LooseParameterDataSize,
				Binding.LooseParameterData,
				Binding.UserData,
				Context.WorkerIndex);
		}
		else if (BindingType == ERayTracingBindingType::MissShader)
		{
			check(Binding.BindingType == ERayTracingLocalShaderBindingType::Transient);
			SetRayTracingMissShader(Device, 
				ShaderTableForDevice, Binding.RecordIndex,
				Pipeline, Binding.ShaderIndexInPipeline,
				Binding.NumUniformBuffers,
				Binding.UniformBuffers,
				Binding.LooseParameterDataSize,
				Binding.LooseParameterData,
				Binding.UserData,
				Context.WorkerIndex);
		}
		else
		{
			checkNoEntry();
		}
	};

	// One helper worker task will be created at most per this many work items, plus one worker for current thread (unless running on a task thread),
	// up to a hard maximum of FD3D12RayTracingScene::MaxBindingWorkers.
	// Internally, parallel for tasks still subdivide the work into smaller chunks and perform fine-grained load-balancing.
	const int32 ItemsPerTask = 1024;

	ParallelForWithExistingTaskContext(TEXT("RHISetBindingsOnShaderBindingTable_ParallelFor"), MakeArrayView(TaskContexts), NumBindings, ItemsPerTask, BindingTask);

	ShaderTableForDevice->bIsDirty = true;
}

// -------------------------------------------------------------------------------------------------------------------------
//   Ray Tracing Cluster Operations
// -------------------------------------------------------------------------------------------------------------------------

// --- Helpers ---

#if WITH_NVAPI
static NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAGS TranslateClusterOperationFlags(const ERayTracingClusterOperationFlags& Flags)
{
	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAGS Result = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAG_NONE;

	const bool bFastTrace = EnumHasAnyFlags(Flags, ERayTracingClusterOperationFlags::FAST_TRACE);
	const bool bFastBuild = EnumHasAnyFlags(Flags, ERayTracingClusterOperationFlags::FAST_BUILD);

	if (bFastTrace)
	{
		Result |= NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAG_FAST_TRACE;
	}

	if (!bFastTrace && bFastBuild)
	{
		Result |= NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAG_FAST_BUILD;
	}

	if (EnumHasAnyFlags(Flags, ERayTracingClusterOperationFlags::ALLOW_OMM))
	{
		Result |= NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAG_ALLOW_OMM;
	}

	if (EnumHasAnyFlags(Flags, ERayTracingClusterOperationFlags::NO_OVERLAP))
	{
		Result |= NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_FLAG_NO_OVERLAP;
	}

	return Result;
}

static NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE TranslateClusterOperationMode(const ERayTracingClusterOperationMode& Mode)
{
	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE Result = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE_IMPLICIT_DESTINATIONS;

	switch (Mode)
	{
	case ERayTracingClusterOperationMode::IMPLICIT_DESTINATIONS:
		Result = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE_IMPLICIT_DESTINATIONS;
		break;

	case ERayTracingClusterOperationMode::EXPLICIT_DESTINATIONS:
		Result = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE_EXPLICIT_DESTINATIONS;
		break;

	case ERayTracingClusterOperationMode::GET_SIZES:
		Result = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MODE_GET_SIZES;
		break;

	default:
		checkNoEntry();
		break;
	}

	return Result;
}

static DXGI_FORMAT TranslateCLASBuildOperationVertexFormat(const FRayTracingClusterOperationCLASInitializer& Initializer)
{
	DXGI_FORMAT Format = DXGI_FORMAT_UNKNOWN;
	switch (Initializer.VertexFormat)
	{
	case VET_Float3:
		Format = DXGI_FORMAT_R32G32B32_FLOAT;
		break;
	default:
		checkNoEntry();
		break;
	}
	check(Format != DXGI_FORMAT_UNKNOWN);
	return Format;
}

static NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUT_TRIANGLES_DESC TranslateClusterTriangleDesc(const FRayTracingClusterOperationCLASInitializer& Initializer)
{
	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUT_TRIANGLES_DESC TriangleDesc = {};
	TriangleDesc.vertexFormat = TranslateCLASBuildOperationVertexFormat(Initializer);
	TriangleDesc.maxGeometryIndexValue = Initializer.MaxGeometryIndex;
	TriangleDesc.maxUniqueGeometryCountPerArg = Initializer.MaxUniqueGeometryCount;
	TriangleDesc.maxTriangleCountPerArg = Initializer.MaxTriangleCount;
	TriangleDesc.maxVertexCountPerArg = Initializer.MaxVertexCount;
	TriangleDesc.maxTotalTriangleCount = Initializer.MaxTotalTriangleCount;
	TriangleDesc.maxTotalVertexCount = Initializer.MaxTotalVertexCount;
	TriangleDesc.minPositionTruncateBitCount = Initializer.MinPositionTruncateBitCount;
	return TriangleDesc;
}

// --- Cluster Op Translation ---

static uint32 TranslateMoveOperation(const FRayTracingClusterOperationMoveInitializer& Initializer, NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS& Inputs)
{
	Inputs.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_TYPE_MOVE_CLUSTER_OBJECT;
	Inputs.movesDesc.maxBytesMoved = Initializer.MaxBytes;

	switch (Initializer.Type)
	{
	case ERayTracingClusterOperationMoveType::BOTTOM_LEVEL:
		Inputs.movesDesc.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MOVE_TYPE_BOTTOM_LEVEL_ACCELERATION_STRUCTURE;
		break;
	case ERayTracingClusterOperationMoveType::CLUSTER_LEVEL:
		Inputs.movesDesc.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MOVE_TYPE_CLUSTER_LEVEL_ACCELERATION_STRUCTURE;
		break;
	case ERayTracingClusterOperationMoveType::TEMPLATE:
		Inputs.movesDesc.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_MOVE_TYPE_TEMPLATE;
		break;
	default:
		checkNoEntry();
	}

	check(sizeof(RAYTRACING_CLUSTER_OPS_MOVE_DESC) == sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_MOVE_ARGS));
	return sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_MOVE_ARGS);
}

static uint32 TranslateCLASBuildOperation(const FRayTracingClusterOperationCLASInitializer& Initializer, NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS& Inputs)
{
	Inputs.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_TYPE_BUILD_CLAS_FROM_TRIANGLES;
	Inputs.trianglesDesc = TranslateClusterTriangleDesc(Initializer);

	static_assert(sizeof(RAYTRACING_CLUSTER_OPS_BUILD_CLAS_DESC) == sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_TRIANGLE_CLUSTER_ARGS));
	return sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_TRIANGLE_CLUSTER_ARGS);
}

static uint32 TranslateCLASTemplateBuildOperation(const FRayTracingClusterOperationCLASInitializer& Initializer, NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS& Inputs)
{
	Inputs.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_TYPE_BUILD_CLUSTER_TEMPLATES_FROM_TRIANGLES;
	Inputs.trianglesDesc = TranslateClusterTriangleDesc(Initializer);

	static_assert(sizeof(RAYTRACING_CLUSTER_OPS_BUILD_CLAS_TEMPLATE_DESC) == sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_TRIANGLE_TEMPLATE_ARGS));
	return sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_TRIANGLE_TEMPLATE_ARGS);
}

static uint32 TranslateCLASTemplateInstantiateOperation(const FRayTracingClusterOperationCLASInitializer& Initializer, NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS& Inputs)
{
	Inputs.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_TYPE_INSTANTIATE_CLUSTER_TEMPLATES;
	Inputs.trianglesDesc = TranslateClusterTriangleDesc(Initializer);

	static_assert(sizeof(RAYTRACING_CLUSTER_OPS_INSTANTIATE_CLAS_TEMPLATE_DESC) == sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_INSTANTIATE_TEMPLATE_ARGS));
	return sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_INSTANTIATE_TEMPLATE_ARGS);
}

static uint32 TranslateBLASBuildOperation(const FRayTracingClusterOperationBLASInitializer& Initializer, NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS& Inputs)
{
	Inputs.type = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_TYPE_BUILD_BLAS_FROM_CLAS;
	Inputs.clasDesc.maxTotalClasCount = Initializer.MaxTotalCLASCount;
	Inputs.clasDesc.maxClasCountPerArg = Initializer.MaxCLASPerBLASCount;

	check(sizeof(RAYTRACING_CLUSTER_OPS_BUILD_BLAS_DESC) == sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_CLUSTER_ARGS));
	return sizeof(NVAPI_D3D12_RAYTRACING_ACCELERATION_STRUCTURE_MULTI_INDIRECT_CLUSTER_ARGS);
}

#endif // WITH_NVAPI

// --- Memory Requirements ---

// Determines memory requirements for the specified cluster operation
FRayTracingClusterOperationSize FD3D12DynamicRHI::RHICalcRayTracingClusterOperationSize(const FRayTracingClusterOperationInitializer& Initializer)
{
#if WITH_NVAPI
	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS Inputs = {};
	Inputs.maxArgCount = Initializer.MaxResultCount;
	Inputs.mode = TranslateClusterOperationMode(Initializer.Mode);
	Inputs.flags = TranslateClusterOperationFlags(Initializer.Flags);

	switch (Initializer.Type)
	{
	case ERayTracingClusterOperationType::MOVE:
		TranslateMoveOperation(Initializer.Operation.Move, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_BUILD:
		TranslateCLASBuildOperation(Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_BUILD_TEMPLATES:
		TranslateCLASTemplateBuildOperation(Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_INSTANTIATE_TEMPLATES:
		TranslateCLASTemplateInstantiateOperation(Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::BLAS_BUILD:
		TranslateBLASBuildOperation(Initializer.Operation.BLAS, Inputs);
		break;

	default:
		checkNoEntry();
		break;
	}

	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_REQUIREMENTS_INFO Info = {};

	NVAPI_GET_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_REQUIREMENTS_INFO_PARAMS Params;
	Params.version = NVAPI_GET_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_REQUIREMENTS_INFO_PARAMS_VER;
	Params.pInput = &Inputs;
	Params.pInfo = &Info;

	NvAPI_Status Result = NvAPI_D3D12_GetRaytracingMultiIndirectClusterOperationRequirementsInfo(GetAdapter().GetD3DDevice5(), &Params);
	check(Result == NVAPI_OK);

	FRayTracingClusterOperationSize Size = {};
	Size.ResultMaxSizeInBytes = Info.resultDataMaxSizeInBytes;
	Size.ScratchSizeInBytes = Info.scratchDataSizeInBytes;

	return Size;
#else
	checkNoEntry();
	return {};
#endif // WITH_NVAPI
}

// --- Command List Functions ---

// Validate arguments, resources, residency, and execute the specified cluster operation
void FD3D12CommandContext::RHIExecuteMultiIndirectClusterOperation(const FRayTracingClusterOperationParams& Params)
{
#if WITH_NVAPI
	TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteMultiIndirectClusterOperation);
	SCOPE_CYCLE_COUNTER(STAT_D3D12ClusterOperation);
	LLM_SCOPE_BYNAME(TEXT("FD3D12RT/ClusterOperation"));

	// Early out: no acceleration structures to build, instantiate, or move
	if (Params.Initializer.MaxResultCount == 0) return;

	// Validate resource buffers
	checkf(Params.Resources.In.ResultCount != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid result count input buffer"));
	checkf(Params.Resources.In.Descriptors != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid descriptors input buffer"));

	checkf(Params.Resources.In.Scratch != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid scratch input buffer"));

	checkf(Params.Resources.InOut.Addresses != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid addresses input/output buffer"));

	if (Params.Initializer.Mode == ERayTracingClusterOperationMode::IMPLICIT_DESTINATIONS)
	{
		checkf(Params.Resources.Out.AccelerationStructures != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid acceleration structures output buffer"));
	}
	if (Params.Initializer.Mode == ERayTracingClusterOperationMode::GET_SIZES)
	{
		checkf(Params.Resources.Out.Sizes != nullptr, TEXT("ExecuteMultiIndirectClusterOperation requires a valid sizes output buffer when in GET_SIZES mode"));
	}

	uint32 DescriptorStride = 0;

	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_INPUTS Inputs = {};
	Inputs.maxArgCount = Params.Initializer.MaxResultCount;
	Inputs.mode = TranslateClusterOperationMode(Params.Initializer.Mode);
	Inputs.flags = TranslateClusterOperationFlags(Params.Initializer.Flags);

	switch (Params.Initializer.Type)
	{
	case ERayTracingClusterOperationType::MOVE:
		DescriptorStride = TranslateMoveOperation(Params.Initializer.Operation.Move, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_BUILD:
		DescriptorStride = TranslateCLASBuildOperation(Params.Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_BUILD_TEMPLATES:
		DescriptorStride = TranslateCLASTemplateBuildOperation(Params.Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::CLAS_INSTANTIATE_TEMPLATES:
		DescriptorStride = TranslateCLASTemplateInstantiateOperation(Params.Initializer.Operation.CLAS, Inputs);
		break;

	case ERayTracingClusterOperationType::BLAS_BUILD:
		DescriptorStride = TranslateBLASBuildOperation(Params.Initializer.Operation.BLAS, Inputs);
		break;

	default:
		checkNoEntry();
		break;
	}

	// Inputs
	FD3D12Buffer* InResultCount = FD3D12DynamicRHI::ResourceCast(Params.Resources.In.ResultCount, GetGPUIndex());
	FD3D12Buffer* InDescriptors = FD3D12DynamicRHI::ResourceCast(Params.Resources.In.Descriptors, GetGPUIndex());

	FD3D12Buffer* InScratch = FD3D12DynamicRHI::ResourceCast(Params.Resources.In.Scratch, GetGPUIndex());

	// Input/Output
	FD3D12Buffer* InOutAddresses = FD3D12DynamicRHI::ResourceCast(Params.Resources.InOut.Addresses, GetGPUIndex());

	// Outputs
	FD3D12Buffer* OutAccelerationStructures = FD3D12DynamicRHI::ResourceCast(Params.Resources.Out.AccelerationStructures, GetGPUIndex());
	FD3D12Buffer* OutSizes = (Params.Resources.Out.Sizes != nullptr) ? FD3D12DynamicRHI::ResourceCast(Params.Resources.Out.Sizes, GetGPUIndex()) : nullptr;

	// Residency
	UpdateResidency(InResultCount->GetResource());
	UpdateResidency(InDescriptors->GetResource());

	// Update residency for additional resources used by the command (e.g. referenced by descriptor VAs)
	for (FRHIBuffer* InAdditionalResource : Params.Resources.In.AdditionalResources)
	{
		FD3D12Buffer* Resource = FD3D12DynamicRHI::ResourceCast(InAdditionalResource, GetGPUIndex());
		if (Resource)
		{
			UpdateResidency(Resource->GetResource());
		}
	}

	UpdateResidency(InScratch->GetResource());
	UpdateResidency(InOutAddresses->GetResource());
	if (OutAccelerationStructures) UpdateResidency(OutAccelerationStructures->GetResource());
	if (OutSizes) UpdateResidency(OutSizes->GetResource());

	FlushResourceBarriers();

	// Describe the cluster operation
	NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_DESC Desc = {};
	Desc.inputs = Inputs;

	// Address Resolution Flags
	Desc.addressResolutionFlags = NVAPI_D3D12_RAYTRACING_MULTI_INDIRECT_CLUSTER_OPERATION_ADDRESS_RESOLUTION_FLAG_NONE;

	// Input Buffers
	Desc.indirectArgCount = InResultCount->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.In.ResultCountOffsetInBytes;
	Desc.indirectArgArray.StartAddress = InDescriptors->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.In.DescriptorsOffsetInBytes;
	Desc.indirectArgArray.StrideInBytes = DescriptorStride;
	Desc.batchScratchData = InScratch->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.In.ScratchOffsetInBytes;

	// Input / Output Buffers
	Desc.destinationAddressArray.StartAddress = InOutAddresses->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.InOut.AddressesOffsetInBytes;
	Desc.destinationAddressArray.StrideInBytes = sizeof(uint64);

	// Output Buffers
	if (OutAccelerationStructures)
	{
		Desc.batchResultData = OutAccelerationStructures->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.Out.AccelerationStructuresOffsetInBytes;
	}
	if (OutSizes)
	{
		Desc.resultSizeArray.StartAddress = OutSizes->ResourceLocation.GetGPUVirtualAddress() + Params.Resources.Out.SizesOffsetInBytes;
		Desc.resultSizeArray.StrideInBytes = sizeof(uint32);
	}

	NVAPI_RAYTRACING_EXECUTE_MULTI_INDIRECT_CLUSTER_OPERATION_PARAMS ClusterOpParams = {};
	ClusterOpParams.version = NVAPI_RAYTRACING_EXECUTE_MULTI_INDIRECT_CLUSTER_OPERATION_PARAMS_VER;
	ClusterOpParams.pDesc = &Desc;

	// Execute the Cluster Operation
	NvAPI_Status Result = NvAPI_D3D12_RaytracingExecuteMultiIndirectClusterOperation(GraphicsCommandList4().Get(), &ClusterOpParams);
	check(Result == NVAPI_OK);

	// Wait for the cluster ops to finish
	AddGlobalBarrier(ED3D12Access::UAVCompute, ED3D12Access::UAVCompute);

#else
	checkNoEntry();
#endif // WITH_NVAPI
}

#endif // D3D12_RHI_RAYTRACING
