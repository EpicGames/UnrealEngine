// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIPrivate.h"

#if D3D12_RHI_RAYTRACING

#include "D3D12RayTracingResources.h"

static_assert(sizeof(FD3D12_GPU_VIRTUAL_ADDRESS) == sizeof(D3D12_GPU_VIRTUAL_ADDRESS), "Size of FD3D12_GPU_VIRTUAL_ADDRESS must match D3D12_GPU_VIRTUAL_ADDRESS");

class FD3D12RayTracingPipelineState;
class FD3D12RayTracingShaderBindingTable;

class FD3D12RayTracingGeometry;

/** Persistent SBT needs to be notified about hit group parameter changes because those are cached in the SBT - if persistent bindless handles are used then this could be removed */
struct ID3D12RayTracingGeometryUpdateListener
{
	virtual void RemoveListener(FD3D12RayTracingGeometry* InGeometry) = 0;
	virtual void HitGroupParametersUpdated(FD3D12RayTracingGeometry* InGeometry) = 0;
};

struct FD3D12ShaderIdentifier
{
	uint64 Data[4] = {~0ull, ~0ull, ~0ull, ~0ull};

	// No shader is executed if a shader binding table record with null identifier is encountered.
	static const FD3D12ShaderIdentifier Null;

	bool operator == (const FD3D12ShaderIdentifier& Other) const
	{
		return Data[0] == Other.Data[0]
			&& Data[1] == Other.Data[1]
			&& Data[2] == Other.Data[2]
			&& Data[3] == Other.Data[3];
	}

	bool operator != (const FD3D12ShaderIdentifier& Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return *this != FD3D12ShaderIdentifier();
	}

	void SetData(const void* InData)
	{
		FMemory::Memcpy(Data, InData, sizeof(Data));
	}
};

struct FD3D12RayTracingShaderLibrary
{
	void Reserve(uint32 NumShaders)
	{
		Shaders.Reserve(NumShaders);
		Identifiers.Reserve(NumShaders);
	}

	int32 Find(FShaderHash Hash) const
	{
		for (int32 Index = 0; Index < Shaders.Num(); ++Index)
		{
			if (Hash == Shaders[Index]->GetHash())
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	TArray<TRefCountPtr<FD3D12RayTracingShader>> Shaders;
	TArray<FD3D12ShaderIdentifier> Identifiers;
};

class FD3D12RayTracingPipelineState : public FRHIRayTracingPipelineState
{
public:
	UE_NONCOPYABLE(FD3D12RayTracingPipelineState)

	FD3D12RayTracingPipelineState(FD3D12Device* Device, const FRayTracingPipelineStateInitializer& Initializer);

	FD3D12Device* Device;
	
	FD3D12RayTracingShaderLibrary RayGenShaders;
	FD3D12RayTracingShaderLibrary MissShaders;
	FD3D12RayTracingShaderLibrary HitGroupShaders;
	FD3D12RayTracingShaderLibrary CallableShaders;

	ID3D12RootSignature* GlobalRootSignature = nullptr;

	TRefCountPtr<ID3D12StateObject> StateObject;
	TRefCountPtr<ID3D12StateObjectProperties> PipelineProperties;

	// Maps raygen shader index to a specialized state object (may be -1 if no specialization is used for a shader)
	TArray<int32> SpecializationIndices;

	// State objects with raygen shaders grouped by occupancy
	TArray<TRefCountPtr<ID3D12StateObject>> SpecializedStateObjects;

	uint32 MaxLocalRootSignatureSize = 0;
	uint32 MaxHitGroupViewDescriptors = 0;

	TSet<uint64> PipelineShaderHashes;

	uint32 PipelineStackSize = 0;

#if !NO_LOGGING
	struct FShaderStats
	{
		const TCHAR* Name = nullptr;
		float CompileTimeMS = 0;
		uint32 StackSize = 0;
		uint32 ShaderSize = 0;
	};
	TArray<FShaderStats> ShaderStats;
#endif // !NO_LOGGING

	D3D12ResourceFrameCounter FrameCounter;
};

class FD3D12RayTracingGeometry : public FRHIRayTracingGeometry, public FD3D12AdapterChild, public FD3D12ShaderResourceRenameListener, public FNoncopyable
{
public:

	FD3D12RayTracingGeometry(FRHICommandListBase& RHICmdList, FD3D12Adapter* Adapter, const FRayTracingGeometryInitializer& Initializer);
	~FD3D12RayTracingGeometry();
	
	D3D12_GPU_VIRTUAL_ADDRESS GetGPUVirtualAddress(uint64 GPUIndex) const
	{
		checkf(PerGPUData[GPUIndex].AccelerationStructureBuffer,
			TEXT("Trying to get address of acceleration structure '%s' without allocated memory."), *DebugName.ToString());
		return PerGPUData[GPUIndex].AccelerationStructureBuffer->ResourceLocation.GetGPUVirtualAddress();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FRayTracingAccelerationStructureAddress GetAccelerationStructureAddress(uint64 GPUIndex) const final override
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return GetGPUVirtualAddress(GPUIndex);
	}

	void SetupHitGroupSystemParameters(uint32 InGPUIndex);
	void UpdateResidency(FD3D12CommandContext& CommandContext);
	void CompactAccelerationStructure(FD3D12CommandContext& CommandContext, uint32 InGPUIndex, uint64 InSizeAfterCompaction);
	void CreateAccelerationStructureBuildDesc(FD3D12CommandContext& CommandContext, EAccelerationStructureBuildMode BuildMode, D3D12_GPU_VIRTUAL_ADDRESS ScratchBufferAddress,
		D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC& OutDesc, TArrayView<D3D12_RAYTRACING_GEOMETRY_DESC>& OutGeometryDescs) const;
#if WITH_NVAPI
	void CreateAccelerationStructureBuildDescEx(FD3D12CommandContext& CommandContext, EAccelerationStructureBuildMode BuildMode, D3D12_GPU_VIRTUAL_ADDRESS ScratchBufferAddress,
		NVAPI_D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_DESC_EX& OutDescEx, TArrayView<NVAPI_D3D12_RAYTRACING_GEOMETRY_DESC_EX>& OutGeometryDescsEx) const;
#endif

	// Implement FD3D12ShaderResourceRenameListener interface
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override;

	bool AllocateBufferSRVs(uint32 InGPUIndex);

	void RegisterAsRenameListener(uint32 InGPUIndex);
	void UnregisterAsRenameListener(uint32 InGPUIndex);

	void Swap(FD3D12RayTracingGeometry& Other);

	void ReleaseUnderlyingResource();

	void SetDirty(FRHIGPUMask GPUMask, bool bState)
	{
		for (uint32 GPUIndex : GPUMask)
		{
			PerGPUData[GPUIndex].bIsAccelerationStructureDirty = bState;
		}
	}
	bool IsDirty(uint32 GPUIndex) const
	{
		return PerGPUData[GPUIndex].bIsAccelerationStructureDirty;
	}
	bool BuffersValid(uint32 GPUIndex) const;

	using FRHIRayTracingGeometry::Initializer;
	using FRHIRayTracingGeometry::SizeInfo;

	static constexpr uint32 IndicesPerTriangle = 3;
	static constexpr uint32 IndicesPerLinearSweptSphere = 2;

	struct FPerGPUState
	{
		TRefCountPtr<FD3D12Buffer> AccelerationStructureBuffer;
		bool bIsAccelerationStructureDirty = false;
		bool bRegisteredAsRenameListener = false;
		bool bHasPendingCompactionRequests = false;
		TArray<FD3D12HitGroupSystemParameters> HitGroupSystemParameters;
		TSharedPtr<FD3D12ShaderResourceView> HitGroupSystemIndexBufferSRV;
		TArray<TSharedPtr<FD3D12ShaderResourceView>> HitGroupSystemSegmentVertexBufferSRVs;
	};
	TArray<FPerGPUState, TInlineAllocator<1>> PerGPUData;

	FDebugName DebugName;
	FName OwnerName;		// Store the path name of the owner object for resource tracking

	uint64 AccelerationStructureCompactedSize = 0;
#if ENABLE_RESIDENCY_MANAGEMENT
	uint32 ResourceGeneration = 0;
#endif
		
public:

	void AddUpdateListener(ID3D12RayTracingGeometryUpdateListener* InUpdateListener) const
	{
		FScopeLock Lock(&UpdateListenersCS);
		check(!UpdateListeners.Contains(InUpdateListener));
		UpdateListeners.Add(InUpdateListener);
	}

	void RemoveUpdateListener(ID3D12RayTracingGeometryUpdateListener* InUpdateListener) const
	{
		FScopeLock Lock(&UpdateListenersCS);
		uint32 Removed = UpdateListeners.Remove(InUpdateListener);

		checkf(Removed == 1, TEXT("Should have exactly one registered listener during remove (same listener shouldn't registered twice and we shouldn't call this if not registered"));
	}

	bool HasListeners() const
	{
		FScopeLock Lock(&UpdateListenersCS);
		return UpdateListeners.Num() != 0;
	}

	void HitGroupParamatersUpdated()
	{
		FScopeLock Lock(&UpdateListenersCS);
		for (ID3D12RayTracingGeometryUpdateListener* UpdateListener : UpdateListeners)
		{
			UpdateListener->HitGroupParametersUpdated(this);
		}
	}

private:
	mutable FCriticalSection UpdateListenersCS;
	mutable TArray<ID3D12RayTracingGeometryUpdateListener*> UpdateListeners;
};

class FD3D12RayTracingScene : public FRHIRayTracingScene, public FD3D12AdapterChild, public FNoncopyable
{
public:

	FD3D12RayTracingScene(FD3D12Adapter* Adapter, FRayTracingSceneInitializer Initializer);
	~FD3D12RayTracingScene();

	const FRayTracingSceneInitializer& GetInitializer() const override final { return Initializer; }

	void BindBuffer(FRHIBuffer* Buffer, uint32 BufferOffset);
	void ReleaseBuffer();

	using FRHIRayTracingAccelerationStructure::SizeInfo;
	using FRHIRayTracingScene::ReferencedGeometries;
	using FRHIRayTracingScene::ReferencedBuffers;
	using FRHIRayTracingScene::SetReferencedResources;

	uint32 NumInstances = 0;

#if ENABLE_RESIDENCY_MANAGEMENT
	struct FGeometryResidencyInfo
	{
		uint32 ResourceGeneration = 0;
		const FD3D12Resource* CachedASResource = nullptr;
		TArray<const FD3D12Resource*, TInlineAllocator<2>> CachedDispatchResources;
	};
#endif

	struct FPerGPUState
	{
		TRefCountPtr<FD3D12Buffer> AccelerationStructureBuffer;

#if ENABLE_RESIDENCY_MANAGEMENT
		// Resources that must be resident when TLAS is built or traced against (BLAS acceleration structures).
		TArray<const FD3D12Resource*> ResourcesToMakeResident;
		TMap<const FD3D12Resource*, int32> ResourceRefCounts;

		// Resources that only need to be resident at dispatch time (VB/IB).
		// TODO: Once TriangleObjectPositions() is supported across all IHVs,
		// remove this logic and move VB/IB residency tracking to SBT when using hit shaders.
		TArray<const FD3D12Resource*> DispatchResourcesToMakeResident;
		TMap<const FD3D12Resource*, int32> DispatchResourceRefCounts;

		// Incremental residency tracking
		TMap<FRHIRayTracingGeometry*, FGeometryResidencyInfo> TrackedGeometries;
#endif // ENABLE_RESIDENCY_MANAGEMENT
	};
	TArray<FPerGPUState, TInlineAllocator<1>> PerGPUData;
	uint32 BufferOffset = 0;

	const FRayTracingSceneInitializer Initializer;

#if ENABLE_RESIDENCY_MANAGEMENT
	void UpdateResidencyTracking(FD3D12CommandContext& CommandContext);
#endif // ENABLE_RESIDENCY_MANAGEMENT

	void UpdateResidency(FD3D12CommandContext& CommandContext, bool bIncludeDispatchResources) const;

	bool bBuilt = false;
};

// Manages all the pending BLAS compaction requests
class FD3D12RayTracingCompactionRequestHandler : FD3D12DeviceChild
{
public:

	UE_NONCOPYABLE(FD3D12RayTracingCompactionRequestHandler)

	FD3D12RayTracingCompactionRequestHandler(FD3D12Device* Device);
	~FD3D12RayTracingCompactionRequestHandler()
	{
		check(PendingRequests.IsEmpty());
	}

	void RequestCompact(FD3D12RayTracingGeometry* InRTGeometry);
	bool ReleaseRequest(FD3D12RayTracingGeometry* InRTGeometry);

	void Update(FD3D12CommandContext& InCommandContext);

private:

	FCriticalSection CS;
	TArray<FD3D12RayTracingGeometry*> PendingRequests;
	TArray<FD3D12RayTracingGeometry*> ActiveRequests;
	TArray<D3D12_GPU_VIRTUAL_ADDRESS> ActiveBLASGPUAddresses;

	TRefCountPtr<FD3D12Buffer> PostBuildInfoBuffer;
	FStagingBufferRHIRef PostBuildInfoStagingBuffer;
	FD3D12SyncPointRef PostBuildInfoBufferReadbackSyncPoint;
};

#endif // D3D12_RHI_RAYTRACING
