// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RayTracingDebug.h"

#if D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING

#include "D3D12RayTracing.h"
#include "HAL/FileManagerGeneric.h"
#include "Serialization/BufferArchive.h"
#include "Containers/StringConv.h"

static bool GRayTracingSerializeSceneNextFrame = false;
static FAutoConsoleCommand RayTracingSerializeSceneCmd(
	TEXT("D3D12.RayTracing.SerializeScene"),
	TEXT("Serialize Ray Tracing Scene to disk."),
	FConsoleCommandDelegate::CreateStatic([] { GRayTracingSerializeSceneNextFrame = true; }));

static void DebugSerializeScene(const FD3D12RayTracingScene& Scene, FD3D12Buffer* InstanceBuffer, uint32 InstanceBufferOffset, FD3D12CommandContext& CommandContext)
{
	// #dxr_todo: this could potentially be generalized and moved to high-level code, to be reused for all RHIs if we implement instance desc structure decoding

	const FRayTracingSceneInitializer& SceneInitializer = Scene.GetInitializer();

	FString Name = SceneInitializer.DebugName.ToString();
	FString Filename = FString::Printf(TEXT("Scene_%s"), *Name);
	FString RootPath = FPaths::ScreenShotDir();
	FString OutputFilename = RootPath + Filename + TEXT(".uehwrtscene");

	TArray<uint8> OutputBuffer;

	// Serialization constants

	static constexpr uint64 SceneHeaderMagic = 0x72ffa3376f48683b;
	static constexpr uint64 SceneHeaderVersion = 1;
	static constexpr uint64 BlasHeaderMagic = 0x47226e42ad539683;
	static constexpr uint64 BufferHeaderMagic = 0x7330d54d0195a6de;
	static constexpr uint64 StringTableHeaderMagic = 0xe8f1516d6537c909;
	// All buffers are vector-aligned to allow efficient loading into GPU memory and subsequent GPU access.
	// Serialized scene data is expected to be loaded all at once from disk and copied into a single GPU buffer.
	static constexpr uint32 AlignmentRequirement = 16;

	// Serialized types

	struct FSceneHeader
	{
		uint64 Magic = SceneHeaderMagic;
		uint64 Version = SceneHeaderVersion;

		// Section offsets
		struct FOffsets
		{
			// GPU-relevant data
			uint32 Instances = 0;
			uint32 Geometries = 0;
			uint32 Buffers = 0;
			// Misc data that can be skipped when uploading to GPU
			uint32 Strings = 0;
		} Offsets;

		uint32 NumInstances = 0;
		uint32 NumGeometries = 0;
		uint32 NumBuffers = 0;
		uint32 NumStrings = 0;
	};
	static_assert(sizeof(FSceneHeader) % AlignmentRequirement == 0, "Serialized scene data must be vector-aligned");

	struct FGeometryHeader
	{
		uint64 Magic = BlasHeaderMagic;
		uint32 IndexBuffer = 0;
		uint32 NumSegments = 0;
		uint32 NameString = 0;
		uint32 OwnerString = 0;
		uint64 Padding[1] = {};
	};
	static_assert(sizeof(FGeometryHeader) % AlignmentRequirement == 0, "Serialized scene data must be vector-aligned");

	struct FBufferHeader
	{
		uint64 Magic = BufferHeaderMagic;
		uint32 SizeInBytes = 0;
		uint32 StrideInBytes = 0;
	};
	static_assert(sizeof(FBufferHeader) % AlignmentRequirement == 0, "Serialized scene data must be vector-aligned");

	struct FSegmentHeader
	{
		uint32 VertexBuffer = 0;
		uint32 VertexType = 0; // EVertexElementType
		uint32 VertexBufferOffset = 0;
		uint32 VertexBufferStride = 0;
		uint32 MaxVertices = 0;
		uint32 FirstPrimitive = 0;
		uint32 NumPrimitives = 0;
		uint8 bForceOpaque = 0;
		uint8 bAllowDuplicateAnyHitShaderInvocation = 0;
		uint8 bEnabled = 0;
		uint8 Padding[1] = {};
	};
	static_assert(sizeof(FSegmentHeader) % AlignmentRequirement == 0, "Serialized scene data must be vector-aligned");

	// Serialize everything

	TMap<D3D12_GPU_VIRTUAL_ADDRESS, uint32> GeometryMap;
	TMap<FRHIBuffer*, uint32> BufferMap;
	TMap<FString, uint32> StringMap;

	// ID 0 is reserved for nullptr
	GeometryMap.Add(0, 0);
	BufferMap.Add(nullptr, 0);
	StringMap.Add(FString(), 0);

	enum EAlignmentMode
	{
		Unaligned,
		AlignBeginning,
		CheckBeginning,
	};

	// Append data to output buffer, return offset of the written data (after padding bytes, if any).
	// Optionally, a field can be serialized at aligned boundary. Beginning of each serialized section should be aligned.
	auto Serialize = [&OutputBuffer](const void* Data, uint32 NumBytes, EAlignmentMode AlignmentMode = EAlignmentMode::Unaligned) -> uint32
	{
		if (AlignmentMode == EAlignmentMode::CheckBeginning)
		{
			checkf(OutputBuffer.Num() % AlignmentRequirement == 0, TEXT("Serialized scene data must be vector-aligned"));
		}
		else if (AlignmentMode == EAlignmentMode::AlignBeginning)
		{
			uint32 Remainder = OutputBuffer.Num() % AlignmentRequirement;
			if (Remainder != 0)
			{
				uint32 NumPaddingBytes = AlignmentRequirement - Remainder;
				OutputBuffer.AddZeroed(NumPaddingBytes);
			}
		}

		uint32 DataOffset = OutputBuffer.Num();
		OutputBuffer.Append(reinterpret_cast<const uint8*>(Data), NumBytes);
		return DataOffset;
	};

	FSceneHeader SceneHeader;

	checkf(OutputBuffer.Num() == 0, TEXT("Scene header must be written into the buffer first"));
	Serialize(&SceneHeader, sizeof(SceneHeader), EAlignmentMode::AlignBeginning); // Reserve space for the header

	const int32 NumReferencedGeometries = Scene.ReferencedGeometries.Num();
	for (int32 GeometryIndex = 0; GeometryIndex < NumReferencedGeometries; ++GeometryIndex)
	{
		FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(Scene.ReferencedGeometries[GeometryIndex].GetReference());
		TRefCountPtr<FD3D12Buffer> BlasBuffer = Geometry->PerGPUData[0].AccelerationStructureBuffer;
		D3D12_GPU_VIRTUAL_ADDRESS Address = BlasBuffer->ResourceLocation.GetGPUVirtualAddress();
		GeometryMap.FindOrAdd(Address, GeometryMap.Num());
	}

	{
		SceneHeader.NumInstances = Scene.NumInstances;

		// Instance buffer

		FStagingBufferRHIRef StatingBuffer = RHICreateStagingBuffer();

		check(sizeof(D3D12_RAYTRACING_INSTANCE_DESC) == GRHIRayTracingInstanceDescriptorSize);

		const uint32 InstanceBufferSize = FMath::Min(InstanceBuffer->GetSize() - InstanceBufferOffset, SceneHeader.NumInstances * GRHIRayTracingInstanceDescriptorSize);
		check(InstanceBufferSize == SceneHeader.NumInstances * GRHIRayTracingInstanceDescriptorSize);

		if (SceneHeader.NumInstances != 0)
		{
			CommandContext.RHICopyToStagingBuffer(InstanceBuffer, StatingBuffer, InstanceBufferOffset, InstanceBufferSize);
			CommandContext.FlushCommands(ED3D12FlushFlags::WaitForCompletion);

			D3D12_RAYTRACING_INSTANCE_DESC* InstanceDescs = (D3D12_RAYTRACING_INSTANCE_DESC*)StatingBuffer->Lock(0, InstanceBufferSize);
			check(InstanceDescs);

			for (uint32 InstanceIndex = 0; InstanceIndex < SceneHeader.NumInstances; ++InstanceIndex)
			{
				D3D12_RAYTRACING_INSTANCE_DESC& Desc = InstanceDescs[InstanceIndex];
				uint32* FoundIndex = GeometryMap.Find(Desc.AccelerationStructure);
				if (FoundIndex)
				{
					Desc.AccelerationStructure = *FoundIndex;
				}
				else
				{
					Desc.AccelerationStructure = 0;
				}
			}

			SceneHeader.Offsets.Instances = Serialize(InstanceDescs, SceneHeader.NumInstances * GRHIRayTracingInstanceDescriptorSize, EAlignmentMode::AlignBeginning);
			checkf(SceneHeader.Offsets.Instances% AlignmentRequirement == 0, TEXT("Serialized scene data must be vector-aligned"));

			StatingBuffer->Unlock();
		}
	}

	// Save referenced BLAS geometry parameters

	{
		check(1 + NumReferencedGeometries == GeometryMap.Num());  // entry 0 is reserved for null geometry
		SceneHeader.NumGeometries = GeometryMap.Num();

		const FGeometryHeader NullHeader;
		SceneHeader.Offsets.Geometries = Serialize(&NullHeader, sizeof(NullHeader), EAlignmentMode::AlignBeginning);
		checkf(SceneHeader.Offsets.Geometries % AlignmentRequirement == 0, TEXT("Serialized scene data must be vector-aligned"));

		for (int32 GeometryIndex = 0; GeometryIndex < NumReferencedGeometries; ++GeometryIndex)
		{
			FGeometryHeader GeometryHeader;

			FD3D12RayTracingGeometry* Geometry = FD3D12DynamicRHI::ResourceCast(Scene.ReferencedGeometries[GeometryIndex].GetReference());

			FString DebugName = Geometry->DebugName.ToString();
			FString OwnerName = Geometry->OwnerName.ToString();

			GeometryHeader.NameString = StringMap.FindOrAdd(DebugName, StringMap.Num());
			GeometryHeader.OwnerString = StringMap.FindOrAdd(OwnerName, StringMap.Num());

			TRefCountPtr<FD3D12Buffer> BlasBuffer = Geometry->PerGPUData[0].AccelerationStructureBuffer;

			D3D12_GPU_VIRTUAL_ADDRESS Address = BlasBuffer->ResourceLocation.GetGPUVirtualAddress();
			const uint32& FoundId = GeometryMap.FindChecked(Address);
			check(FoundId == GeometryIndex + 1);

			// Index buffer

			const FRayTracingGeometryInitializer& GeometryInitializer = Geometry->GetInitializer();

			GeometryHeader.IndexBuffer = BufferMap.FindOrAdd(GeometryInitializer.IndexBuffer, BufferMap.Num());

			// Segments

			GeometryHeader.NumSegments = GeometryInitializer.Segments.Num();

			Serialize(&GeometryHeader, sizeof(GeometryHeader), EAlignmentMode::CheckBeginning);

			for (uint32 SegmentIndex = 0; SegmentIndex < GeometryHeader.NumSegments; ++SegmentIndex)
			{
				const FRayTracingGeometrySegment& Segment = GeometryInitializer.Segments[SegmentIndex];

				static_assert(VET_Float3 == 3 && VET_Float4 == 4, "Change in EVertexElementType invalidates the serialized data. Update HeaderVersion and this assert.");

				FSegmentHeader SegmentHeader = {};

				uint32& VertexBufferId = BufferMap.FindOrAdd(Segment.VertexBuffer, BufferMap.Num());

				SegmentHeader.VertexBuffer = VertexBufferId;
				SegmentHeader.VertexType = uint32(Segment.VertexBufferElementType);
				SegmentHeader.VertexBufferOffset = Segment.VertexBufferOffset;
				SegmentHeader.VertexBufferStride = Segment.VertexBufferStride;
				SegmentHeader.MaxVertices = Segment.MaxVertices;
				SegmentHeader.FirstPrimitive = Segment.FirstPrimitive;
				SegmentHeader.NumPrimitives = Segment.NumPrimitives;
				SegmentHeader.bForceOpaque = Segment.bForceOpaque;
				SegmentHeader.bAllowDuplicateAnyHitShaderInvocation = Segment.bAllowDuplicateAnyHitShaderInvocation;
				SegmentHeader.bEnabled = Segment.bEnabled;

				Serialize(&SegmentHeader, sizeof(SegmentHeader), EAlignmentMode::CheckBeginning);
			}
		}
	}

	// Save GPU buffers

	{
		const FBufferHeader NullHeader;
		SceneHeader.Offsets.Buffers = Serialize(&NullHeader, sizeof(NullHeader), EAlignmentMode::AlignBeginning);
		checkf(SceneHeader.Offsets.Buffers % AlignmentRequirement == 0, TEXT("Serialized scene data must be vector-aligned"));

		SceneHeader.NumBuffers = BufferMap.Num();

		for (const TPair<FRHIBuffer*, uint32>& It : BufferMap)
		{
			FRHIBuffer* Buffer = It.Key;
			if (Buffer == nullptr)
			{
				continue;
			}

			FBufferHeader BufferHeader;

			uint32 NumPaddingBytes = 0;
			uint32 BufferSizeInBytes = 0;

			// Adjust the buffer size to ensure that all buffer base addresses are vector-aligned

			BufferSizeInBytes = Buffer->GetSize();

			uint32 SizeRemainder = BufferSizeInBytes % AlignmentRequirement;
			NumPaddingBytes = SizeRemainder ? AlignmentRequirement - SizeRemainder : 0;

			BufferHeader.StrideInBytes = Buffer->GetStride();
			BufferHeader.SizeInBytes = BufferSizeInBytes + NumPaddingBytes; // padded size to ensure alignment

			Serialize(&BufferHeader, sizeof(BufferHeader), EAlignmentMode::CheckBeginning);

			FStagingBufferRHIRef StatingBuffer = RHICreateStagingBuffer();

			CommandContext.RHICopyToStagingBuffer(Buffer, StatingBuffer, 0, BufferSizeInBytes);
			CommandContext.FlushCommands(ED3D12FlushFlags::WaitForCompletion);

			void* BufferData = StatingBuffer->Lock(0, BufferSizeInBytes);
			check(BufferData);

			Serialize(BufferData, BufferSizeInBytes, EAlignmentMode::CheckBeginning);

			if (NumPaddingBytes)
			{
				OutputBuffer.AddZeroed(NumPaddingBytes);
			}

			StatingBuffer->Unlock();
		}
	}

	// Save strings as length + ANSI characters (not null-terminated)

	{
		SceneHeader.NumStrings = StringMap.Num();

		SceneHeader.Offsets.Strings = OutputBuffer.Num();

		for (const TPair<FString, uint32>& It : StringMap)
		{
			auto StringAnsi = StringCast<ANSICHAR>(*It.Key);
			
			uint32 Len = StringAnsi.Length();
			Serialize(&Len, sizeof(Len), EAlignmentMode::Unaligned);

			if (Len)
			{
				ANSICHAR* Chars = const_cast<ANSICHAR*>(StringAnsi.Get()); // Serialize needs non-const
				Serialize(Chars, Len, EAlignmentMode::Unaligned);
			}
		}
	}

	// Save the header at the beginning of the container

	FMemory::Memcpy(OutputBuffer.GetData(), &SceneHeader, sizeof(SceneHeader));

	// Write output to disk

	FArchive* OutputFile = IFileManager::Get().CreateDebugFileWriter(*OutputFilename);
	if (OutputFile)
	{
		OutputFile->Serialize(OutputBuffer.GetData(), OutputBuffer.Num());
		delete OutputFile;
	}
}

void D3D12RayTracingSceneDebugUpdate(const FD3D12RayTracingScene& Scene, FD3D12Buffer* InstanceBuffer, uint32 InstanceBufferOffset, FD3D12CommandContext& CommandContext)
{
	if (GRayTracingSerializeSceneNextFrame)
	{
		// #dxr_todo: Move this to RHICmdList.SerializeAccelerationStructure(...)

		DebugSerializeScene(Scene, InstanceBuffer, InstanceBufferOffset, CommandContext);
		GRayTracingSerializeSceneNextFrame = false;
	}
}

#endif // D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING
