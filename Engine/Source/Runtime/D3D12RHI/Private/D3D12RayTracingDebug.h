// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIPrivate.h"

#define D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING (!UE_BUILD_SHIPPING && D3D12_RHI_RAYTRACING)

#if D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING

void D3D12RayTracingSceneDebugUpdate(const FD3D12RayTracingScene& Scene, FD3D12Buffer* InstanceBuffer, uint32 InstanceBufferOffset, FD3D12CommandContext& CommandContext);

#endif // D3D12_RHI_SUPPORT_RAYTRACING_SCENE_DEBUGGING
