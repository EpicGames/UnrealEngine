// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RenderTarget.cpp: D3D render target implementation.
	=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "RHIStaticStates.h"
#include "ResolveShader.h"
#include "PipelineStateCache.h"
#include "Math/PackedVector.h"
#include "RHISurfaceDataConversion.h"
#include "CommonRenderResources.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

static uint32 D3D12RT_ComputeBytesPerPixel(DXGI_FORMAT Format)
{
	uint32 BytesPerPixel = 0;

	switch (Format)
	{
	case DXGI_FORMAT_R8G8_TYPELESS:
	case DXGI_FORMAT_R8G8_UNORM:
	case DXGI_FORMAT_R8G8_UINT:
	case DXGI_FORMAT_R8G8_SNORM:
	case DXGI_FORMAT_R8G8_SINT:
	case DXGI_FORMAT_R16_TYPELESS:
	case DXGI_FORMAT_R16_FLOAT:
	case DXGI_FORMAT_D16_UNORM:
	case DXGI_FORMAT_R16_UNORM:
	case DXGI_FORMAT_R16_UINT:
	case DXGI_FORMAT_R16_SNORM:
	case DXGI_FORMAT_R16_SINT:
	case DXGI_FORMAT_B5G6R5_UNORM:
	case DXGI_FORMAT_B5G5R5A1_UNORM:
		BytesPerPixel = 2;
		break;
	case DXGI_FORMAT_B8G8R8A8_TYPELESS:
	case DXGI_FORMAT_B8G8R8A8_UNORM:
	case DXGI_FORMAT_R8G8B8A8_TYPELESS:
	case DXGI_FORMAT_R8G8B8A8_UNORM:
	case DXGI_FORMAT_R24G8_TYPELESS:
	case DXGI_FORMAT_R10G10B10A2_UNORM:
	case DXGI_FORMAT_R11G11B10_FLOAT:
	case DXGI_FORMAT_R16G16_UNORM:
	case DXGI_FORMAT_R32_UINT:
	case DXGI_FORMAT_R32_TYPELESS:
	case DXGI_FORMAT_R32_FLOAT:
	case DXGI_FORMAT_B8G8R8A8_UNORM_SRGB:
	case DXGI_FORMAT_B8G8R8X8_TYPELESS:
	case DXGI_FORMAT_B8G8R8X8_UNORM_SRGB:
	case DXGI_FORMAT_B8G8R8X8_UNORM:
	case DXGI_FORMAT_R10G10B10A2_TYPELESS:
	case DXGI_FORMAT_R10G10B10A2_UINT:
	case DXGI_FORMAT_R8G8B8A8_UNORM_SRGB:
	case DXGI_FORMAT_R8G8B8A8_UINT:
	case DXGI_FORMAT_R8G8B8A8_SNORM:
	case DXGI_FORMAT_R8G8B8A8_SINT:
	case DXGI_FORMAT_R16G16_TYPELESS:
	case DXGI_FORMAT_R16G16_FLOAT:
	case DXGI_FORMAT_R16G16_UINT:
	case DXGI_FORMAT_R16G16_SNORM:
	case DXGI_FORMAT_R16G16_SINT:
	case DXGI_FORMAT_D32_FLOAT:
	case DXGI_FORMAT_R32_SINT:
	case DXGI_FORMAT_R9G9B9E5_SHAREDEXP:
		BytesPerPixel = 4;
		break;
	case DXGI_FORMAT_R16G16B16A16_FLOAT:
	case DXGI_FORMAT_R16G16B16A16_UNORM:
	case DXGI_FORMAT_R16G16B16A16_TYPELESS:
	case DXGI_FORMAT_R16G16B16A16_UINT:
	case DXGI_FORMAT_R16G16B16A16_SNORM:
	case DXGI_FORMAT_R16G16B16A16_SINT:
	case DXGI_FORMAT_R32G32_TYPELESS:
	case DXGI_FORMAT_R32G32_FLOAT:
	case DXGI_FORMAT_R32G32_UINT:
	case DXGI_FORMAT_R32G32_SINT:
		BytesPerPixel = 8;
		break;
	case DXGI_FORMAT_R32G8X24_TYPELESS:
	case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
	case DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS:
		BytesPerPixel = 8;
		break;
	case DXGI_FORMAT_R8_TYPELESS:
	case DXGI_FORMAT_R8_UNORM:
	case DXGI_FORMAT_R8_UINT:
	case DXGI_FORMAT_R8_SNORM:
	case DXGI_FORMAT_R8_SINT:
	case DXGI_FORMAT_A8_UNORM:
	case DXGI_FORMAT_R1_UNORM:
		BytesPerPixel = 1;
		break;
	case DXGI_FORMAT_R32G32B32A32_UINT:
	case DXGI_FORMAT_R32G32B32A32_TYPELESS:
	case DXGI_FORMAT_R32G32B32A32_FLOAT:
	case DXGI_FORMAT_R32G32B32A32_SINT:
		BytesPerPixel = 16;
		break;
	default:
		// format not supported yet
		check(0);
		break;
	}

	// @@ DXGIUtilities supercedes; remove this function ??
	check(BytesPerPixel == UE::DXGIUtilities::GetFormatSizeInBytes(Format) );

	return BytesPerPixel;
}

static inline DXGI_FORMAT ConvertTypelessToUnorm(DXGI_FORMAT Format)
{
	// prefer DXGIUtilities::FindSharedResourceFormat ?
	//	or something? lots of these mappers in DXGIUtilities already

	// required to prevent 
	// D3D11: ERROR: ID3D11DeviceContext::ResolveSubresource: The Format (0x1b, R8G8B8A8_TYPELESS) is never able to resolve multisampled resources. [ RESOURCE_MANIPULATION ERROR #294: DEVICE_RESOLVESUBRESOURCE_FORMAT_INVALID ]
	// D3D11: **BREAK** enabled for the previous D3D11 message, which was: [ RESOURCE_MANIPULATION ERROR #294: DEVICE_RESOLVESUBRESOURCE_FORMAT_INVALID ]
	switch (Format)
	{
	case DXGI_FORMAT_R8G8B8A8_TYPELESS:
		return DXGI_FORMAT_R8G8B8A8_UNORM;

	case DXGI_FORMAT_B8G8R8A8_TYPELESS:
		return DXGI_FORMAT_B8G8R8A8_UNORM;

	default:
		return Format;
	}
}

static FResolveRect GetDefaultRect(const FResolveRect& Rect, uint32 DefaultWidth, uint32 DefaultHeight)
{
	if (Rect.X1 >= 0 && Rect.X2 >= 0 && Rect.Y1 >= 0 && Rect.Y2 >= 0)
	{
		return Rect;
	}
	else
	{
		return FResolveRect(0, 0, DefaultWidth, DefaultHeight);
	}
}

template<typename TPixelShader>
void FD3D12CommandContext::ResolveTextureUsingShader(
	FD3D12Texture* SourceTexture,
	FD3D12Texture* DestTexture,
	FD3D12RenderTargetView* DestTextureRTV,
	FD3D12DepthStencilView* DestTextureDSV,
	const D3D12_RESOURCE_DESC& ResolveTargetDesc,
	const FResolveRect& SourceRect,
	const FResolveRect& DestRect,
	typename TPixelShader::FParameter PixelShaderParameter
	)
{
	// Save the current viewports so they can be restored
	D3D12_VIEWPORT SavedViewports[D3D12_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE];
	uint32 NumSavedViewports = StateCache.GetNumViewports();
	StateCache.GetViewports(&NumSavedViewports, SavedViewports);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	// No alpha blending, no depth tests or writes, no stencil tests or writes, no backface culling.
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

	// Make sure the destination is not bound as a shader resource.
	ClearShaderResources(DestTexture, EShaderParameterTypeMask::SRVMask | EShaderParameterTypeMask::UAVMask);

	// Determine if the entire destination surface is being resolved to.
	// If the entire surface is being resolved to, then it means we can clear it and signal the driver that it can discard
	// the surface's previous contents, which breaks dependencies between frames when using alternate-frame SLI.
	const bool bClearDestTexture =
		DestRect.X1 == 0
		&& DestRect.Y1 == 0
		&& (uint64)DestRect.X2 == ResolveTargetDesc.Width
		&&	DestRect.Y2 == ResolveTargetDesc.Height;

	if (ResolveTargetDesc.Flags & D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL)
	{
		// Clear the destination texture.
		if (bClearDestTexture)
		{
			FlushResourceBarriers();

			GraphicsCommandList()->ClearDepthStencilView(DestTextureDSV->GetOfflineCpuHandle(), D3D12_CLEAR_FLAG_DEPTH | D3D12_CLEAR_FLAG_STENCIL, 0, 0, 0, nullptr);
			UpdateResidency(DestTextureDSV->GetResource());
		}

		// Write to the dest texture as a depth-stencil target.
		FD3D12RenderTargetView* NullRTV = nullptr;
		StateCache.SetRenderTargets(1, &NullRTV, DestTextureDSV);

		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();

		if (DestTexture)
		{
			GraphicsPSOInit.DepthStencilTargetFormat = DestTexture->GetFormat();
			GraphicsPSOInit.DepthStencilTargetFlag = DestTexture->GetFlags();
			GraphicsPSOInit.NumSamples = DestTexture->GetNumSamples();
		}
	}
	else
	{
		// Clear the destination texture.
		if (bClearDestTexture)
		{
			FlushResourceBarriers();

			FLinearColor ClearColor(0, 0, 0, 0);
			GraphicsCommandList()->ClearRenderTargetView(DestTextureRTV->GetOfflineCpuHandle(), (float*)&ClearColor, 0, nullptr);
			UpdateResidency(DestTextureRTV->GetResource());
		}

		// Write to the dest surface as a render target.
		StateCache.SetRenderTargets(1, &DestTextureRTV, nullptr);

		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		if (DestTexture)
		{
			GraphicsPSOInit.RenderTargetFormats[0] = DestTexture->GetFormat();
			GraphicsPSOInit.RenderTargetFlags[0] = DestTexture->GetFlags();
			GraphicsPSOInit.NumSamples = DestTexture->GetNumSamples();
		}
	}

	{
		TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
		// Lambda to guard access to 'this'
		([&ResolveTargetDesc, &GraphicsPSOInit, &SourceRect, &DestRect, &PixelShaderParameter, &SourceTexture](auto& RHICmdList)
		{
			SCOPED_DRAW_EVENT(RHICmdList, ResolveTextureUsingShader);

			RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (uint32)ResolveTargetDesc.Width, ResolveTargetDesc.Height, 1.0f);

			// Set the vertex and pixel shader
			auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FResolveVS> ResolveVertexShader(ShaderMap);
			TShaderMapRef<TPixelShader> ResolvePixelShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = ResolveVertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ResolvePixelShader.GetPixelShader();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			RHICmdList.SetBlendFactor(FLinearColor::White);

			SetShaderParametersLegacyVS(RHICmdList, ResolveVertexShader, SourceRect, DestRect, ResolveTargetDesc.Width, ResolveTargetDesc.Height);
			SetShaderParametersLegacyPS(RHICmdList, ResolvePixelShader, PixelShaderParameter);

			const uint32 TextureIndex = ResolvePixelShader->UnresolvedSurface.GetBaseIndex();

			RHICmdList.RunOnContext([TextureIndex, SourceTexture](FD3D12CommandContext& Context)
			{
				// Set the source texture.
				Context.StateCache.SetShaderResourceView(SF_Pixel, SourceTexture->GetShaderResourceView(), TextureIndex);
			});

			RHICmdList.DrawPrimitive(0, 2, 1);
		})(RHICmdList);
	}

	ClearShaderResources(SourceTexture, EShaderParameterTypeMask::SRVMask | EShaderParameterTypeMask::UAVMask);

	// Reset saved viewport
	{
		StateCache.SetViewports(NumSavedViewports, SavedViewports);
	}
}

static DXGI_FORMAT GetPlaneFormat(DXGI_FORMAT InFormat, uint32 InPlaneSlice)
{
	// in D3D12 mixed formats are in split planes not interleaved (change from D3D11)
	//	?? but these look wrong ??
	if (InFormat == DXGI_FORMAT_R32G8X24_TYPELESS || InFormat == DXGI_FORMAT_R24G8_TYPELESS)
	{
		if (InPlaneSlice == 0)
		{
			return DXGI_FORMAT_R32_TYPELESS;
		}
		if (InPlaneSlice == 1)
		{
			return DXGI_FORMAT_R8_TYPELESS;
		}
	}

	return InFormat;
}

void FD3D12CommandContext::ResolveTexture(UE::RHICore::FResolveTextureInfo Info)
{
	uint32 GPUIndex = GetGPUIndex();

	FD3D12Texture* SourceTexture         = GetD3D12TextureFromRHITexture(Info.SourceTexture, GPUIndex);
	FD3D12Resource* SourceResource       = SourceTexture->GetResource();
	const FRHITextureDesc& SourceDesc    = SourceTexture->GetDesc();

	FD3D12Texture* DestTexture           = GetD3D12TextureFromRHITexture(Info.DestTexture, GPUIndex);
	FD3D12Resource* DestResource         = DestTexture->GetResource();
	const FRHITextureDesc& DestDesc      = DestTexture->GetDesc();

	if (SourceDesc.Format == PF_DepthStencil)
	{
		ResolveTextureUsingShader<FResolveDepthPS>(
			SourceTexture,
			DestTexture,
			DestTexture->GetRenderTargetView(0, -1),
			DestTexture->GetDepthStencilView(FExclusiveDepthStencil::DepthWrite_StencilWrite),
			DestResource->GetDesc(),
			GetDefaultRect(Info.ResolveRect, SourceDesc.Extent.X, SourceDesc.Extent.Y),
			GetDefaultRect(Info.ResolveRect, DestDesc.Extent.X, DestDesc.Extent.Y),
			FDummyResolveParameter()
		);
	}
	else
	{
		const DXGI_FORMAT DestFormatTypeless = ConvertTypelessToUnorm((DXGI_FORMAT)GPixelFormats[DestDesc.Format].PlatformFormat);

		int32 ArraySliceBegin = Info.ArraySlice;
		int32 ArraySliceEnd   = Info.ArraySlice + 1;

		if (Info.ArraySlice < 0)
		{
			ArraySliceBegin = 0;
			ArraySliceEnd   = SourceDesc.ArraySize;
		}

		for (int32 ArraySlice = ArraySliceBegin; ArraySlice < ArraySliceEnd; ArraySlice++)
		{
			int32 DestSubresource   = CalcSubresource(Info.MipLevel, ArraySlice, DestDesc.NumMips);
			int32 SourceSubresource = CalcSubresource(Info.MipLevel, ArraySlice, SourceDesc.NumMips);

			// TODO: Is this taken care of now that we're paying attention to the RHI states?

			// Source resource is in RTV, transition to D3D12_RESOURCE_STATE_RESOLVE_SOURCE and go back to D3D12_RESOURCE_STATE_RENDER_TARGET
			// TODO: high level setting to D3D12_RESOURCE_STATE_RESOLVE_SOURCE
			FScopedResourceBarrier ConditionalScopeResourceBarrierSrc(
				*this,
				SourceTexture->GetResource(),
				ED3D12Access::RTV,
				ED3D12Access::ResolveSrc,
				SourceSubresource);

			// Dest resource is already in D3D12_RESOURCE_STATE_RESOLVE_DEST, leave the FScopedResourceBarrier in case of using single state resources
			FScopedResourceBarrier ConditionalScopeResourceBarrierDst(
				*this,
				DestTexture->GetResource(),
				ED3D12Access::ResolveDst,
				DestSubresource);

			FlushResourceBarriers();
			GraphicsCommandList()->ResolveSubresource(DestResource->GetResource(), DestSubresource, SourceTexture->GetResource()->GetResource(), SourceSubresource, DestFormatTypeless);
		}
	}

	UpdateResidency(SourceTexture->GetResource());
	UpdateResidency(DestTexture->GetResource());

	ConditionalSplitCommandList();

	DEBUG_EXECUTE_COMMAND_LIST(this);
}

TRefCountPtr<FD3D12Resource> FD3D12DynamicRHI::GetStagingTexture(FRHITexture* TextureRHI, FIntRect InRect, FIntRect& StagingRectOUT, FReadSurfaceDataFlags InFlags, D3D12_PLACED_SUBRESOURCE_FOOTPRINT& ReadbackHeapDesc, uint32 GPUIndex)
{
	FD3D12Device* Device = GetRHIDevice(GPUIndex);
	FD3D12Adapter* Adapter = Device->GetParentAdapter();
	const FRHIGPUMask Node = Device->GetGPUMask();

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, GPUIndex);
	D3D12_RESOURCE_DESC const& SourceDesc = Texture->GetResource()->GetDesc();

	bool bRequiresTempStagingTexture = Texture->GetResource()->GetHeapType() != D3D12_HEAP_TYPE_READBACK;
	if (bRequiresTempStagingTexture == false)
	{
		// Returning the same texture is considerably faster than creating and copying to
		// a new staging texture as we do not have to wait for the GPU pipeline to catch up
		// to the staging texture preparation work.

		// Texture2Ds on the readback heap will have been flattened to 1D, so we need to retrieve pitch
		// information from the original 2D version to correctly use sub-rects.
		Texture->GetReadBackHeapDesc(ReadbackHeapDesc, InFlags.GetMip());
		StagingRectOUT = InRect;

		// We need to execute the command list so we can read the data from readback heap
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		return (Texture->GetResource());
	}
	else
	{
		// a temporary staging texture is needed.
		int32 SizeX = InRect.Width();
		int32 SizeY = InRect.Height();
		// Read back the surface data in the defined rect
		D3D12_BOX Rect;
		Rect.left = InRect.Min.X;
		Rect.top = InRect.Min.Y;
		Rect.right = InRect.Max.X;
		Rect.bottom = InRect.Max.Y;
		Rect.back = 1;
		Rect.front = 0;

		// create a temp 2d texture to copy render target to
		TRefCountPtr<FD3D12Resource> TempTexture2D;

		const uint32 PlaneSlice = 0;
		const DXGI_FORMAT DestFormat = GetPlaneFormat(SourceDesc.Format, PlaneSlice);

		const uint32 BlockBytes = D3D12RT_ComputeBytesPerPixel(DestFormat);
		const uint32 XBytesAligned = Align((uint32)SourceDesc.Width * BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
		const uint32 MipBytesAligned = XBytesAligned * SourceDesc.Height;
		VERIFYD3D12RESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, Node, Node, MipBytesAligned, TempTexture2D.GetInitReference(), nullptr));

		// Staging rectangle is now the whole surface.
		StagingRectOUT.Min = FIntPoint::ZeroValue;
		StagingRectOUT.Max = FIntPoint(SizeX, SizeY);

		// Copy the data to a staging resource.
		uint32 Subresource = 0;
		if (Texture->GetDesc().IsTextureCube())
		{
			uint32 D3DFace = GetD3D12CubeFace(InFlags.GetCubeFace());
			Subresource = CalcSubresource(InFlags.GetMip(), InFlags.GetArrayIndex() * 6 + D3DFace, TextureRHI->GetNumMips());
		}
		else
		{
			const bool bIsTextureArray = Texture->GetDesc().IsTextureArray();
			Subresource = CalcSubresource(InFlags.GetMip(), bIsTextureArray ? InFlags.GetArrayIndex() : 0, TextureRHI->GetNumMips());
		}

		const bool bSubrect = (Rect.left != 0 || Rect.top != 0 || Rect.right != SourceDesc.Width || Rect.bottom != SourceDesc.Height);

		D3D12_PLACED_SUBRESOURCE_FOOTPRINT DestFootprint{};
		DestFootprint.Footprint.Depth = 1;
		DestFootprint.Footprint.Height = SourceDesc.Height;
		DestFootprint.Footprint.Width = SourceDesc.Width;
		DestFootprint.Footprint.Format = DestFormat;
		DestFootprint.Footprint.RowPitch = XBytesAligned;
		check((DestFootprint.Footprint.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT) == 0);	// Make sure we align correctly.

		CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(TempTexture2D->GetResource(), DestFootprint);
		CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(Texture->GetResource()->GetResource(), Subresource);

		// Remember the width, height, pitch, etc...
		ReadbackHeapDesc = DestFootprint;

		// TODO: transition only subresource not the whole resource
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState));

		RHICmdList.EnqueueLambda(
		[Texture, GPUIndex, Rect, bSubrect, DestCopyLocation = MoveTemp(DestCopyLocation), SourceCopyLocation = MoveTemp(SourceCopyLocation), TempTexture2D = CopyTemp(TempTexture2D)](FRHICommandList& ExecutingCmdList)
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);

			//FScopedResourceBarrier ScopeResourceBarrierSource(Context, Texture->GetResource(), &Texture->ResourceLocation, D3D12_RESOURCE_STATE_COPY_SOURCE, SourceCopyLocation.SubresourceIndex);
			Context.FlushResourceBarriers();
			// Upload heap doesn't need to transition

			const D3D12_BOX* RectPtr = nullptr; // API prefers NULL for entire texture.
			if (bSubrect)
			{
				// ..Sub rectangle required, use the D3D12_BOX.
				RectPtr = &Rect;
			}

			Context.CopyTextureRegionChecked(
				&DestCopyLocation,
				0, 0, 0,
				Texture->GetFormat(),
				&SourceCopyLocation,
				RectPtr,
				Texture->GetFormat(),
				TempTexture2D->GetName(),		// DestName
				Texture->GetName()				// SourceName
			);

			Context.UpdateResidency(Texture->GetResource());
			Context.UpdateResidency(TempTexture2D);
		});

		RHICmdList.TransitionInternal(FRHITransitionInfo(TextureRHI, ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState));


		// We need to execute the command list so we can read the data from readback heap
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		return TempTexture2D;
	}
}

void FD3D12DynamicRHI::ReadSurfaceDataNoMSAARaw(FRHITexture* TextureRHI, FIntRect InRect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags)
{
	const uint32 GPUIndex = InFlags.GetGPUIndex();

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();

	// Check the format of the surface
	FIntRect StagingRect;
	D3D12_PLACED_SUBRESOURCE_FOOTPRINT readBackHeapDesc;
	TRefCountPtr<FD3D12Resource> TempTexture2D = GetStagingTexture(TextureRHI, InRect, StagingRect, InFlags, readBackHeapDesc, GPUIndex);

	uint32 BytesPerPixel = GPixelFormats[TextureRHI->GetFormat()].BlockBytes;

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * BytesPerPixel);

	uint32 BytesPerLine = BytesPerPixel * InRect.Width();
	const uint32 XBytesAligned = Align((uint32)readBackHeapDesc.Footprint.Width * BytesPerPixel, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	uint64 SrcStart = readBackHeapDesc.Offset + StagingRect.Min.X * BytesPerPixel + StagingRect.Min.Y * XBytesAligned;

	// Lock the staging resource.
	void* pData;
	D3D12_RANGE ReadRange = { SrcStart, SrcStart + XBytesAligned * (SizeY - 1) + BytesPerLine };
	VERIFYD3D12RESULT(TempTexture2D->GetResource()->Map(0, &ReadRange, &pData));

	uint8* DestPtr = OutData.GetData();
	uint8* SrcPtr = (uint8*)pData + SrcStart;
	for (uint32 Y = 0; Y < SizeY; Y++)
	{
		memcpy(DestPtr, SrcPtr, BytesPerLine);
		DestPtr += BytesPerLine;
		SrcPtr += XBytesAligned;
	}

	TempTexture2D->GetResource()->Unmap(0, nullptr);

#if UE_MEMORY_TRACE_ENABLED
	// We only need to free memory for a texture that was created specifically for this operation -
	// not for a texture that was reused for readback.
	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, GPUIndex);
	if (TempTexture2D != Texture->GetResource())
	{
		check(TempTexture2D.GetRefCount() == 1);
		// Free the temporary texture after read back finishes.
		// This matches the MemoryTrace_Alloc call in Adapter->CreateBuffer() via the GetStagingTexture call above.
		MemoryTrace_Free(TempTexture2D->GetGPUVirtualAddress(), EMemoryTraceRootHeap::VideoMemory);
	}
#endif
}

void FD3D12DynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	TArray<uint8> OutDataRaw;

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, InFlags.GetGPUIndex());

	// Check the format of the surface
	D3D12_RESOURCE_DESC const& TextureDesc = Texture->GetResource()->GetDesc();

	check(TextureDesc.SampleDesc.Count >= 1);

	if (TextureDesc.SampleDesc.Count == 1)
	{
		ReadSurfaceDataNoMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}
	else
	{
		ReadSurfaceDataMSAARaw(TextureRHI, InRect, OutDataRaw, InFlags);
	}

	const uint32 SizeX = InRect.Width() * TextureDesc.SampleDesc.Count;
	const uint32 SizeY = InRect.Height();

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);

	uint32 BytesPerPixel = D3D12RT_ComputeBytesPerPixel(TextureDesc.Format);
	uint32 SrcPitch = SizeX * BytesPerPixel;

	// switching on the EPixelFormat is risky if the mapping is not what you expect
	//	verify against TextureDesc.Format

	EPixelFormat Format = TextureRHI->GetFormat();
	check( GPixelFormats[Format].PlatformFormat == TextureDesc.Format );
	check( GPixelFormats[Format].BlockBytes == D3D12RT_ComputeBytesPerPixel(TextureDesc.Format) );

	if ( ! ConvertRAWSurfaceDataToFLinearColor(Format, SizeX, SizeY, OutDataRaw.GetData(), SrcPitch, OutData.GetData(), InFlags) )
	{
		checkf(0, TEXT("Unsupported surface format!"));
		OutData.Empty();
	}
}

void FD3D12DynamicRHI::RHIReadSurfaceData(FRHITexture* InRHITexture, FIntRect InRect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	if ( InRect.Width() == 0 || InRect.Height() == 0 )
	{
		OutData.Empty();
		return;
	}

	if (!ensure(InRHITexture))
	{
		OutData.SetNumUninitialized(InRect.Width() * InRect.Height());
		FMemory::Memzero(OutData.GetData(), sizeof(FColor) * OutData.Num());
		return;
	}

	const uint32 GPUIndex = InFlags.GetGPUIndex();
	TArray<uint8> OutDataRaw;

	// Retrieve the base texture
	FD3D12Texture* DestTexture2D = FD3D12CommandContext::RetrieveTexture(InRHITexture, GPUIndex);

	// Check the format of the surface
	FD3D12ResourceDesc const& TextureDesc = DestTexture2D->GetResource()->GetDesc();

	check(TextureDesc.SampleDesc.Count >= 1);

	if (TextureDesc.SampleDesc.Count == 1)
	{
		ReadSurfaceDataNoMSAARaw(DestTexture2D, InRect, OutDataRaw, InFlags);
	}
	else
	{
		ReadSurfaceDataMSAARaw(DestTexture2D, InRect, OutDataRaw, InFlags);
	}

	const uint32 SizeX = InRect.Width() * TextureDesc.SampleDesc.Count;
	const uint32 SizeY = InRect.Height();

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);
	
	// dest format :
	EPixelFormat PixelFormat = DestTexture2D->GetFormat();

	check( PixelFormat != PF_Unknown );
	check( PixelFormat == TextureDesc.PixelFormat || TextureDesc.PixelFormat == PF_Unknown );

	const FPixelFormatInfo & FormatInfo = GPixelFormats[PixelFormat];

	uint32 BytesPerPixel = FormatInfo.BlockBytes;
	uint32 SrcPitch = SizeX * BytesPerPixel;
	
	// switching on the EPixelFormat is risky if the mapping is not what you expect
	//	verify against TextureDesc.Format
	
	DXGI_FORMAT DXGIFormat = TextureDesc.Format;

	if ( DXGIFormat == DXGI_FORMAT_UNKNOWN )
	{
		// when called on actual textures, DXGIFormat is valid
		// but this is also called on untyped buffers, in which case we only know the type from dest PixelFormat
		DXGIFormat = (DXGI_FORMAT) FormatInfo.PlatformFormat;
	}
	else
	{
		// source and dest format must match, except for _TYPELESS vs _UNORM
		check( ConvertTypelessToUnorm((DXGI_FORMAT)FormatInfo.PlatformFormat) == ConvertTypelessToUnorm(DXGIFormat) );
		check( FormatInfo.BlockBytes == D3D12RT_ComputeBytesPerPixel(DXGIFormat) );
	}

	// ConvertDXGIToFColor switches on the hardware format, not the EPixelFormat :

	if ( ! ConvertDXGIToFColor(DXGIFormat, SizeX, SizeY, OutDataRaw.GetData(), SrcPitch, OutData.GetData(), InFlags) )
	{
		checkf(0, TEXT("Unsupported surface format!"));
		OutData.Empty();
	}
}

void FD3D12DynamicRHI::ReadSurfaceDataMSAARaw(FRHITexture* TextureRHI, FIntRect InRect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags)
{	
	const uint32 GPUIndex = InFlags.GetGPUIndex();
	FD3D12Device* Device = GetRHIDevice(GPUIndex);
	FD3D12Adapter* Adapter = Device->GetParentAdapter();
	const FRHIGPUMask NodeMask = Device->GetGPUMask();

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, GPUIndex);

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();

	// Check the format of the surface
	D3D12_RESOURCE_DESC const& TextureDesc = Texture->GetResource()->GetDesc();

	uint32 BytesPerPixel = D3D12RT_ComputeBytesPerPixel(TextureDesc.Format);

	const uint32 NumSamples = TextureDesc.SampleDesc.Count;

	// Read back the surface data from the define rect
	D3D12_BOX	Rect;
	Rect.left = InRect.Min.X;
	Rect.top = InRect.Min.Y;
	Rect.right = InRect.Max.X;
	Rect.bottom = InRect.Max.Y;
	Rect.back = 1;
	Rect.front = 0;

	// Create a non-MSAA render target to resolve individual samples of the source surface to.
	D3D12_RESOURCE_DESC NonMSAADesc;
	ZeroMemory(&NonMSAADesc, sizeof(NonMSAADesc));
	NonMSAADesc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	NonMSAADesc.Width = SizeX;
	NonMSAADesc.Height = SizeY;
	NonMSAADesc.MipLevels = 1;
	NonMSAADesc.DepthOrArraySize = 1;
	NonMSAADesc.Format = TextureDesc.Format;
	NonMSAADesc.SampleDesc.Count = 1;
	NonMSAADesc.SampleDesc.Quality = 0;
	NonMSAADesc.Flags = D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET;
	TRefCountPtr<FD3D12Resource> NonMSAATexture2D;

	const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT, NodeMask.GetNative(), NodeMask.GetNative());
	VERIFYD3D12RESULT(
		Adapter->CreateCommittedResource(
			NonMSAADesc,
			NodeMask,
			HeapProps,
			ED3D12Access::RTV,
			nullptr,
			NonMSAATexture2D.GetInitReference(),
			nullptr));

	FD3D12ResourceLocation ResourceLocation(Device);
	ResourceLocation.AsStandAlone(NonMSAATexture2D);

	D3D12_RENDER_TARGET_VIEW_DESC RTVDesc = {};

	// typeless is not supported, similar code might be needed for other typeless formats
	RTVDesc.Format = ConvertTypelessToUnorm(NonMSAADesc.Format);

	RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2D;
	RTVDesc.Texture2D.MipSlice = 0;

	FD3D12RenderTargetView NonMSAARTV(Device, nullptr);			// Always single GPU object, so FirstLinkedObject is nullptr
	NonMSAARTV.CreateView(&ResourceLocation, RTVDesc);

	// Create a CPU-accessible staging texture to copy the resolved sample data to.
	TRefCountPtr<FD3D12Resource> StagingTexture2D;

	const uint32 PlaneSlice = 0;
	const DXGI_FORMAT DestFormat = GetPlaneFormat(TextureDesc.Format, PlaneSlice);

	const uint32 BlockBytes = D3D12RT_ComputeBytesPerPixel(DestFormat);
	const uint32 XBytesAligned = Align(SizeX * BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	const uint32 MipBytesAligned = XBytesAligned * SizeY;
	VERIFYD3D12RESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, NodeMask, NodeMask, MipBytesAligned, StagingTexture2D.GetInitReference(), nullptr));

	// Determine the subresource index for cubemaps.
	uint32 Subresource = 0;
	if (Texture->GetDesc().IsTextureCube())
	{
		uint32 D3DFace = GetD3D12CubeFace(InFlags.GetCubeFace());
		Subresource = CalcSubresource(InFlags.GetMip(), InFlags.GetArrayIndex() * 6 + D3DFace, TextureRHI->GetNumMips());
	}
	else
	{
		const bool bIsTextureArray = Texture->GetDesc().IsTextureArray();
		Subresource = CalcSubresource(InFlags.GetMip(), bIsTextureArray ? InFlags.GetArrayIndex() : 0, TextureRHI->GetNumMips());
	}

	// Setup the descriptions for the copy to the readback heap.

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT DestFootprint{};
	DestFootprint.Footprint.Depth = 1;
	DestFootprint.Footprint.Height = SizeY;
	DestFootprint.Footprint.Width = SizeX;
	DestFootprint.Footprint.Format = DestFormat;
	DestFootprint.Footprint.RowPitch = XBytesAligned;
	check((DestFootprint.Footprint.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT) == 0);	// Make sure we align correctly.

	CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(StagingTexture2D->GetResource(), DestFootprint);
	CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(NonMSAATexture2D->GetResource(), Subresource);

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * NumSamples * BytesPerPixel);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Can be optimized by doing all subsamples into a large enough rendertarget in one pass (multiple draw calls)
	for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		RHICmdList.EnqueueLambda(
		[Texture, GPUIndex, SampleIndex, InRect, Rect, NonMSAADesc, NonMSAATexture2D, NonMSAARTV, StagingTexture2D, DestCopyLocation = MoveTemp(DestCopyLocation), SourceCopyLocation = MoveTemp(SourceCopyLocation)](FRHICommandListBase& ExecutingCmdList) mutable
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);

			// Resolve the sample to the non-MSAA render target.
			Context.ResolveTextureUsingShader<FResolveSingleSamplePS>(
				ResourceCast(Texture->GetTexture2D()),
				NULL,
				&NonMSAARTV,
				NULL,
				NonMSAADesc,
				FResolveRect(InRect.Min.X, InRect.Min.Y, InRect.Max.X, InRect.Max.Y),
				FResolveRect(0, 0, InRect.Width(), InRect.Height()),
				SampleIndex
				);

			FScopedResourceBarrier ScopeResourceBarrierSource(
				Context,
				NonMSAATexture2D,
				ED3D12Access::CopySrc,
				SourceCopyLocation.SubresourceIndex);
			// Upload heap doesn't need to transition

			// Copy the resolved sample data to the staging texture.
			Context.CopyTextureRegionChecked(
				&DestCopyLocation,
				0, 0, 0,
				Texture->GetFormat(),
				&SourceCopyLocation,
				&Rect,
				Texture->GetFormat(),
				StagingTexture2D->GetName(),		// DestName
				Texture->GetName()					// SourceName
			);

			Context.UpdateResidency(StagingTexture2D);
			Context.UpdateResidency(NonMSAATexture2D);
		});

		// We need to execute the command list so we can read the data in the map below
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		// Lock the staging texture.
		void* pData;
		VERIFYD3D12RESULT(StagingTexture2D->GetResource()->Map(0, nullptr, &pData));

		// Read the data out of the buffer, could be optimized
		for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y++)
		{
			uint8* SrcPtr = (uint8*)pData + (Y - InRect.Min.Y) * XBytesAligned + InRect.Min.X * BytesPerPixel;
			uint8* DestPtr = &OutData[(Y - InRect.Min.Y) * SizeX * NumSamples * BytesPerPixel + SampleIndex * BytesPerPixel];

			for (int32 X = InRect.Min.X; X < InRect.Max.X; X++)
			{
				for (uint32 i = 0; i < BytesPerPixel; ++i)
				{
					*DestPtr++ = *SrcPtr++;
				}

				DestPtr += (NumSamples - 1) * BytesPerPixel;
			}
		}

		StagingTexture2D->GetResource()->Unmap(0, nullptr);
	}
}

void FD3D12DynamicRHI::RHIMapStagingSurface(FRHITexture* TextureRHI, FRHIGPUFence* FenceRHI, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex)
{	
	FD3D12Texture* DestTexture = ResourceCast(TextureRHI, GPUIndex);

	check(DestTexture);
	FD3D12Resource* Texture = DestTexture->GetResource();

	DXGI_FORMAT Format = (DXGI_FORMAT)GPixelFormats[DestTexture->GetFormat()].PlatformFormat;

	uint32 BytesPerPixel = D3D12RT_ComputeBytesPerPixel(Format);

	if (FenceRHI && !FenceRHI->Poll())
	{
		ResourceCast(FenceRHI)->Wait(FRHICommandListImmediate::Get(), FRHIGPUMask::FromIndex(GPUIndex));
	}

	void* pData;
	D3D12_RANGE ReadRange = { 0, Texture->GetDesc().Width };

	VERIFYD3D12RESULT_EX(
		Texture->GetResource()->Map(0, &ReadRange, &pData),
		GetAdapter().GetD3DDevice()
	);

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT ReadBackHeapDesc;
	DestTexture->GetReadBackHeapDesc(ReadBackHeapDesc, 0);
	OutData = pData;
	OutWidth = ReadBackHeapDesc.Footprint.RowPitch / BytesPerPixel;
	OutHeight = ReadBackHeapDesc.Footprint.Height;

	// MS: It seems like the second frame in some scenes comes into RHIMapStagingSurface BEFORE the copy to the staging texture, thus the readbackHeapDesc isn't set. This could be bug in UE.
	if (ReadBackHeapDesc.Footprint.Format != DXGI_FORMAT_UNKNOWN)
	{
		check(OutWidth != 0);
		check(OutHeight != 0);
	}

	check(OutData);
}

void FD3D12DynamicRHI::RHIUnmapStagingSurface(FRHITexture* TextureRHI, uint32 GPUIndex)
{	
	FD3D12Texture* DestTexture = ResourceCast(TextureRHI, GPUIndex);

	ID3D12Resource* Texture = DestTexture->GetResource()->GetResource();

	Texture->Unmap(0, nullptr);
}

void FD3D12DynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace, int32 ArrayIndex, int32 MipIndex)
{
	FReadSurfaceDataFlags ReadFlags(RCM_MinMax, CubeFace);
	ReadFlags.SetArrayIndex(ArrayIndex);
	ReadFlags.SetMip(MipIndex);
	RHIReadSurfaceFloatData(TextureRHI, InRect, OutData, ReadFlags);
}

void FD3D12DynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags)
{
	const uint32 GPUIndex = InFlags.GetGPUIndex();
	FD3D12Device* Device = GetRHIDevice(GPUIndex);
	FD3D12Adapter* Adapter = Device->GetParentAdapter();
	const FRHIGPUMask Node = Device->GetGPUMask();

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, GPUIndex);

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();

	// Check the format of the surface
	D3D12_RESOURCE_DESC const& TextureDesc = Texture->GetResource()->GetDesc();

	if ( ! ensure(TextureDesc.Format == GPixelFormats[PF_FloatRGBA].PlatformFormat) )
	{
		OutData.Empty();
		return;
	}

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY);

	// Read back the surface data from defined rect
	D3D12_BOX	Rect;
	Rect.left = InRect.Min.X;
	Rect.top = InRect.Min.Y;
	Rect.right = InRect.Max.X;
	Rect.bottom = InRect.Max.Y;
	Rect.back = 1;
	Rect.front = 0;

	// create a temp 2d texture to copy render target to
	TRefCountPtr<FD3D12Resource> TempTexture2D;

	const uint32 PlaneSlice = 0;
	const DXGI_FORMAT DestFormat = GetPlaneFormat(TextureDesc.Format, PlaneSlice);

	const uint32 BlockBytes = D3D12RT_ComputeBytesPerPixel(DestFormat);
	const uint32 XBytesAligned = Align((uint32)TextureDesc.Width * BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	const uint32 MipBytesAligned = XBytesAligned * TextureDesc.Height;
	VERIFYD3D12RESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, Node, Node, MipBytesAligned, TempTexture2D.GetInitReference(), nullptr));

	// Ensure we're dealing with a Texture2D, which the rest of this function already assumes
	bool bIsTextureCube = Texture->GetDesc().IsTextureCube();
	
	// Copy the data to a staging resource.
	uint32 Subresource = 0;
	if (bIsTextureCube)
	{
		uint32 D3DFace = GetD3D12CubeFace(InFlags.GetCubeFace());
		Subresource = CalcSubresource(InFlags.GetMip(), InFlags.GetArrayIndex() * 6 + D3DFace, TextureDesc.MipLevels);
	}
	else
	{
		const bool bIsTextureArray = Texture->GetDesc().IsTextureArray();
		Subresource = CalcSubresource(InFlags.GetMip(), bIsTextureArray ? InFlags.GetArrayIndex() : 0, TextureDesc.MipLevels);
	}

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT DestFootprint{};
	DestFootprint.Footprint.Depth = 1;
	DestFootprint.Footprint.Height = TextureDesc.Height;
	DestFootprint.Footprint.Width = TextureDesc.Width;
	DestFootprint.Footprint.Format = DestFormat;
	DestFootprint.Footprint.RowPitch = XBytesAligned;
	check((DestFootprint.Footprint.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT) == 0);	// Make sure we align correctly.

	CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(TempTexture2D->GetResource(), DestFootprint);
	CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(Texture->GetResource()->GetResource(), Subresource);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda(
	[Texture, GPUIndex, Rect, DestCopyLocation = MoveTemp(DestCopyLocation), SourceCopyLocation = MoveTemp(SourceCopyLocation), TempTexture2D = CopyTemp(TempTexture2D)](FRHICommandListBase& ExecutingCmdList)
	{
		FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);

		FScopedResourceBarrier ConditionalScopeResourceBarrier(
			Context,
			Texture->GetResource(),
			ED3D12Access::CopySrc,
			SourceCopyLocation.SubresourceIndex);
		// Don't need to transition upload heaps

		Context.FlushResourceBarriers();
		Context.CopyTextureRegionChecked(
			&DestCopyLocation,
			0, 0, 0,
			Texture->GetFormat(),
			&SourceCopyLocation,
			&Rect,
			Texture->GetFormat(),
			TempTexture2D->GetName(),		// DestName
			Texture->GetName()				// SourceName
		);

		Context.UpdateResidency(Texture->GetResource());
		Context.UpdateResidency(TempTexture2D);
	});

	// We need to execute the command list so we can read the data from the map below
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	// Lock the staging resource.
	void* pData;
	D3D12_RANGE Range = { 0, MipBytesAligned };
	VERIFYD3D12RESULT(TempTexture2D->GetResource()->Map(0, &Range, &pData));

	for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y++)
	{
		FFloat16Color* SrcPtr = (FFloat16Color*)((uint8*)pData + (Y - InRect.Min.Y) * XBytesAligned);
		int32 Index = (Y - InRect.Min.Y) * SizeX;
		check(Index + ((int32)SizeX - 1) < OutData.Num());
		FFloat16Color* DestColor = &OutData[Index];
		FFloat16* DestPtr = (FFloat16*)(DestColor);
		FMemory::Memcpy(DestPtr, SrcPtr, SizeX * sizeof(FFloat16) * 4);
	}

	TempTexture2D->GetResource()->Unmap(0, nullptr);

#if UE_MEMORY_TRACE_ENABLED
	// Free the temporary texture after read back finishes. This matches the MemoryTrace_Alloc call in Adapter->CreateBuffer() above.
	MemoryTrace_Free(TempTexture2D->GetGPUVirtualAddress(), EMemoryTraceRootHeap::VideoMemory);
#endif
}

void FD3D12DynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI, FIntRect InRect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData)
{
	RHIRead3DSurfaceFloatData(TextureRHI, InRect, ZMinMax, OutData, FReadSurfaceDataFlags(RCM_MinMax));
}

void FD3D12DynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI, FIntRect InRect, FIntPoint ZMinMax, TArray<FFloat16Color>& OutData, FReadSurfaceDataFlags InFlags)
{
	const uint32 GPUIndex = InFlags.GetGPUIndex();
	FD3D12Device* Device = GetRHIDevice(GPUIndex);
	FD3D12Adapter* Adapter = Device->GetParentAdapter();
	const FRHIGPUMask Node = Device->GetGPUMask();

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI, GPUIndex);

	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();
	const uint32 SizeZ = ZMinMax.Y - ZMinMax.X;

	// Check the format of the surface
	D3D12_RESOURCE_DESC const& TextureDesc = Texture->GetResource()->GetDesc();
	bool bIsRGBAFmt = TextureDesc.Format == GPixelFormats[PF_FloatRGBA].PlatformFormat;
	bool bIsR16FFmt = TextureDesc.Format == GPixelFormats[PF_R16F].PlatformFormat;
	bool bIsR32FFmt = TextureDesc.Format == GPixelFormats[PF_R32_FLOAT].PlatformFormat;
	if ( ! ensure(bIsRGBAFmt || bIsR16FFmt || bIsR32FFmt) )
	{
		OutData.Empty();
		return;
	}

	// Allocate the output buffer.
	OutData.SetNumUninitialized(SizeX * SizeY * SizeZ);

	// Read back the surface data from defined rect
	D3D12_BOX	Rect;
	Rect.left = InRect.Min.X;
	Rect.top = InRect.Min.Y;
	Rect.right = InRect.Max.X;
	Rect.bottom = InRect.Max.Y;
	Rect.back = ZMinMax.Y;
	Rect.front = ZMinMax.X;

	// create a temp 3d texture to copy render target to
	TRefCountPtr<FD3D12Resource> TempTexture3D;
	const uint32 BlockBytes = GPixelFormats[TextureRHI->GetFormat()].BlockBytes;
	const uint32 XBytesAligned = Align(TextureDesc.Width * BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	const uint32 DepthBytesAligned = XBytesAligned * TextureDesc.Height;
	const uint32 MipBytesAligned = DepthBytesAligned * TextureDesc.DepthOrArraySize;
	VERIFYD3D12RESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, Node, Node, MipBytesAligned, TempTexture3D.GetInitReference(), nullptr));

	// Copy the data to a staging resource.
	uint32 Subresource = 0;
	uint32 BytesPerPixel = D3D12RT_ComputeBytesPerPixel(TextureDesc.Format);
	D3D12_SUBRESOURCE_FOOTPRINT DestSubresource;
	DestSubresource.Depth = TextureDesc.DepthOrArraySize;
	DestSubresource.Height = TextureDesc.Height;
	DestSubresource.Width = TextureDesc.Width;
	DestSubresource.Format = TextureDesc.Format;
	DestSubresource.RowPitch = XBytesAligned;
	check(DestSubresource.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT == 0);	// Make sure we align correctly.

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedTexture3D = { 0 };
	PlacedTexture3D.Offset = 0;
	PlacedTexture3D.Footprint = DestSubresource;

	CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(TempTexture3D->GetResource(), PlacedTexture3D);
	CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(Texture->GetResource()->GetResource(), Subresource);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda(
	[Texture, GPUIndex, Rect, DestCopyLocation = MoveTemp(DestCopyLocation), SourceCopyLocation = MoveTemp(SourceCopyLocation), TempTexture3D = CopyTemp(TempTexture3D)](FRHICommandListBase& ExecutingCmdList)
	{
		FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);

		FScopedResourceBarrier ConditionalScopeResourceBarrier(
			Context,
			Texture->GetResource(),
			ED3D12Access::CopySrc,
			SourceCopyLocation.SubresourceIndex);
		// Don't need to transition upload heaps

		Context.FlushResourceBarriers();
		Context.CopyTextureRegionChecked(
			&DestCopyLocation,
			0, 0, 0,
			Texture->GetFormat(),
			&SourceCopyLocation,
			&Rect,
			Texture->GetFormat(),
			TempTexture3D->GetName(),		// DestName
			Texture->GetName()				// SourceName
		);

		Context.UpdateResidency(Texture->GetResource());
		Context.UpdateResidency(TempTexture3D);
	});

	// We need to execute the command list so we can read the data from the map below
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	// Lock the staging resource.
	void* pData;
	VERIFYD3D12RESULT(TempTexture3D->GetResource()->Map(0, nullptr, &pData));

	// Read the data out of the buffer
	if (bIsRGBAFmt)
	{
		// Texture is RGBA16F format
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const FFloat16Color* SrcPtr = (const FFloat16Color*)((const uint8*)pData + (Y - InRect.Min.Y) * XBytesAligned + (Z - ZMinMax.X) * DepthBytesAligned);
				int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY;
				check(Index < OutData.Num());
				FFloat16Color* DestPtr = &OutData[Index];
				FMemory::Memcpy(DestPtr, SrcPtr, SizeX * sizeof(FFloat16Color));
			}
		}
	}
	else if (bIsR16FFmt)
	{
		// Texture is R16F format
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const FFloat16* SrcPtr = (const FFloat16*)((const uint8*)pData + (Y - InRect.Min.Y) * XBytesAligned + (Z - ZMinMax.X) * DepthBytesAligned);
				for (int32 X = InRect.Min.X; X < InRect.Max.X; ++X)
				{
					int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY + X;
					check(Index < OutData.Num());
					OutData[Index].R = SrcPtr[X];
					OutData[Index].A = FFloat16(1.0f); // ensure full alpha (as if you sampled on GPU)
				}
			}
		}
	}
	else if (bIsR32FFmt)
	{
		// Texture is PF_R32_FLOAT format
		for (int32 Z = ZMinMax.X; Z < ZMinMax.Y; ++Z)
		{
			for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; ++Y)
			{
				const float* SrcPtr = (const float*)((const uint8*)pData + (Y - InRect.Min.Y) * XBytesAligned + (Z - ZMinMax.X) * DepthBytesAligned);
				for (int32 X = InRect.Min.X; X < InRect.Max.X; ++X)
				{
					int32 Index = (Y - InRect.Min.Y) * SizeX + (Z - ZMinMax.X) * SizeX * SizeY + X;
					check(Index < OutData.Num());
					OutData[Index].R = FFloat16(SrcPtr[X]);
					OutData[Index].A = FFloat16(1.0f); // ensure full alpha (as if you sampled on GPU)
				}
			}
		}
	}
	else
	{
		// unsupported format; checked for this earlier
		check(0);
	}

	TempTexture3D->GetResource()->Unmap(0, nullptr);
}
