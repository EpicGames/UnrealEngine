// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Residency.h: D3D memory residency functions.
=============================================================================*/

#pragma once

#include "D3D12RHI.h" // For ENABLE_RESIDENCY_MANAGEMENT

#if !defined(D3D12_PLATFORM_NEEDS_RESIDENCY_MANAGEMENT)
	#define D3D12_PLATFORM_NEEDS_RESIDENCY_MANAGEMENT 1
#endif

#if !(D3D12_PLATFORM_NEEDS_RESIDENCY_MANAGEMENT)
static_assert(ENABLE_RESIDENCY_MANAGEMENT == 0, "This platform doesn't need memory residency management. Please disable it.");
namespace D3DX12Residency
{
	class ManagedObject {};
	class ResidencySet {};
	class ResidencyManager {};

	class IDXGIAdapter3;
}
#else
#include "D3D12Util.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable: 6031)
	#include <D3DX12Residency.h>
#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#if ENABLE_RESIDENCY_MANAGEMENT
extern bool GEnableResidencyManagement;
extern bool GD3D12StartResourceResident;
#endif

struct FD3D12ResidencyHandle : public D3DX12Residency::ManagedObject
{
#if DO_CHECK
	class FD3D12GPUObject* GPUObject = nullptr;
#endif
};

namespace D3DX12Residency
{
#if ENABLE_RESIDENCY_MANAGEMENT
	inline void Initialize(FD3D12ResidencyHandle& Object, ID3D12Pageable* pResource, uint64 ObjectSize, class FD3D12GPUObject* GPUObject)
	{
		if (GEnableResidencyManagement)
		{
#if DO_CHECK
			Object.GPUObject = GPUObject;
#endif
			Object.Initialize(pResource, ObjectSize, 0, GD3D12StartResourceResident ? ManagedObject::RESIDENCY_STATUS::RESIDENT : ManagedObject::RESIDENCY_STATUS::EVICTED);
		}
	}
#endif

	inline bool IsInitialized(FD3D12ResidencyHandle& Object)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		return GEnableResidencyManagement && Object.IsInitialized();
#else
		return false;
#endif
	}

	inline bool IsInitialized(FD3D12ResidencyHandle* pObject)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		return GEnableResidencyManagement && pObject && IsInitialized(*pObject);
#else
		return false;
#endif
	}

	inline void BeginTrackingObject(ResidencyManager& ResidencyManager, FD3D12ResidencyHandle& Object)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement)
		{
			ResidencyManager.BeginTrackingObject(&Object);
		}
#endif
	}

	inline void EndTrackingObject(ResidencyManager& ResidencyManager, FD3D12ResidencyHandle& Object)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement)
		{
			ResidencyManager.EndTrackingObject(&Object);
		}
#endif
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	inline void InitializeResidencyManager(ResidencyManager& ResidencyManager, ID3D12Device* Device, uint32 GPUIndex, IDXGIAdapter3* Adapter, uint32 MaxLatency)
	{
		if (GEnableResidencyManagement)
		{
			ResidencyManager.SetStartEvicted(!GD3D12StartResourceResident);
			VERIFYD3D12RESULT(ResidencyManager.Initialize(Device, GPUIndex, Adapter, MaxLatency));
		}
	}
#endif

	inline void DestroyResidencyManager(ResidencyManager& ResidencyManager)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement)
		{
			ResidencyManager.Destroy();
		}
#endif
	}

	inline ResidencySet* CreateResidencySet(ResidencyManager& ResidencyManager)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		return GEnableResidencyManagement ? ResidencyManager.CreateResidencySet() : nullptr;
#else
		return nullptr;
#endif
	}

	inline void DestroyResidencySet(ResidencyManager& ResidencyManager, ResidencySet* pSet)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement && pSet)
		{
			ResidencyManager.DestroyResidencySet(pSet);
		}
#endif
	}

	inline void Open(ResidencySet* pSet)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement && pSet)
		{
			VERIFYD3D12RESULT(pSet->Open());
		}
#endif
	}

	inline void Close(ResidencySet* pSet)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement && pSet)
		{
			VERIFYD3D12RESULT(pSet->Close());
		}
#endif
	}

	inline void Insert(ResidencySet& Set, FD3D12ResidencyHandle& Object)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement)
		{
			check(Object.IsInitialized());
			Set.Insert(&Object);
		}
#endif
	}

	inline void Insert(ResidencySet& Set, FD3D12ResidencyHandle* pObject)
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (GEnableResidencyManagement)
		{
			check(pObject && pObject->IsInitialized());
			Set.Insert(pObject);
		}
#endif
	}
}

typedef D3DX12Residency::ResidencySet FD3D12ResidencySet;
typedef D3DX12Residency::ResidencyManager FD3D12ResidencyManager;
