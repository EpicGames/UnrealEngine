// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12ResourceCollection.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

#include "D3D12RHIPrivate.h"
#include "D3D12CommandContext.h"
#include "D3D12TextureReference.h"

FD3D12ResourceCollection::FD3D12ResourceCollection(FD3D12Device* InParent,FD3D12Buffer* InBuffer, TConstArrayView<FRHIResourceCollectionMember> InMembers, FD3D12ResourceCollection* FirstLinkedObject)
	: FRHIResourceCollection(InMembers)
	, FD3D12DeviceChild(InParent)
	, Buffer(InBuffer->GetLinkedObject(InParent->GetGPUIndex()))
{
	const uint32 GpuIndex = InParent->GetGPUIndex();

	for (const FRHIResourceCollectionMember& Member : InMembers)
	{
		switch (Member.Type)
		{
		case FRHIResourceCollectionMember::EType::Texture:
		{
			if (FRHITextureReference* TextureReferenceRHI = static_cast<FRHITexture*>(Member.Resource)->GetTextureReference())
			{
				FD3D12RHITextureReference* TextureReference = FD3D12CommandContext::RetrieveObject<FD3D12RHITextureReference>(TextureReferenceRHI, GpuIndex);
				AllTextureReferences.Emplace(TextureReference);
			}
			else
			{
				FD3D12Texture* Texture = FD3D12CommandContext::RetrieveTexture(static_cast<FRHITexture*>(Member.Resource), GpuIndex);
				AllSrvs.Emplace(Texture->GetShaderResourceView());
			}
		}
		break;
		case FRHIResourceCollectionMember::EType::TextureReference:
		{
			FD3D12RHITextureReference* TextureReference = FD3D12CommandContext::RetrieveObject<FD3D12RHITextureReference>(Member.Resource, GpuIndex);
			AllTextureReferences.Emplace(TextureReference);
		}
		break;
		case FRHIResourceCollectionMember::EType::ShaderResourceView:
		{
			FD3D12ShaderResourceView_RHI* ShaderResourceView = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(Member.Resource, GpuIndex);
			AllSrvs.Emplace(ShaderResourceView);
		}
		break;
		}
	}

	D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
	SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
	SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
	SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
	SRVDesc.Buffer.FirstElement = InBuffer->ResourceLocation.GetOffsetFromBaseOfResource() / 4;
	SRVDesc.Buffer.NumElements = UE::RHICore::CalculateResourceCollectionMemorySize(InMembers) / 4;
	SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;

	BufferSRV = MakeShared<FD3D12ShaderResourceView>(InParent, FirstLinkedObject ? FirstLinkedObject->BufferSRV.Get() : nullptr, ERHIDescriptorType::BufferSRV);
	BufferSRV->CreateView(InBuffer, SRVDesc, FD3D12ShaderResourceView::EFlags::None);
}

FD3D12ResourceCollection::~FD3D12ResourceCollection() = default;

FRHIDescriptorHandle FD3D12ResourceCollection::GetBindlessHandle() const
{
	return BufferSRV->GetBindlessHandle();
}

FRHIResourceCollectionRef FD3D12DynamicRHI::RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
{
	FRHIBuffer* RHIBuffer = UE::RHICore::CreateResourceCollectionBuffer(RHICmdList, InMembers);
	FD3D12Buffer* Buffer = ResourceCast(RHIBuffer);

	FRHIViewDesc::FBufferSRV::FInitializer ViewDesc = FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw);
	FShaderResourceViewRHIRef ShaderResourceView = RHICmdList.CreateShaderResourceView(Buffer, ViewDesc);

	return GetAdapter().CreateLinkedObject<FD3D12ResourceCollection>(FRHIGPUMask::All(), [Buffer, InMembers](FD3D12Device* Device, FD3D12ResourceCollection* FirstLinkedObject)
	{
		return new FD3D12ResourceCollection(Device, Buffer, InMembers, FirstLinkedObject);
	});
}

void FD3D12DynamicRHI::RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
{
	TConstArrayView<FRHIResourceCollectionMember> MemberUpdates = UE::RHICore::GetValidResourceCollectionUpdateList(InResourceCollection, InStartIndex, InMemberUpdates);
	if (MemberUpdates.IsEmpty())
	{
		return;
	}

	FD3D12ResourceCollection* ResourceCollection = ResourceCast(InResourceCollection);
	TConstArrayView<FRHIResourceCollectionMember> CommandMemberUpdates = RHICmdList.IsTopOfPipe() ? RHICmdList.AllocArray(MemberUpdates) : MemberUpdates;

	RHICmdList.EnqueueLambda([ResourceCollection, InStartIndex, CommandMemberUpdates](FRHICommandListBase& RHICmdList)
	{
		const uint32 UploadSize = CommandMemberUpdates.Num() * sizeof(uint32);
		const uint32 DestOffset = (InStartIndex + 1) * sizeof(uint32);

		ResourceCollection->UpdateMembers(InStartIndex, CommandMemberUpdates);

		for (FD3D12Buffer::FLinkedObjectIterator BufferIt(ResourceCollection->Buffer); BufferIt; ++BufferIt)
		{
			FD3D12Buffer& Buffer = *BufferIt;
			FD3D12Device* Device = Buffer.GetParentDevice();
			FD3D12UploadHeapAllocator& Allocator = Device->GetParentAdapter()->GetUploadHeapAllocator(Device->GetGPUIndex());

			FD3D12ResourceLocation UploadResourceLocation(Device);
			uint32* UploadData = reinterpret_cast<uint32*>(Allocator.AllocUploadResource(UploadSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, UploadResourceLocation));

			UE::RHICore::FillResourceCollectionUpdateMemory(UploadData, CommandMemberUpdates);

			FD3D12CommandContext& Context = FD3D12CommandContext::Get(RHICmdList, Device->GetGPUIndex());

			FD3D12Resource* SourceResource = UploadResourceLocation.GetResource();
			FD3D12Resource* DestResource = Buffer.ResourceLocation.GetResource();

			// Clear the resource if still bound to make sure the SRVs are rebound again on next operation (and get correct resource transitions enqueued)
			Context.ConditionalClearShaderResource(&Buffer.ResourceLocation, EShaderParameterTypeMask::SRVMask);

			FScopedResourceBarrier ScopeResourceBarrierDest(Context, DestResource, ED3D12Access::CopyDest, 0);

			// Don't need to transition upload heaps
			Context.FlushResourceBarriers();

			Context.UpdateResidency(DestResource);
			Context.UpdateResidency(SourceResource);

			Context.CopyBufferRegionChecked(
				DestResource->GetResource(), DestResource->GetName(),
				Buffer.ResourceLocation.GetOffsetFromBaseOfResource() + DestOffset,
				SourceResource->GetResource(), SourceResource->GetName(),
				UploadResourceLocation.GetOffsetFromBaseOfResource(),
				UploadSize
			);

			Context.ConditionalSplitCommandList();
		}
	});
}

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
