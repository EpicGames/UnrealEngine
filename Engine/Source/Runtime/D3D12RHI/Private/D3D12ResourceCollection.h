// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12Resources.h"
#include "D3D12RHICommon.h"
#include "RHIResources.h"
#include "RHICoreResourceCollection.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

class FD3D12Buffer;
class FD3D12RHITextureReference;
class FD3D12ShaderResourceView;

class FD3D12ResourceCollection : public FRHIResourceCollection, public FD3D12DeviceChild, public FD3D12LinkedAdapterObject<FD3D12ResourceCollection>
{
public:
	FD3D12ResourceCollection(FD3D12Device* InParent, FD3D12Buffer* InBuffer, TConstArrayView<FRHIResourceCollectionMember> InMembers, FD3D12ResourceCollection* FirstLinkedObject);
	~FD3D12ResourceCollection();

	virtual FRHIDescriptorHandle GetBindlessHandle() const final;

	FD3D12ShaderResourceView* GetShaderResourceView() const { return BufferSRV.Get(); }

	TRefCountPtr<FD3D12Buffer>           Buffer;
	TSharedPtr<FD3D12ShaderResourceView> BufferSRV;

	TArray<FD3D12ShaderResourceView*>    AllSrvs;
	TArray<FD3D12RHITextureReference*>   AllTextureReferences;
};

template<>
struct TD3D12ResourceTraits<FRHIResourceCollection>
{
	using TConcreteType = FD3D12ResourceCollection;
};

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING