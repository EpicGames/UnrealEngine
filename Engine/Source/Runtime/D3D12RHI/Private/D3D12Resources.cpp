// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Resources.cpp: D3D RHI utility implementation.
=============================================================================*/

#include "D3D12Resources.h"
#include "D3D12RHIPrivate.h"
#include "D3D12IntelExtensions.h"
#include "D3D12RayTracing.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreStats.h"

static TAutoConsoleVariable<int32> CVarD3D12ReservedResourceHeapSizeMB(
	TEXT("d3d12.ReservedResourceHeapSizeMB"),
	16,
	TEXT("Size of the backing heaps for reserved resources in megabytes (default 16MB)."),
	ECVF_ReadOnly
);

/////////////////////////////////////////////////////////////////////
//	ID3D12ResourceAllocator
/////////////////////////////////////////////////////////////////////


void ID3D12ResourceAllocator::AllocateTexture(
	uint32 GPUIndex,
	D3D12_HEAP_TYPE InHeapType,
	const FD3D12ResourceDesc& InDesc,
	EPixelFormat InUEFormat,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	const TCHAR* InName,
	FD3D12ResourceLocation& ResourceLocation)
{
	// Check if texture can be 4K aligned
	FD3D12ResourceDesc Desc = InDesc;
	bool b4KAligment = FD3D12Texture::CanBe4KAligned(Desc, InUEFormat);
	Desc.Alignment = b4KAligment ? D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT : D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;

	// Get the size and alignment for the allocation
	D3D12_RESOURCE_ALLOCATION_INFO Info = FD3D12DynamicRHI::GetD3DRHI()->GetAdapter().GetDevice(GPUIndex)->GetResourceAllocationInfo(Desc);
	AllocateResource(
		GPUIndex,
		InHeapType,
		Desc,
		Info.SizeInBytes,
		Info.Alignment,
		InInitialD3D12Access,
		InResourceStateMode,
		InDefaultD3D12Access,
		InClearValue,
		InName,
		ResourceLocation);
}

#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
TArray<DXGI_FORMAT, TInlineAllocator<4>> FD3D12ResourceDesc::GetCastableFormats() const
{
	TArray<DXGI_FORMAT, TInlineAllocator<4>> Result;

	if (UAVPixelFormat != PF_Unknown)
	{
		// We have to add the 'implied' castable formats for SRVs. Since we don't have any sRGB flags here, just add both formats.
		Result.Emplace(UE::DXGIUtilities::FindShaderResourceFormat(Format, true));
		Result.Emplace(UE::DXGIUtilities::FindShaderResourceFormat(Format, false));

		// Add the uncompressed UAV format we want
		Result.Emplace((DXGI_FORMAT)GPixelFormats[UAVPixelFormat].PlatformFormat);
	}

	return Result;
}
#endif // D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV

/////////////////////////////////////////////////////////////////////
//	FD3D12 Resource
/////////////////////////////////////////////////////////////////////

#if UE_BUILD_DEBUG
int64 FD3D12Resource::TotalResourceCount = 0;
int64 FD3D12Resource::NoStateTrackingResourceCount = 0;
#endif

FD3D12Resource::FD3D12Resource(FD3D12Device* ParentDevice,
	FRHIGPUMask VisibleNodes,
	ID3D12Resource* InResource,
	ED3D12Access InInitialD3D12Access,
	const FD3D12ResourceDesc& InDesc,
	FD3D12Heap* InHeap,
	D3D12_HEAP_TYPE InHeapType) : 
	FD3D12Resource(
		ParentDevice,
		VisibleNodes,
		InResource,
		InInitialD3D12Access,
		ED3D12ResourceStateMode::Default,
		ED3D12Access::Unknown,
		InDesc,
		InHeap,
		InHeapType)
{
}

FD3D12Resource::FD3D12Resource(FD3D12Device* ParentDevice,
	FRHIGPUMask VisibleNodes,
	ID3D12Resource* InResource,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	FD3D12ResourceDesc const& InDesc,
	FD3D12Heap* InHeap,
	D3D12_HEAP_TYPE InHeapType)
	: FD3D12DeviceChild(ParentDevice)
	, FD3D12MultiNodeGPUObject(ParentDevice->GetGPUMask(), VisibleNodes)
	, Resource(InResource)
	, Heap(InHeap)
	, Desc(InDesc)
	, HeapType(InHeapType)
	, PlaneCount(UE::DXGIUtilities::GetPlaneCount(Desc.Format))
	, bRequiresResourceStateTracking(true)
	, bRequiresResidencyTracking(bool(ENABLE_RESIDENCY_MANAGEMENT))
	, bDepthStencil(false)
	, bDeferDelete(true)
{
#if UE_BUILD_DEBUG
	FPlatformAtomics::InterlockedIncrement(&TotalResourceCount);
#endif

	D3D12_HEAP_DESC HeapDesc = {};
	D3D12_HEAP_PROPERTIES* HeapProps = nullptr;
	if (InHeap)
	{
		HeapDesc = InHeap->GetHeapDesc();
		HeapProps = &HeapDesc.Properties;
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	// Residency tracking is only used for GPU-only resources owned by the Engine.
	// Back buffers may be referenced outside of command lists (during presents), however D3DX12Residency.h library
	// uses fences tied to command lists to detect when it's safe to evict a resource, which is wrong for back buffers.
	// External/shared resources may be referenced by command buffers in third-party code.
	const bool bShouldTrackResource = (GNumExplicitGPUsForRendering == 1) || IsGPUOnly(InHeapType, HeapProps);
	bRequiresResidencyTracking = !Desc.bExternal && !Desc.bBackBuffer && bShouldTrackResource;
#endif

	// On Windows it's sadly enough not possible to get the GPU virtual address from the resource directly
	if (Resource
#if PLATFORM_WINDOWS
		&& Desc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER
#endif
		)
	{
		GPUVirtualAddress = Resource->GetGPUVirtualAddress();
	}

	InitializeResourceState(
		HeapProps,
		InInitialD3D12Access,
		InResourceStateMode,
		InDefaultD3D12Access);

#if NV_AFTERMATH
	AftermathHandle = UE::RHICore::Nvidia::Aftermath::D3D12::RegisterResource(InResource);
#endif

	if (Desc.bReservedResource)
	{
		checkf(Heap == nullptr, TEXT("Reserved resources are not expected to have a heap"));
		ReservedResourceData = MakeUnique<FD3D12ReservedResourceData>();
	}
}

FD3D12Resource::~FD3D12Resource()
{
#if ENABLE_RESIDENCY_MANAGEMENT
	if (D3DX12Residency::IsInitialized(ResidencyHandle))
	{
		D3DX12Residency::EndTrackingObject(GetParentDevice()->GetResidencyManager(), *ResidencyHandle);
	}
	delete ResidencyHandle;
#endif // ENABLE_RESIDENCY_MANAGEMENT

#if NV_AFTERMATH
	UE::RHICore::Nvidia::Aftermath::D3D12::UnregisterResource(AftermathHandle);
#endif

	if (Desc.bBackBuffer)
	{
		// Don't make the windows association call and release back buffer at the same time (see notes on critical section)
		FScopeLock Lock(&FD3D12Viewport::DXGIBackBufferLock);
		Resource.SafeRelease();
	}

	// Update reserved resources' physical memory stats.
	if (ReservedResourceData.IsValid() && ReservedResourceData->NumCommittedTiles > 0)
	{
		bool bBuffer = Desc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER;
		const uint64 DecommitBytes = GRHIGlobals.ReservedResources.TileSizeInBytes * ReservedResourceData->NumCommittedTiles;
		UE::RHICore::UpdateReservedResourceStatsOnCommit(DecommitBytes, bBuffer, false /* Decommit */);
		
		// The backing heaps are going to be released once this resource is destroyed.
		for (const TRefCountPtr<FD3D12Heap>& BackingHeap : ReservedResourceData->BackingHeaps)
		{
			DEC_MEMORY_STAT_BY(STAT_D3D12ReservedResourcePhysical, BackingHeap->GetHeapDesc().SizeInBytes);
		}
	}
}

struct FD3D12UpdateTileMappingsParams
{
	D3D12_TILE_RANGE_FLAGS RangeFlags = D3D12_TILE_RANGE_FLAG_NONE;
	D3D12_TILED_RESOURCE_COORDINATE Coord = {};
	D3D12_TILE_REGION_SIZE Size = {};
	ID3D12Heap* Heap = nullptr;
	uint32 HeapOffsetInTiles = 0;
};

void FD3D12Resource::CommitReservedResource(ID3D12CommandQueue* D3DCommandQueue, uint64 RequiredCommitSizeInBytes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CommitReservedResource);

	static constexpr uint64 TileSizeInBytes = GRHIGlobals.ReservedResources.TileSizeInBytes;
	static_assert(TileSizeInBytes == 65536, "Reserved resource tiles are expected to always be 64KB");

	check(Desc.bReservedResource);
	check(ReservedResourceData.IsValid());
	LLM_REALLOC_SCOPE(ReservedResourceData.Get());
	UE_MEMSCOPE_PTR(ReservedResourceData.Get());

	checkf(GRHIGlobals.ReservedResources.Supported,
		TEXT("Current RHI does not support reserved resources"));

	if (Desc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D)
	{
		checkf(GRHIGlobals.ReservedResources.SupportsVolumeTextures,
			TEXT("Current RHI does not support reserved volume textures"));
	}

	uint32 D3DResourceNumTiles = 0;
	D3D12_PACKED_MIP_INFO PackedMipDesc = {};
	D3D12_TILE_SHAPE TileShape = {};
	const uint32 FirstSubresource = 0;

	const bool bBuffer = Desc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER;

	const uint32 NumSubresources = SubresourceCount;
	const uint32 NumMipLevels = GetMipLevels();
	const uint32 NumArraySlices = GetArraySize();

	uint32 NumSubresourceTilings = NumMipLevels;
	TArray<D3D12_SUBRESOURCE_TILING, TInlineAllocator<16>> MipTilingInfo;
	
	check(NumSubresourceTilings >= 1);
	MipTilingInfo.SetNum(NumSubresourceTilings);

	ID3D12Device* D3DDevice = GetParentDevice()->GetDevice();
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();

	D3DDevice->GetResourceTiling(GetResource(), &D3DResourceNumTiles, &PackedMipDesc, &TileShape, &NumSubresourceTilings, FirstSubresource, MipTilingInfo.GetData());

	if (bBuffer)
	{
		// Buffers obviously don't have mips, but we can pretend they do to make the code below agnostic to resource type
		PackedMipDesc.NumStandardMips = 1;
	}

	check(MipTilingInfo.Num() == PackedMipDesc.NumStandardMips + PackedMipDesc.NumPackedMips);

	const uint32 NumPackedTilesPerArraySlice = PackedMipDesc.NumTilesForPackedMips;
	const uint32 NumTotalPackedMipTiles = NumPackedTilesPerArraySlice * NumArraySlices;
	const uint32 NumTotalStandardMipTiles = D3DResourceNumTiles - NumTotalPackedMipTiles;

	const uint64 TotalSize = D3DResourceNumTiles * TileSizeInBytes;

	RequiredCommitSizeInBytes = FMath::Min<uint64>(RequiredCommitSizeInBytes, TotalSize);
	RequiredCommitSizeInBytes = AlignArbitrary(RequiredCommitSizeInBytes, TileSizeInBytes);

	const uint64 MaxHeapSize = uint64(CVarD3D12ReservedResourceHeapSizeMB.GetValueOnAnyThread()) * 1024 * 1024;
	const uint64 NumHeaps = FMath::DivideAndRoundUp(TotalSize, MaxHeapSize);

	ReservedResourceData->BackingHeaps.Reserve(NumHeaps);

	const uint32 MaxTilesPerHeap = uint32(MaxHeapSize / TileSizeInBytes);

	// Set high residency priority based on the same heuristics as D3D12 committed resources,
	// i.e. normal priority unless it's a UAV/RT/DS texture.
	const bool bRenderOrDepthTarget = EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);
	const bool bHighPriorityResource = bRenderOrDepthTarget || EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);
	const uint32 GPUIndex = GetParentDevice()->GetGPUIndex();

	D3D12_HEAP_PROPERTIES BackingHeapProps = {};
	BackingHeapProps.Type = D3D12_HEAP_TYPE_DEFAULT;
	BackingHeapProps.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_UNKNOWN;
	BackingHeapProps.MemoryPoolPreference = D3D12_MEMORY_POOL_UNKNOWN;
	BackingHeapProps.CreationNodeMask = GetGPUMask().GetNative();
	BackingHeapProps.VisibleNodeMask = GetVisibilityMask().GetNative();

	uint32 NumStandardTilesPerArraySlice = 0;
	uint32 NumTotalTiles = 0;

	if (bBuffer)
	{
		NumTotalTiles = NumStandardTilesPerArraySlice = D3DResourceNumTiles;
		checkf(D3DResourceNumTiles == MipTilingInfo[0].WidthInTiles,
			TEXT("Reserved buffers are expected to have trivial tiling configuration: single 1D subresource that contains all tiles."));
	}
	else
	{
		NumStandardTilesPerArraySlice = NumTotalStandardMipTiles / NumArraySlices;
		NumTotalTiles = (NumStandardTilesPerArraySlice + NumPackedTilesPerArraySlice) * NumArraySlices;
	}

	const uint32 NumTotalTilesPerArraySlice = NumStandardTilesPerArraySlice + NumPackedTilesPerArraySlice;

	checkf(D3DResourceNumTiles == NumTotalTiles,
		TEXT("D3D resource size in tiles: %d, computed size in tiles: %d"),
		D3DResourceNumTiles, NumTotalTiles);

	const uint32 NumRequiredCommitTiles = RequiredCommitSizeInBytes / TileSizeInBytes;

	auto GetTiledResourceCoordinate = [&MipTilingInfo, &PackedMipDesc, D3DResourceNumTiles, NumTotalTilesPerArraySlice, NumSubresources, MaxTilesPerHeap]
		(uint32 OffsetInTiles, uint32 NumTiles) -> D3D12_TILED_RESOURCE_COORDINATE
	{
		check(OffsetInTiles < D3DResourceNumTiles);

		const uint32 ArraySliceIndex = OffsetInTiles / NumTotalTilesPerArraySlice;
		const uint32 TileIndexInArraySlice = OffsetInTiles % NumTotalTilesPerArraySlice;
		const uint32 NumTotalMips = MipTilingInfo.Num();

		uint32 MipLevel = 0;

		{
			uint32 NextMipTileThreshold = 0;
			while (MipLevel < PackedMipDesc.NumStandardMips)
			{
				const D3D12_SUBRESOURCE_TILING& CurrentMipTiling = MipTilingInfo[MipLevel];
				NextMipTileThreshold += CurrentMipTiling.WidthInTiles * CurrentMipTiling.HeightInTiles * CurrentMipTiling.DepthInTiles;

				if (TileIndexInArraySlice < NextMipTileThreshold)
				{
					break;
				}

				MipLevel += 1;
			}
		}

		D3D12_TILED_RESOURCE_COORDINATE ResourceCoordinate = {}; // Coordinates are in tiles, not pixels

		ResourceCoordinate.Subresource = MipLevel + ArraySliceIndex * NumTotalMips;

		const D3D12_SUBRESOURCE_TILING& CurrentMipTiling = MipTilingInfo[MipLevel];

		if (MipLevel < PackedMipDesc.NumStandardMips)
		{
			// Standard mip level case

			check(CurrentMipTiling.StartTileIndexInOverallResource != ~0u)

			const uint32 NumTilesPerVolumeSlice = CurrentMipTiling.WidthInTiles * CurrentMipTiling.HeightInTiles;

			const uint32 TileIndexInMipLevel = TileIndexInArraySlice - CurrentMipTiling.StartTileIndexInOverallResource;

			ResourceCoordinate.X = TileIndexInMipLevel % CurrentMipTiling.WidthInTiles;
			ResourceCoordinate.Y = (TileIndexInMipLevel / CurrentMipTiling.WidthInTiles) % CurrentMipTiling.HeightInTiles;
			ResourceCoordinate.Z = TileIndexInMipLevel / NumTilesPerVolumeSlice;
		}
		else
		{
			// Packed mip level case

			checkf(NumTiles <= MaxTilesPerHeap,
			       TEXT("Reserved texture packed mip level requires tiles: %d, maximum supported tiles: %d. ")
			       TEXT("Increase d3d12.ReservedResourceHeapSizeMB or avoid packed mips by using a larger texture dimensions."),
			       NumTiles, MaxTilesPerHeap);

			// Entire packed mip chain must be covered in one map operation, so mapping origin is always 0
			ResourceCoordinate.X = 0;
			ResourceCoordinate.Y = 0;
			ResourceCoordinate.Z = 0;
		}

		return ResourceCoordinate;
	};

	TArray<FD3D12UpdateTileMappingsParams> MappingParams;
	TArray<FD3D12ResidencyHandle*> UsedResidencyHandles;

	const uint32 NumPreviousCommittedTiles = ReservedResourceData->NumCommittedTiles;
	if (ReservedResourceData->NumCommittedTiles > NumRequiredCommitTiles) // Decommit / shrink case
	{
		check(!ReservedResourceData->BackingHeaps.IsEmpty());

		// Iterate through heaps in reverse order, unmap ranges and release heaps if they are completely unused
		while (ReservedResourceData->NumCommittedTiles > NumRequiredCommitTiles)
		{
			TRefCountPtr<FD3D12Heap>& LastHeap = ReservedResourceData->BackingHeaps.Last();
			const uint32 NumTotalTilesInHeap = LastHeap->GetHeapDesc().SizeInBytes / TileSizeInBytes;

			check(ReservedResourceData->NumSlackTiles <= NumTotalTilesInHeap);
			const uint32 NumUsedTilesInHeap = NumTotalTilesInHeap - ReservedResourceData->NumSlackTiles;

			check(NumUsedTilesInHeap <= ReservedResourceData->NumCommittedTiles);
			const uint32 HeapFirstTile = ReservedResourceData->NumCommittedTiles - NumUsedTilesInHeap;

			const uint32 RegionEnd = ReservedResourceData->NumCommittedTiles;
			const uint32 RegionBegin = FMath::Max(HeapFirstTile, NumRequiredCommitTiles);

			D3D12_TILE_REGION_SIZE RegionSize = {};
			RegionSize.UseBox = false;
			RegionSize.NumTiles = RegionEnd - RegionBegin;

			// Coordinates are in tiles, not pixels
			D3D12_TILED_RESOURCE_COORDINATE ResourceCoordinate = GetTiledResourceCoordinate(RegionBegin, RegionSize.NumTiles);

			FD3D12UpdateTileMappingsParams Params = {};
			Params.RangeFlags = D3D12_TILE_RANGE_FLAG_NULL;
			Params.Coord = ResourceCoordinate;
			Params.Size = RegionSize;
			MappingParams.Add(Params);

			if (HeapFirstTile == RegionBegin)
			{
				// All tiles from this heap were unmapped, so it can be dropped
				DEC_MEMORY_STAT_BY(STAT_D3D12ReservedResourcePhysical, LastHeap->GetHeapDesc().SizeInBytes);
				LastHeap->DeferDelete();
				ReservedResourceData->BackingHeaps.Pop();
				int32 NumResidencyHandles = ReservedResourceData->NumResidencyHandlesPerHeap.Last();
				ReservedResourceData->NumResidencyHandlesPerHeap.Pop();
				while (NumResidencyHandles != 0)
				{
					ReservedResourceData->ResidencyHandles.Pop();
					--NumResidencyHandles;
				}

				ReservedResourceData->NumSlackTiles = 0;
			}
			else
			{
				// Heap remains referenced, but now contains some free tiles at the end (which we just unmapped)
				ReservedResourceData->NumSlackTiles += RegionSize.NumTiles;
				check(ReservedResourceData->NumSlackTiles <= NumTotalTilesInHeap);
			}

			check(ReservedResourceData->NumCommittedTiles >= RegionSize.NumTiles);
			ReservedResourceData->NumCommittedTiles -= RegionSize.NumTiles;
		}
	}
	else // Commit / grow case
	{
		bool bForceGetGPUAddress = false;
		// See FD3D12Buffer::UpdateAllocationTags: we might need to rebase the allocation tag and we need the gpu address for that
		LLM_IF_ENABLED(bForceGetGPUAddress=true);
		while (ReservedResourceData->NumCommittedTiles < NumRequiredCommitTiles)
		{
			const uint32 NumRemainingTiles = NumRequiredCommitTiles - ReservedResourceData->NumCommittedTiles;

			ID3D12Heap* D3DHeap = nullptr;

			uint32 HeapRangeStartOffsetInTiles = 0;

			D3D12_TILE_REGION_SIZE RegionSize = {};
			RegionSize.UseBox = false;

			if (ReservedResourceData->NumSlackTiles)
			{
				// Consume any heap slack space before allocating a new heap

				const TRefCountPtr<FD3D12Heap>& LastHeap = ReservedResourceData->BackingHeaps.Last();
				const uint32 NumTotalTilesInHeap = LastHeap->GetHeapDesc().SizeInBytes / TileSizeInBytes;

				RegionSize.NumTiles = FMath::Min(ReservedResourceData->NumSlackTiles, NumRemainingTiles);
				HeapRangeStartOffsetInTiles = NumTotalTilesInHeap - ReservedResourceData->NumSlackTiles;

				D3DHeap = LastHeap->GetHeap();

				check(RegionSize.NumTiles <= ReservedResourceData->NumSlackTiles);
				ReservedResourceData->NumSlackTiles -= RegionSize.NumTiles;

				UsedResidencyHandles.Append(LastHeap->GetResidencyHandles());
			}
			else
			{
				// Create a new heap to service the commit request

				RegionSize.NumTiles = FMath::Min(MaxTilesPerHeap, NumRemainingTiles);
				HeapRangeStartOffsetInTiles = 0;

#if NAME_OBJECTS
				const int32 HeapIndex = ReservedResourceData->BackingHeaps.Num();
				FString HeapName = FString::Printf(TEXT("%s.Heap[%d]"), DebugName.IsValid() ? *DebugName.ToString() : TEXT("UNKNOWN"), HeapIndex);
				const TCHAR* HeapNameChars = *HeapName;
#else
				const TCHAR* HeapNameChars = TEXT("ReservedResourceBackingHeap");
#endif // NAME_OBJECTS

				const D3D12_HEAP_FLAGS TextureHeapFlags = bRenderOrDepthTarget
					? D3D12_HEAP_FLAG_ALLOW_ONLY_RT_DS_TEXTURES 
					: D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;

				const D3D12_HEAP_FLAGS HeapFlags = bBuffer ? D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS : TextureHeapFlags;

				static_assert((D3D12_HEAP_FLAG_DENY_BUFFERS | D3D12_HEAP_FLAG_DENY_RT_DS_TEXTURES) == D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES);
				static_assert((D3D12_HEAP_FLAG_DENY_BUFFERS | D3D12_HEAP_FLAG_DENY_NON_RT_DS_TEXTURES) == D3D12_HEAP_FLAG_ALLOW_ONLY_RT_DS_TEXTURES);

				const uint32 ThisHeapSize = RegionSize.NumTiles * TileSizeInBytes;
				D3D12_HEAP_DESC NewHeapDesc = {};
				NewHeapDesc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
				NewHeapDesc.Flags = HeapFlags;
				NewHeapDesc.SizeInBytes = ThisHeapSize;
				NewHeapDesc.Properties = BackingHeapProps;

				VERIFYD3D12RESULT(D3DDevice->CreateHeap(&NewHeapDesc, IID_PPV_ARGS(&D3DHeap)));
				INC_MEMORY_STAT_BY(STAT_D3D12ReservedResourcePhysical, NewHeapDesc.SizeInBytes);

				if (bHighPriorityResource)
				{
					Adapter->SetResidencyPriority(D3DHeap, D3D12_RESIDENCY_PRIORITY_HIGH, GPUIndex, NewHeapDesc.SizeInBytes);
				}

				TRefCountPtr<FD3D12Heap> NewHeap = new FD3D12Heap(GetParentDevice(), GetVisibilityMask());
				NewHeap->SetHeap(D3DHeap, HeapNameChars, true /*bTrack*/, bForceGetGPUAddress);
				NewHeap->BeginTrackingResidency(ThisHeapSize);

				TConstArrayView<FD3D12ResidencyHandle*> HeapResidencyHandles = NewHeap->GetResidencyHandles();
				ReservedResourceData->ResidencyHandles.Append(HeapResidencyHandles);
				ReservedResourceData->NumResidencyHandlesPerHeap.Add(HeapResidencyHandles.Num());
				ReservedResourceData->BackingHeaps.Add(MoveTemp(NewHeap));

				UsedResidencyHandles.Append(HeapResidencyHandles);
			}

			// Coordinates are in tiles, not pixels
			D3D12_TILED_RESOURCE_COORDINATE ResourceCoordinate = GetTiledResourceCoordinate(ReservedResourceData->NumCommittedTiles, RegionSize.NumTiles);

			FD3D12UpdateTileMappingsParams Params = {};
			Params.RangeFlags = D3D12_TILE_RANGE_FLAG_NONE;
			Params.Coord = ResourceCoordinate;
			Params.Size = RegionSize;
			Params.Heap = D3DHeap;
			Params.HeapOffsetInTiles = HeapRangeStartOffsetInTiles;
			MappingParams.Add(Params);

			ReservedResourceData->NumCommittedTiles += RegionSize.NumTiles;
		}
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	FD3D12ResidencyManager& ResidencyManager = GetParentDevice()->GetResidencyManager();

	if (GEnableResidencyManagement && !UsedResidencyHandles.IsEmpty())
	{
		HRESULT HR = S_OK;

		FD3D12ResidencySet* ResidencySet = ResidencyManager.CreateResidencySet();
		HR = ResidencySet->Open();
		checkf(SUCCEEDED(HR), TEXT("Failed to open residency set. Error code: 0x%08x."), uint32(HR));

		for (FD3D12ResidencyHandle* Handle : UsedResidencyHandles)
		{
			if (D3DX12Residency::IsInitialized(Handle))
			{
				ResidencySet->Insert(Handle);
			}
		}

		HR = ResidencySet->Close();
		checkf(SUCCEEDED(HR), TEXT("Failed to close residency set. Error code: 0x%08x."), uint32(HR));

		// NOTE: ResidencySet ownership is taken over by the residency manager.
		// It is destroyed when paging work completes, which may happen async on another thread in some cases.
		HR = ResidencyManager.MakeResident(D3DCommandQueue, MoveTemp(ResidencySet));
		checkf(SUCCEEDED(HR), TEXT("Failed to process residency set. Error code: 0x%08x."), uint32(HR));
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT

	for (const FD3D12UpdateTileMappingsParams& Params : MappingParams)
	{
		D3DCommandQueue->UpdateTileMappings(GetResource(), 1 /*NumRegions*/,
			&Params.Coord, &Params.Size, Params.Heap,
			1 /*NumRanges*/,
			&Params.RangeFlags,
			&Params.HeapOffsetInTiles,
			&Params.Size.NumTiles,
			D3D12_TILE_MAPPING_FLAG_NONE);
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	if (GEnableResidencyManagement && !UsedResidencyHandles.IsEmpty())
	{
		// Signal the fence for this queue after UpdateTileMappings complete.
		// This is analogous to executing a command list that references a set of resources.
		ResidencyManager.SignalFence(D3DCommandQueue);
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT

	checkf(ReservedResourceData->NumCommittedTiles == NumRequiredCommitTiles,
		TEXT("Reserved resource was not fully processed while committing physical memory. Expected to process tiles: %d, actually processed: %d"),
		D3DResourceNumTiles, ReservedResourceData->NumCommittedTiles);

	if (ReservedResourceData->NumCommittedTiles != NumPreviousCommittedTiles)
	{
		int64 CommitDeltaInBytes = TileSizeInBytes * FMath::Abs((int32)ReservedResourceData->NumCommittedTiles - (int32)NumPreviousCommittedTiles);
		UE::RHICore::UpdateReservedResourceStatsOnCommit(CommitDeltaInBytes, bBuffer, ReservedResourceData->NumCommittedTiles > NumPreviousCommittedTiles);
	}
}

ID3D12Pageable* FD3D12Resource::GetPageable()
{
	if (IsPlacedResource())
	{
		return (ID3D12Pageable*)(GetHeap()->GetHeap());
	}
	else
	{
		return (ID3D12Pageable*)GetResource();
	}
}

void FD3D12Resource::StartTrackingForResidency()
{
#if ENABLE_RESIDENCY_MANAGEMENT

	if (!bRequiresResidencyTracking)
	{
		return;
	}

	checkf(!Desc.bBackBuffer, TEXT("Residency tracking is not expected for back buffers"));
	checkf(!Desc.bExternal, TEXT("Residency tracking is not expected for externally-owned resources"));

	if (!IsPlacedResource() && !IsReservedResource())
	{
		checkf(!ResidencyHandle, TEXT("Residency tracking is already initialzied for this resource"));
		ResidencyHandle = new FD3D12ResidencyHandle;

		const D3D12_RESOURCE_ALLOCATION_INFO Info = GetParentDevice()->GetResourceAllocationInfoUncached(Desc);
		D3DX12Residency::Initialize(*ResidencyHandle, Resource.GetReference(), Info.SizeInBytes, this);
		D3DX12Residency::BeginTrackingObject(GetParentDevice()->GetResidencyManager(), *ResidencyHandle);
	}
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

void FD3D12Resource::DeferDelete()
{
	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(this);
}

/////////////////////////////////////////////////////////////////////
//	FD3D12 Heap
/////////////////////////////////////////////////////////////////////

FD3D12Heap::FD3D12Heap(FD3D12Device* Parent, FRHIGPUMask VisibleNodes, HeapId InTraceParentHeapId) :
	FD3D12DeviceChild(Parent),
	FD3D12MultiNodeGPUObject(Parent->GetGPUMask(), VisibleNodes),
	TraceParentHeapId(InTraceParentHeapId)
{
}

FD3D12Heap::~FD3D12Heap()
{
#if UE_MEMORY_TRACE_ENABLED
	if (GPUVirtualAddress != 0)
	{
		MemoryTrace_UnmarkAllocAsHeap(GPUVirtualAddress, TraceHeapId);
		MemoryTrace_Free(GPUVirtualAddress, EMemoryTraceRootHeap::VideoMemory);
	}
#endif	

#if TRACK_RESOURCE_ALLOCATIONS
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
	if (GPUVirtualAddress != 0 && bTrack)
	{
		Adapter->ReleaseTrackedHeap(this);
	}
#endif // TRACK_RESOURCE_ALLOCATIONS

#if ENABLE_RESIDENCY_MANAGEMENT
	if (D3DX12Residency::IsInitialized(ResidencyHandle))
	{
		D3DX12Residency::EndTrackingObject(GetParentDevice()->GetResidencyManager(), *ResidencyHandle);
	}
	delete ResidencyHandle;
#endif // ENABLE_RESIDENCY_MANAGEMENT

	// Release actual d3d object
	Heap.SafeRelease();
}

void FD3D12Heap::DeferDelete()
{
	// ProcessDeferredDeletionQueue() performs final Release(), but deletion queue itself only holds a raw pointer, so explicit addref is required.
	AddRef();

	FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(this);
}

void FD3D12Heap::SetHeap(ID3D12Heap* HeapIn, const TCHAR* const InName, bool bInTrack, bool bForceGetGPUAddress)
{
	*Heap.GetInitReference() = HeapIn; 
	bTrack = bInTrack;
	HeapName = InName;
	HeapDesc = Heap->GetDesc();

	SetD3D12ObjectName(HeapIn, InName);

#if ENABLE_RESIDENCY_MANAGEMENT
	const bool bMultiGPUEnabled = (GNumExplicitGPUsForRendering > 1);
	bRequiresResidencyTracking = !bMultiGPUEnabled || IsGPUOnly(HeapDesc.Properties.Type, &HeapDesc.Properties);
#endif // ENABLE_RESIDENCY_MANAGEMENT

	// Create a buffer placed resource on the heap to extract the gpu virtual address
	// if we are tracking all allocations
	FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();	
	if ((bForceGetGPUAddress || Adapter->IsTrackingAllAllocations())
		&& !(HeapDesc.Flags & D3D12_HEAP_FLAG_DENY_BUFFERS)
		&& HeapDesc.Properties.Type == D3D12_HEAP_TYPE_DEFAULT)
	{
		uint64 HeapSize = HeapDesc.SizeInBytes;
		TRefCountPtr<ID3D12Resource> TempResource;
		const D3D12_RESOURCE_DESC BufDesc = CD3DX12_RESOURCE_DESC::Buffer(HeapSize, D3D12_RESOURCE_FLAG_NONE);
		VERIFYD3D12RESULT(Adapter->GetD3DDevice()->CreatePlacedResource(Heap, 0, &BufDesc, D3D12_RESOURCE_STATE_COMMON, nullptr, IID_PPV_ARGS(TempResource.GetInitReference())));
		GPUVirtualAddress = TempResource->GetGPUVirtualAddress();
#if UE_MEMORY_TRACE_ENABLED
		TraceHeapId = MemoryTrace_HeapSpec(TraceParentHeapId, *(FString(InName) + TEXT(" D3D12Heap")));
		// Calling GetResourceAllocationInfo is not trivial, only do it if memory trace is enabled
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel))
		{
			const D3D12_RESOURCE_DESC ResourceDesc = TempResource->GetDesc();
			const D3D12_RESOURCE_ALLOCATION_INFO Info = Adapter->GetD3DDevice()->GetResourceAllocationInfo(0, 1, &ResourceDesc);
			MemoryTrace_Alloc(GPUVirtualAddress, Info.SizeInBytes, Info.Alignment, EMemoryTraceRootHeap::VideoMemory);
			MemoryTrace_MarkAllocAsHeap(GPUVirtualAddress, TraceHeapId);
		}
#endif				
#if TRACK_RESOURCE_ALLOCATIONS
		if (bTrack)
		{
			Adapter->TrackHeapAllocation(this);
		}
#endif
	}
}

void FD3D12Heap::DisallowTrackingResidency()
{
#if ENABLE_RESIDENCY_MANAGEMENT
	checkf(ResidencyHandle == nullptr, TEXT("Can't disallow residency tracking after it has started. Call this function instead of BeginTrackingResidency()."));
	bRequiresResidencyTracking = false;
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

void FD3D12Heap::BeginTrackingResidency(uint64 Size)
{
#if ENABLE_RESIDENCY_MANAGEMENT
	checkf(bRequiresResidencyTracking, TEXT("Residency tracking is not expected for this resource"));
	checkf(!ResidencyHandle, TEXT("Residency tracking is already initialzied for this resource"));
	ResidencyHandle = new FD3D12ResidencyHandle;
	D3DX12Residency::Initialize(*ResidencyHandle, Heap.GetReference(), Size, this);
	D3DX12Residency::BeginTrackingObject(GetParentDevice()->GetResidencyManager(), *ResidencyHandle);
#endif // ENABLE_RESIDENCY_MANAGEMENT
}

/////////////////////////////////////////////////////////////////////
//	FD3D12 Adapter
/////////////////////////////////////////////////////////////////////

static D3D12_RESOURCE_FLAGS GetRayTracingResourceFlags(ED3D12Access InDefaultD3D12Access)
{
#if D3D12_RHI_RAYTRACING
	if (EnumHasAnyFlags(InDefaultD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite))
	{
		return D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS | 
		       D3D12_RESOURCE_FLAG_RAYTRACING_ACCELERATION_STRUCTURE;
	}
#endif // D3D12_RHI_RAYTRACING
	return D3D12_RESOURCE_FLAG_NONE;
}

HRESULT FD3D12Adapter::CreateCommittedResource(
	const FD3D12ResourceDesc& InDesc,
	FRHIGPUMask CreationNode,
	const D3D12_HEAP_PROPERTIES& HeapProps,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* ClearValue,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	bool bVerifyHResult)
{
	if (!ppOutResource)
	{
		return E_POINTER;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(CreateCommittedResource);

	LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

	TRefCountPtr<ID3D12Resource> pResource;
	const bool bRequiresInitialization = (InDesc.Flags & (D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL)) != 0;
	D3D12_HEAP_FLAGS HeapFlags = (bHeapNotZeroedSupported && !bRequiresInitialization) ? FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED : D3D12_HEAP_FLAG_NONE;

	CA_SUPPRESS(6286);
	const bool bShouldTrackResource = (GNumExplicitGPUsForRendering == 1) || IsGPUOnly(HeapProps.Type);

#if ENABLE_RESIDENCY_MANAGEMENT
	if (!GD3D12StartResourceResident && bShouldTrackResource)
	{
		HeapFlags |= D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT;
	}
#endif

	FD3D12ResourceDesc LocalDesc = InDesc;
	if (InDesc.Flags & D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS)
	{
		HeapFlags |= D3D12_HEAP_FLAG_SHARED;

		// Simultaneous access flag is used to detect shared heap requirement but can't be used when allocating buffer resource
		if (InDesc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
		{
			LocalDesc.Flags &= ~D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;
		}
	}

	LocalDesc.Flags |= GetRayTracingResourceFlags(InDefaultD3D12Access);

#if D3D12_WITH_CUSTOM_TEXTURE_LAYOUT
	ApplyCustomTextureLayout(LocalDesc, *this);
#endif

	HRESULT hr =
		Barriers->CreateCommittedResource(
			*this,
			HeapProps, 
			HeapFlags,
			LocalDesc,
			InInitialD3D12Access,
			ClearValue,
			pResource);

	if (SUCCEEDED(hr))
	{
		// Set the output pointer
		*ppOutResource = new FD3D12Resource(
			GetDevice(CreationNode.ToIndex()),
			CreationNode,
			pResource,
			InInitialD3D12Access,
			InResourceStateMode,
			InDefaultD3D12Access,
			LocalDesc,
			nullptr,
			HeapProps.Type);
		(*ppOutResource)->AddRef();

		// Set a default name (can override later).
		SetD3D12ResourceName(*ppOutResource, Name);

		if (bShouldTrackResource)
		{
			(*ppOutResource)->StartTrackingForResidency();
		}
		

		TraceMemoryAllocation(*ppOutResource);
	}
	else	
	{
		UE_LOGF(LogD3D12RHI, Display, "D3D12 CreateCommittedResource failed with params:\n\tHeap Type: %d\n\tHeap Flags: %d\n\tResource Dimension: %d\n\tResource Width: %llu\n\tResource Height: %u\n\tArray Size: %u\n\tMip Levels: %u\n\tFormat: %d\n\tResource Flags: %d",
			HeapProps.Type, HeapFlags, LocalDesc.Dimension, LocalDesc.Width, LocalDesc.Height, LocalDesc.DepthOrArraySize, LocalDesc.MipLevels, LocalDesc.PixelFormat, LocalDesc.Flags);

		if (bVerifyHResult)
		{
			VERIFYD3D12RESULT_EX(hr, RootDevice);
		}
	}

	return hr;
}

HRESULT FD3D12Adapter::CreateReservedResource(
	const FD3D12ResourceDesc& InDesc,
	FRHIGPUMask CreationNode,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* ClearValue,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	bool bVerifyHResult)
{
	if (!ppOutResource)
	{
		return E_POINTER;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(CreateReservedResource);

	LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

	TRefCountPtr<ID3D12Resource> pResource;

	FD3D12ResourceDesc LocalDesc = InDesc;

	checkf(LocalDesc.bReservedResource,
		TEXT("FD3D12ResourceDesc is expected to be initialized as a reserved resource. See FD3D12DynamicRHI::GetResourceDesc()."));

	if (LocalDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE1D
		|| LocalDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE2D
		|| LocalDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D)
	{
		checkf(LocalDesc.Layout == D3D12_TEXTURE_LAYOUT_64KB_UNDEFINED_SWIZZLE,
			TEXT("Reserved textures are expected to have layout %d (64KB_UNDEFINED_SWIZZLE), but have %d. See FD3D12DynamicRHI::GetResourceDesc()."),
			uint32(D3D12_TEXTURE_LAYOUT_64KB_UNDEFINED_SWIZZLE), uint32(LocalDesc.Layout));
	}

	checkf(LocalDesc.Alignment == 0 || LocalDesc.Alignment == 65536,
		TEXT("Reserved resources must use either 64KB alignment or 0 (unspecified/default), but have %d. See FD3D12DynamicRHI::GetResourceDesc()."),
			uint32(LocalDesc.Alignment));

	LocalDesc.Flags |= GetRayTracingResourceFlags(InDefaultD3D12Access);

	HRESULT hr = 
		Barriers->CreateReservedResource(
			*this,
			LocalDesc, 
			InInitialD3D12Access,
			ClearValue,
			pResource);

	if (SUCCEEDED(hr))
	{
		// Set the output pointer
		*ppOutResource = new FD3D12Resource(
			GetDevice(CreationNode.ToIndex()),
			CreationNode,
			pResource,
			InInitialD3D12Access,
			InResourceStateMode,
			InDefaultD3D12Access,
			LocalDesc,
			nullptr /*Heap*/,
			D3D12_HEAP_TYPE_DEFAULT);
		(*ppOutResource)->AddRef();

		// Set a default name (can override later).
		SetD3D12ResourceName(*ppOutResource, Name);
		
		// NOTE: reserved resource residency is not tracked/managed by the engine, so we don't need to call StartTrackingForResidency().
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Display, "D3D12 CreateReservedResource failed with params:\n\tResource Dimension: %d\n\tResource Width: %llu\n\tResource Height: %u\n\tFormat: %d\n\tResource Flags: %d",
			LocalDesc.Dimension, LocalDesc.Width, LocalDesc.Height, LocalDesc.PixelFormat, LocalDesc.Flags);

		if (bVerifyHResult)
		{
			VERIFYD3D12RESULT_EX(hr, RootDevice);
		}
	}

	return hr;
}

HRESULT FD3D12Adapter::CreatePlacedResource(
	const FD3D12ResourceDesc& InDesc,
	FD3D12Heap* BackingHeap,
	uint64 HeapOffset,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* ClearValue,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	bool bVerifyHResult)
{
	if (!ppOutResource)
	{
		return E_POINTER;
	}

	ID3D12Heap* Heap = BackingHeap->GetHeap();

	TRefCountPtr<ID3D12Resource> pResource;
	HRESULT hr =
		Barriers->CreatePlacedResource(
			*this,
			Heap,
			HeapOffset,
			InDesc,
			InInitialD3D12Access,
			ClearValue,
			pResource);

	if (SUCCEEDED(hr))
	{
		FD3D12Device* Device = BackingHeap->GetParentDevice();
		const D3D12_HEAP_DESC HeapDesc = Heap->GetDesc();

		// Set the output pointer
		*ppOutResource = new FD3D12Resource(Device,
			Device->GetVisibilityMask(),
			pResource,
			InInitialD3D12Access,
			InResourceStateMode,
			InDefaultD3D12Access,
			InDesc,
			BackingHeap,
			HeapDesc.Properties.Type);

#if PLATFORM_WINDOWS
		if (IsTrackingAllAllocations() && BackingHeap->GetHeapDesc().Properties.Type == D3D12_HEAP_TYPE_DEFAULT)
		{
			// Manually set the GPU virtual address from the heap gpu virtual address & offset
			if (InDesc.Dimension != D3D12_RESOURCE_DIMENSION_BUFFER)
			{
				check(BackingHeap->GetGPUVirtualAddress() != 0);
				(*ppOutResource)->SetGPUVirtualAddress(BackingHeap->GetGPUVirtualAddress() + HeapOffset);
			}
			else
			{
				check((*ppOutResource)->GetGPUVirtualAddress() != 0);
				check((*ppOutResource)->GetGPUVirtualAddress() == BackingHeap->GetGPUVirtualAddress() + HeapOffset);
			}
		}
#endif		
		// Don't track resources allocated on transient heaps
		if (!BackingHeap->GetIsTransient())
		{
			TraceMemoryAllocation(*ppOutResource);
		}

		// Set a default name (can override later).
		SetD3D12ResourceName(*ppOutResource, Name);

		(*ppOutResource)->AddRef();
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Display, "D3D12 CreatePlacedResource failed with params:\n\tHeap Type: %d\n\tHeap Flags: %d\n\tResource Dimension: %d\n\tResource Width: %llu\n\tResource Height: %u\n\tFormat: %d\n\tResource Flags: %d",
			BackingHeap->GetHeapDesc().Properties.Type, BackingHeap->GetHeapDesc().Flags, InDesc.Dimension, InDesc.Width, InDesc.Height, InDesc.PixelFormat, InDesc.Flags);

		if (bVerifyHResult)
		{
			VERIFYD3D12RESULT_EX(hr, RootDevice);
		}
	}

	return hr;
}

void FD3D12Adapter::TraceMemoryAllocation(FD3D12Resource* Resource)
{
#if UE_MEMORY_TRACE_ENABLED
	// Calling GetResourceAllocationInfo is not cheap so check memory allocation tracking is enabled
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(MemAllocChannel))
	{
		const D3D12_RESOURCE_ALLOCATION_INFO Info = Resource->GetParentDevice()->GetResourceAllocationInfo(Resource->GetDesc());
		D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = Resource->GetGPUVirtualAddress();
		// Textures don't have valid GPUVirtualAddress when IsTrackingAllAllocations() is false, so don't do memory trace in this case.
		if (IsTrackingAllAllocations() && GPUAddress != 0)
		{
			MemoryTrace_Alloc(GPUAddress, Info.SizeInBytes, Info.Alignment, EMemoryTraceRootHeap::VideoMemory);
		}
	}
#endif
}

HRESULT FD3D12Adapter::CreateBuffer(
	D3D12_HEAP_TYPE HeapType,
	FRHIGPUMask CreationNode,
	FRHIGPUMask VisibleNodes,
	uint64 HeapSize,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	D3D12_RESOURCE_FLAGS Flags)
{
	const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(HeapType, CreationNode.GetNative(), VisibleNodes.GetNative());
	const ED3D12Access InitialD3D12Access = DetermineInitialBufferD3D12Access(HeapProps.Type, &HeapProps);
	return CreateBuffer(
		HeapProps,
		CreationNode,
		InitialD3D12Access,
		ED3D12ResourceStateMode::Default,
		ED3D12Access::Unknown,
		HeapSize,
		ppOutResource,
		Name,
		Flags);
}

HRESULT FD3D12Adapter::CreateBuffer(
	D3D12_HEAP_TYPE HeapType,
	FRHIGPUMask CreationNode,
	FRHIGPUMask VisibleNodes,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode ResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	uint64 HeapSize,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	D3D12_RESOURCE_FLAGS Flags)
{
	const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(HeapType, CreationNode.GetNative(), VisibleNodes.GetNative());
	return CreateBuffer(
		HeapProps,
		CreationNode,
		InInitialD3D12Access,
		ResourceStateMode,
		InDefaultD3D12Access,
		HeapSize,
		ppOutResource,
		Name,
		Flags);
}

HRESULT FD3D12Adapter::CreateBuffer(
	const D3D12_HEAP_PROPERTIES& HeapProps,
	FRHIGPUMask CreationNode,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode ResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	uint64 HeapSize,
	FD3D12Resource** ppOutResource,
	const TCHAR* Name,
	D3D12_RESOURCE_FLAGS Flags)
{
	if (!ppOutResource)
	{
		return E_POINTER;
	}

	const D3D12_RESOURCE_DESC BufDesc = CD3DX12_RESOURCE_DESC::Buffer(HeapSize, Flags);
	return CreateCommittedResource(
		BufDesc,
		CreationNode,
		HeapProps,
		InInitialD3D12Access,
		ResourceStateMode,
		InDefaultD3D12Access,
		nullptr,
		ppOutResource,
		Name);
}

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
void FD3D12Adapter::CreateUAVAliasResourceDesc(const FD3D12ResourceLocation& Location)
{
	FD3D12Resource* SourceResource = Location.GetResource();

	const FD3D12ResourceDesc& SourceDesc = SourceResource->GetDesc();
	const EPixelFormat SourceFormat = SourceDesc.PixelFormat;
	const EPixelFormat AliasTextureFormat = SourceDesc.UAVPixelFormat;

	if (ensure(SourceFormat != PF_Unknown) && (SourceFormat != AliasTextureFormat))
	{
		D3D12_RESOURCE_DESC AliasTextureDesc = SourceDesc;
		AliasTextureDesc.Format = (DXGI_FORMAT)GPixelFormats[AliasTextureFormat].PlatformFormat;
		AliasTextureDesc.Width = SourceDesc.Width / GPixelFormats[SourceFormat].BlockSizeX;
		AliasTextureDesc.Height = SourceDesc.Height / GPixelFormats[SourceFormat].BlockSizeY;
		// layout of UAV must match source resource
		AliasTextureDesc.Layout = SourceResource->GetResource()->GetDesc().Layout;

		EnumAddFlags(AliasTextureDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS);

		SourceResource->SetUAVAccessResourceDesc(AliasTextureDesc);
	}
}
#endif

/////////////////////////////////////////////////////////////////////
//	FD3D12 Resource Location
/////////////////////////////////////////////////////////////////////

FD3D12ResourceLocation::FD3D12ResourceLocation(FD3D12Device* Parent)
	: FD3D12DeviceChild(Parent)
	, Allocator(nullptr)
{
	FMemory::Memzero(AllocatorData);
}

FD3D12ResourceLocation::~FD3D12ResourceLocation()
{
	ReleaseResource();
}

void FD3D12ResourceLocation::Clear()
{
	InternalClear<true>();
}

template void FD3D12ResourceLocation::InternalClear<false>();
template void FD3D12ResourceLocation::InternalClear<true>();

template<bool bReleaseResource>
void FD3D12ResourceLocation::InternalClear()
{
	if (bReleaseResource)
	{
		ReleaseResource();
	}

	// Reset members
	Type = ResourceLocationType::eUndefined;
	UnderlyingResource = nullptr;
	MappedBaseAddress = nullptr;
	GPUVirtualAddress = 0;
	Size = 0;
	OffsetFromBaseOfResource = 0;
	FMemory::Memzero(AllocatorData);

	Allocator = nullptr;
	AllocatorType = AT_Unknown;
}

void FD3D12ResourceLocation::TransferOwnership(FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source)
{
	// The bTransient field is not preserved
	check(!Destination.bTransient && !Source.bTransient);

	// Preserve the owner fields
	FD3D12BaseShaderResource* DstOwner = Destination.Owner;
	FD3D12BaseShaderResource* SrcOwner = Source.Owner;

	// Clear out the destination
	Destination.Clear();

	if (Source.GetAllocatorType() == FD3D12ResourceLocation::AT_Pool)
	{
		// Note: pool allocated locations handle the copy themselves to prevent data races when updating their linked list.
		Source.GetPoolAllocator()->TransferOwnership(Source, Destination);
	}
	else
	{
		FMemory::Memmove(&Destination, &Source, sizeof(FD3D12ResourceLocation));
	}

	// update tracked allocation
#if !PLATFORM_WINDOWS && ENABLE_LOW_LEVEL_MEM_TRACKER
	if (Source.GetType() == ResourceLocationType::eSubAllocation)
	{
		FLowLevelMemTracker::Get().OnLowLevelAllocMoved(ELLMTracker::Default, Destination.GetAddressForLLMTracking(), Source.GetAddressForLLMTracking());
	}
#endif

	// Destroy the source but don't invoke any resource destruction
	Source.InternalClear<false>();

	Destination.Owner = DstOwner;
	Source.Owner = SrcOwner;
}

#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
void FD3D12ResourceLocation::Alias(FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source)
{
	// Should not be linked list allocated - otherwise internal linked list data needs to be updated as well in a threadsafe way
	check(Source.GetAllocatorType() != FD3D12ResourceLocation::AT_Pool);

	check(Source.GetResource() != nullptr);
	Destination.Clear();

	FMemory::Memmove(&Destination, &Source, sizeof(FD3D12ResourceLocation));
	Destination.SetType(ResourceLocationType::eAliased);
	Source.SetType(ResourceLocationType::eAliased);

	// Addref the source as another resource location references it
	Source.GetResource()->AddRef();
}
#endif

void FD3D12ResourceLocation::ReferenceNode(FD3D12Device* DestinationDevice, FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source)
{
	check(Source.GetResource() != nullptr);
	Destination.Clear();

	FMemory::Memmove(&Destination, &Source, sizeof(FD3D12ResourceLocation));
	Destination.SetType(ResourceLocationType::eNodeReference);

	Destination.Parent = DestinationDevice;

	// Addref the source as another resource location references it
	Source.GetResource()->AddRef();

	if (Source.GetAllocatorType() == FD3D12ResourceLocation::AT_Pool)
	{
		Source.GetPoolAllocatorPrivateData().PoolData.AddAlias(
			&Destination.GetPoolAllocatorPrivateData().PoolData);
	}
}

void FD3D12ResourceLocation::ReleaseResource()
{
#if TRACK_RESOURCE_ALLOCATIONS
	if (IsTransient())
	{
		FD3D12Adapter* Adapter = GetParentDevice()->GetParentAdapter();
		if (Adapter->IsTrackingAllAllocations())
		{
			bool bDefragFree = false;
			Adapter->ReleaseTrackedAllocationData(this, bDefragFree);
		}
	}
#endif

	switch (Type)
	{
	case ResourceLocationType::eStandAlone:
	{
		bool bIncrement = false;
		UpdateStandAloneStats(bIncrement);

		// Multi-GPU support : When the resource enters this point for the first time the number of references should be the same as number of GPUs.
		// Shouldn't queue deferred deletion until all references are released as this could cause issues at the end of the pipe.
		// Instead reduce number of references until nothing else holds the resource.
		if (GNumExplicitGPUsForRendering > 1 && UnderlyingResource->GetRefCount() > 1)
		{
			check(UnderlyingResource->GetRefCount() <= GNumExplicitGPUsForRendering)
			UnderlyingResource->Release();
		}
		else
		{
			check(UnderlyingResource->GetRefCount() == 1);

			if (UnderlyingResource->ShouldDeferDelete())
			{
				UnderlyingResource->DeferDelete();
			}
			else
			{
				UnderlyingResource->Release();
			}
		}
		break;
	}
	case ResourceLocationType::eSubAllocation:
	{
		check(Allocator != nullptr);
		if (AllocatorType == AT_Pool)
		{
			// Unlink any aliases -- the contents of aliases are cleaned up separately elsewhere via iteration over
			// the FD3D12LinkedAdapterObject.
			for (FRHIPoolAllocationData* Alias = GetPoolAllocatorPrivateData().PoolData.GetFirstAlias();
				 Alias;
				 Alias = GetPoolAllocatorPrivateData().PoolData.GetFirstAlias())
			{
				Alias->RemoveAlias();
			}

			PoolAllocator->DeallocateResource(*this);
		}
		else
		{
			Allocator->Deallocate(*this);
		}
		break;
	}
	case ResourceLocationType::eNodeReference:
	case ResourceLocationType::eAliased:
	{
		if (GetAllocatorType() == FD3D12ResourceLocation::AT_Pool)
		{
			GetPoolAllocatorPrivateData().PoolData.RemoveAlias();
		}

		if (UnderlyingResource->ShouldDeferDelete() && UnderlyingResource->GetRefCount() == 1)
		{
			UnderlyingResource->DeferDelete();
		}
		else
		{
			UnderlyingResource->Release();
		}
		break;
	}
	case ResourceLocationType::eHeapAliased:
	{
		check(UnderlyingResource->GetRefCount() == 1);
		if (UnderlyingResource->ShouldDeferDelete())
		{
			UnderlyingResource->DeferDelete();
		}
		else
		{
			UnderlyingResource->Release();
		}
		break;
	}
	case ResourceLocationType::eFastAllocation:
	case ResourceLocationType::eUndefined:
	default:
		// Fast allocations are volatile by default so no work needs to be done.
		break;
	}
}

void FD3D12ResourceLocation::UpdateStandAloneStats(bool bIncrement)
{
	if (UnderlyingResource->GetHeapType() == D3D12_HEAP_TYPE_DEFAULT)
	{
		FD3D12ResourceDesc Desc = UnderlyingResource->GetDesc();
		bool bIsBuffer = (Desc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER);
		bool bIsRenderTarget = (Desc.Flags & D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET || Desc.Flags & D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);
		bool bIsUAV = (Desc.Flags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS) > 0;
		
		if (bIsBuffer)
		{
			// Simultaneous access flag is used to detect shared heap requirement but can't be used for buffers on device calls
			Desc.Flags &= ~D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;
		}

		// Get the desired size and allocated size for stand alone resources - allocated are very slow anyway
		D3D12_RESOURCE_ALLOCATION_INFO Info = UnderlyingResource->GetParentDevice()->GetResourceAllocationInfoUncached(Desc);

		int64 SizeInBytes = bIncrement ? Info.SizeInBytes : -(int64)Info.SizeInBytes;
		int32 Count = bIncrement ? 1 : -1;

		if (bIsBuffer)
		{
			if (bIsUAV)
			{
				INC_DWORD_STAT_BY(STAT_D3D12UAVBufferStandAloneCount, Count);
				INC_MEMORY_STAT_BY(STAT_D3D12UAVBufferStandAloneAllocated, SizeInBytes);
			}
			else
			{
				INC_DWORD_STAT_BY(STAT_D3D12BufferStandAloneCount, Count);
				INC_MEMORY_STAT_BY(STAT_D3D12BufferStandAloneAllocated, SizeInBytes);
			}
		}
		else
		{
			if (bIsRenderTarget)
			{
				INC_DWORD_STAT_BY(STAT_D3D12RenderTargetStandAloneCount, Count);
				INC_MEMORY_STAT_BY(STAT_D3D12RenderTargetStandAloneAllocated, SizeInBytes);
			}
			else if (bIsUAV)
			{
				INC_DWORD_STAT_BY(STAT_D3D12UAVTextureStandAloneCount, Count);
				INC_MEMORY_STAT_BY(STAT_D3D12UAVTextureStandAloneAllocated, SizeInBytes);
			}
			else
			{
				INC_DWORD_STAT_BY(STAT_D3D12TextureStandAloneCount, Count);
				INC_MEMORY_STAT_BY(STAT_D3D12TextureStandAloneAllocated, SizeInBytes);
			}
		}

		// Track all committed resource allocations
		if (bIncrement)
		{
			bool bCollectCallstack = true;
			UnderlyingResource->GetParentDevice()->GetParentAdapter()->TrackAllocationData(this, Info.SizeInBytes, bCollectCallstack);
		}
		else
		{
			bool bDefragFree = false;
			UnderlyingResource->GetParentDevice()->GetParentAdapter()->ReleaseTrackedAllocationData(this, bDefragFree);
		}
	}
}

void FD3D12ResourceLocation::SetResource(FD3D12Resource* Value)
{
	check(UnderlyingResource == nullptr);

	GPUVirtualAddress = Value->GetGPUVirtualAddress();

	UnderlyingResource = Value;
}


void FD3D12ResourceLocation::AsStandAlone(FD3D12Resource* Resource, uint64 InSize, bool bInIsTransient, const D3D12_HEAP_PROPERTIES* CustomHeapProperties)
{
	SetType(FD3D12ResourceLocation::ResourceLocationType::eStandAlone);
	SetResource(Resource);
	SetSize(InSize);

	if (IsCPUAccessible(Resource->GetHeapType(), CustomHeapProperties))
	{
		D3D12_RANGE range = { 0, IsCPUWritable(Resource->GetHeapType()) ? 0 : InSize };
		SetMappedBaseAddress(Resource->Map(&range));
	}
	SetGPUVirtualAddress(Resource->GetGPUVirtualAddress());
	SetTransient(bInIsTransient);

	bool bIncrement = true;
	UpdateStandAloneStats(bIncrement);
}


bool FD3D12ResourceLocation::OnAllocationMoved(
	FD3D12ContextArray const& Contexts,
	FRHIPoolAllocationData* InNewData,
	ED3D12Access& OutD3D12Access)
{
	// Assume linked list allocated for now - only defragging allocator
	FRHIPoolAllocationData& AllocationData = GetPoolAllocatorPrivateData().PoolData;
	check(InNewData == &AllocationData);
	check(AllocationData.IsAllocated()); // Should be allocated
	check(AllocationData.GetSize() == Size); // Same size
	check(Type == ResourceLocationType::eSubAllocation); // Suballocated
	check(GetMappedBaseAddress() == nullptr); // And VRAM only
	
	// Get the resource and the actual new allocator
	FD3D12Resource* CurrentResource = GetResource();
	FD3D12PoolAllocator* NewAllocator = GetPoolAllocator();

	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CurrentResource->GetName(), FName("FD3D12ResourceLocation::OnAllocationMoved"), NAME_None);

	// Textures don't have valid GPUVirtualAddress when IsTrackingAllAllocations() is false, so don't do memory trace in this case.
	const bool bTrackingAllAllocations = GetParentDevice()->GetParentAdapter()->IsTrackingAllAllocations();
	const bool bMemoryTrace = bTrackingAllAllocations || GPUVirtualAddress != 0;

	// If sub allocated and not placed only update the internal data
	if (NewAllocator->GetAllocationStrategy() == EResourceAllocationStrategy::kManualSubAllocation)
	{
		check(!CurrentResource->IsPlacedResource());
		D3D12_GPU_VIRTUAL_ADDRESS OldGPUAddress = GPUVirtualAddress;
		OffsetFromBaseOfResource = AllocationData.GetOffset();
		UnderlyingResource = NewAllocator->GetBackingResource(*this);
		GPUVirtualAddress = UnderlyingResource->GetGPUVirtualAddress() + OffsetFromBaseOfResource;

#if UE_MEMORY_TRACE_ENABLED
		if (bMemoryTrace)
		{
			MemoryTrace_ReallocFree(OldGPUAddress, EMemoryTraceRootHeap::VideoMemory);
			MemoryTrace_ReallocAlloc(GPUVirtualAddress, AllocationData.GetSize(), AllocationData.GetAlignment(), EMemoryTraceRootHeap::VideoMemory);
		}
#endif
	}
	else
	{
		check(CurrentResource->IsPlacedResource());
		check(OffsetFromBaseOfResource == 0);

#if UE_MEMORY_TRACE_ENABLED
		if (bMemoryTrace)
		{
			// CreatePlacedResource function below calls MemoryTrace_Alloc to track new memory, so call MemoryTrace_Free to match (instead of calling MemoryTrace_ReallocFree/MemoryTrace_ReallocAlloc).
			MemoryTrace_Free(GPUVirtualAddress, EMemoryTraceRootHeap::VideoMemory);
		}
#endif
		// recreate the placed resource (ownership of current resource is already handled during the internal move)
		FD3D12HeapAndOffset HeapAndOffset = NewAllocator->GetBackingHeapAndAllocationOffsetInBytes(*this);

		OutD3D12Access = ED3D12Access::Unknown;
		ED3D12ResourceStateMode ResourceStateMode;
		if (CurrentResource->RequiresResourceStateTracking())
		{
			// The newly created placed resource will be copied into by the defragger. Create it in the COPY_DEST to avoid an additional transition.
			// Standard resource state tracking will handle transitioning the resource out of this state as required.
			OutD3D12Access = ED3D12Access::CopyDest;
			ResourceStateMode = ED3D12ResourceStateMode::MultiState;
		}
		else
		{
			OutD3D12Access = CurrentResource->GetDefaultAccess();
			ResourceStateMode = ED3D12ResourceStateMode::Default;
		}

		// TODO: fix retrieval of ClearValue from owner (currently not a problem because not defragging RT/DS resource yet)
		D3D12_CLEAR_VALUE* ClearValue = nullptr;

		FName Name = CurrentResource->GetName();

		FD3D12Resource* NewResource = nullptr;
		FD3D12Adapter* const Adapter = CurrentResource->GetParentDevice()->GetParentAdapter();
		VERIFYD3D12RESULT(
			Adapter->CreatePlacedResource(
				CurrentResource->GetDesc(),
				HeapAndOffset.Heap,
				HeapAndOffset.Offset,
				OutD3D12Access,
				ResourceStateMode,
				OutD3D12Access,
				ClearValue,
				&NewResource,
				*Name.ToString()));

		UnderlyingResource = NewResource;
		GPUVirtualAddress = UnderlyingResource->GetGPUVirtualAddress() + OffsetFromBaseOfResource;
	}

	// Refresh aliases
	for (FRHIPoolAllocationData* OtherAlias = AllocationData.GetFirstAlias(); OtherAlias; OtherAlias = OtherAlias->GetNext())
	{
		FD3D12ResourceLocation* OtherResourceLocation = (FD3D12ResourceLocation*)OtherAlias->GetOwner();

		OtherResourceLocation->OffsetFromBaseOfResource = OffsetFromBaseOfResource;
		OtherResourceLocation->UnderlyingResource = UnderlyingResource;
		OtherResourceLocation->GPUVirtualAddress = GPUVirtualAddress;
	}

	// Notify all the dependent resources about the change
	Owner->ResourceRenamed(Contexts);

	if (OutD3D12Access == ED3D12Access::Unknown)
	{
		FD3D12Texture* CurrentResourceRHITexture = nullptr;

		if (CurrentResource->GetDesc().Dimension == D3D12_RESOURCE_DIMENSION_BUFFER)
		{
			FD3D12Buffer* CurrentResourceRHIBuffer = (FD3D12Buffer*)Owner;
			OutD3D12Access = ConvertToD3D12Access(CurrentResourceRHIBuffer->GetTrackedAccess_Unsafe());
		}
		else
		{
			CurrentResourceRHITexture = (FD3D12Texture*)Owner;
			OutD3D12Access = ConvertToD3D12Access(CurrentResourceRHITexture->GetTrackedAccess_Unsafe());
		}
	}

	check(IsValidAccess(OutD3D12Access));
	check(OutD3D12Access != ED3D12Access::Unknown);

	return true;
}


void FD3D12ResourceLocation::UnlockPoolData()
{
	if (AllocatorType == AT_Pool)
	{
		GetPoolAllocatorPrivateData().PoolData.Unlock();
	}
}

bool FD3D12ResourceLocation::IsStandaloneOrPooledPlacedResource() const
{
	bool bStandalone = Type == ResourceLocationType::eStandAlone;
	bool bPoolPlacedResource = (!bStandalone && AllocatorType == AT_Pool) ? PoolAllocator->GetAllocationStrategy() == EResourceAllocationStrategy::kPlacedResource : false;
	return bStandalone || bPoolPlacedResource;
}

void FD3D12DynamicRHI::RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
{
	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FD3D12DynamicRHI::RHIReplaceResources"),
		[ReplaceInfos = MoveTemp(ReplaceInfos)](FD3D12ContextArray const& Contexts)
		{
			for (FRHIResourceReplaceInfo const& Info : ReplaceInfos)
			{
				switch (Info.GetType())
				{
				default:
					checkNoEntry();
					break;

				case FRHIResourceReplaceInfo::EType::Buffer:
					{
						FD3D12Buffer* Dst = ResourceCast(Info.GetBuffer().Dst);
						FD3D12Buffer* Src = ResourceCast(Info.GetBuffer().Src);

						if (Src)
						{
							// The source buffer should not have any associated views.
							check(!Src->HasLinkedViews());
							Dst->TakeOwnership(*Src);
						}
						else
						{
							Dst->ReleaseOwnership();
						}

						Dst->ResourceRenamed(Contexts);
					}
					break;

#if D3D12_RHI_RAYTRACING
				case FRHIResourceReplaceInfo::EType::RTGeometry:
					{
						FD3D12RayTracingGeometry* Src = ResourceCast(Info.GetRTGeometry().Src);
						FD3D12RayTracingGeometry* Dst = ResourceCast(Info.GetRTGeometry().Dst);

						if (Src)
						{
							Dst->Swap(*Src);
						}
						else
						{
							Dst->ReleaseUnderlyingResource();
						}
					}
					break;
#endif // D3D12_RHI_RAYTRACING
				}
			}
		}
	);
}

void FD3D12DynamicRHI::RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr)
{
	RHICmdList.EnqueueLambda([AddressesPtr, Resources, GPUIndex](FRHICommandListBase&)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RHIWriteResourceGPUAddresses);

			for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex)
			{
				const FRHIResource* Resource = Resources[ResourceIndex];

				if (Resource->GetType() == RRT_Buffer)
				{
					const FD3D12Buffer* Buffer = ResourceCast(static_cast<const FRHIBuffer*>(Resource), GPUIndex);
					AddressesPtr[ResourceIndex] = Buffer->GetGPUVirtualAddress();
				}
				else if (Resource->GetType() == RRT_Texture)
				{
					unimplemented();
				}
#if D3D12_RHI_RAYTRACING
				else if (Resource->GetType() == RRT_RayTracingGeometry)
				{
					const FD3D12RayTracingGeometry* RayTracingGeometry = ResourceCast(static_cast<const FRHIRayTracingGeometry*>(Resource));
					AddressesPtr[ResourceIndex] = RayTracingGeometry->GetGPUVirtualAddress(GPUIndex);
				}
#endif
				else
				{
					checkNoEntry();
				}
			}
		});
}
