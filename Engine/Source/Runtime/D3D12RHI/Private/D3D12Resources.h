// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Resources.h: D3D resource RHI definitions.
	=============================================================================*/

#pragma once

#include "BoundShaderStateCache.h"
#include "D3D12DirectCommandListManager.h"
#include "D3D12NvidiaExtensions.h"
#include "D3D12Residency.h"
#include "D3D12ShaderResources.h"
#include "D3D12State.h"
#include "D3D12Util.h"
#include "RHIPoolAllocator.h"
#include "Templates/UniquePtr.h"

// Forward Decls
class FD3D12CommandList;
class FD3D12Resource;
class FD3D12StateCache;
class FD3D12CommandListManager;
class FD3D12CommandContext;
class FD3D12PoolAllocator;
struct FD3D12ComputePipelineState;
struct FD3D12WorkGraphPipelineState;
struct FD3D12GraphicsPipelineState;
struct FD3D12ResourceDesc;

class FD3D12SyncPoint;
using FD3D12SyncPointRef = TRefCountPtr<FD3D12SyncPoint>;

#if D3D12_RHI_RAYTRACING
class FD3D12RayTracingGeometry;
class FD3D12RayTracingScene;
class FD3D12RayTracingPipelineState;
class FD3D12RayTracingShaderBindingTable;
class FD3D12RayTracingShader;
#endif // D3D12_RHI_RAYTRACING

#ifndef D3D12_WITH_CUSTOM_TEXTURE_LAYOUT
#define D3D12_WITH_CUSTOM_TEXTURE_LAYOUT 0
#endif

#if D3D12_WITH_CUSTOM_TEXTURE_LAYOUT
extern void ApplyCustomTextureLayout(FD3D12ResourceDesc& TextureLayout, FD3D12Adapter& Adapter);
#endif

enum class ED3D12ResourceStateMode
{
	Default,					//< Decide if tracking is required based on flags
	SingleState,				//< Force disable state tracking of resource - resource will always be in the initial resource state
	MultiState,					//< Force enable state tracking of resource
};

class FD3D12PendingResourceBarrier
{
public:
	FD3D12Resource*       Resource;
	D3D12_RESOURCE_STATES State;
	uint32                SubResource;

	FD3D12PendingResourceBarrier(FD3D12Resource* Resource, D3D12_RESOURCE_STATES State, uint32 SubResource)
		: Resource(Resource)
		, State(State)
		, SubResource(SubResource)
	{}
};

class FD3D12Heap : public FRefCountedObject, public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
public:
	FD3D12Heap(FD3D12Device* Parent, FRHIGPUMask VisibleNodes, HeapId InTraceParentHeapId = EMemoryTraceRootHeap::VideoMemory);
	~FD3D12Heap();

	inline ID3D12Heap* GetHeap() const { return Heap.GetReference(); }
	void SetHeap(ID3D12Heap* HeapIn, const TCHAR* const InName, bool bTrack = true, bool bForceGetGPUAddress = false);

	void BeginTrackingResidency(uint64 Size);
	void DisallowTrackingResidency(); // Part of workaround for UE-174791 and UE-202367

	void DeferDelete();

	inline FName GetName() const { return HeapName; }
	inline D3D12_HEAP_DESC GetHeapDesc() const { return HeapDesc; }
	inline TConstArrayView<FD3D12ResidencyHandle*> GetResidencyHandles()
	{ 
#if ENABLE_RESIDENCY_MANAGEMENT
		if (bRequiresResidencyTracking)
		{
			checkf(ResidencyHandle, TEXT("Resource requires residency tracking, but BeginTrackingResidency() was not called."));
			return MakeArrayView(&ResidencyHandle, 1); 
		}
		else
#endif // ENABLE_RESIDENCY_MANAGEMENT
		{
			return {};
		}		
	}
	inline D3D12_GPU_VIRTUAL_ADDRESS GetGPUVirtualAddress() const { return GPUVirtualAddress; }
	inline void SetIsTransient(bool bInIsTransient) { bIsTransient = bInIsTransient; }
	inline bool GetIsTransient() const { return bIsTransient; }

private:

	TRefCountPtr<ID3D12Heap> Heap;
	FName HeapName;
	bool bTrack = true;
	D3D12_HEAP_DESC HeapDesc;
	D3D12_GPU_VIRTUAL_ADDRESS GPUVirtualAddress = 0;
	FD3D12ResidencyHandle* ResidencyHandle = nullptr; // Residency handle owned by this object
	HeapId TraceHeapId;
	HeapId TraceParentHeapId;
	bool bIsTransient = false; // Whether this is a transient heap
	bool bRequiresResidencyTracking = bool(ENABLE_RESIDENCY_MANAGEMENT);
};


// This can be used to guarantee that a struct's padding is zero, which is necessary for hashing in some cases
// NOTE: classes with virtual methods are not supported. The struct must be the first declared base class
// to ensure the correct ordering of the ctor calls
template <typename T>
struct TZeroedStruct
{
	TZeroedStruct<T>()
	{
		FMemory::Memzero(this, sizeof(T));
	}
};

struct FD3D12ResourceDesc : public TZeroedStruct<FD3D12ResourceDesc>, D3D12_RESOURCE_DESC
{
	FD3D12ResourceDesc() :
		TZeroedStruct<FD3D12ResourceDesc>()
		{
		}
	FD3D12ResourceDesc(const CD3DX12_RESOURCE_DESC& Other)
		: TZeroedStruct<FD3D12ResourceDesc>()
		, D3D12_RESOURCE_DESC(Other)
	{
	}
	
	// TODO: use this type everywhere and disallow implicit conversion
	/*explicit*/ FD3D12ResourceDesc(const D3D12_RESOURCE_DESC& Other)
		: TZeroedStruct<FD3D12ResourceDesc>()
		, D3D12_RESOURCE_DESC(Other)
	{
	}

	EPixelFormat PixelFormat{ PF_Unknown };

	// PixelFormat for the Resource that aliases our current resource.
	EPixelFormat UAVPixelFormat{ PF_Unknown };

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	bool bRequires64BitAtomicSupport : 1 = false;
#endif

	bool bReservedResource : 1 = false;

	bool bBackBuffer : 1 = false;

	// External resources are owned by another application or middleware, not the Engine
	bool bExternal : 1 = false;

	// If we support the new format list casting, use the newer APIs; otherwise, fall back to our UAV Aliasing approach.
#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	inline bool NeedsUAVAliasWorkarounds() const { return false; }
	inline bool SupportsUncompressedUAV() const { return UAVPixelFormat != PF_Unknown; }

	TArray<DXGI_FORMAT, TInlineAllocator<4>> GetCastableFormats() const;
#else
	inline bool NeedsUAVAliasWorkarounds() const { return UAVPixelFormat != PF_Unknown; }
	inline bool SupportsUncompressedUAV() const { return false; }
#endif
};

class FD3D12Resource : public UE::Private::FQueryableRefCountedObject, public FD3D12DeviceChild, public FD3D12MultiNodeGPUObject
{
	friend class FD3D12Buffer;
private:
	TRefCountPtr<ID3D12Resource> Resource;
	TRefCountPtr<FD3D12Heap> Heap;

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	// Since certain formats cannot be aliased in D3D12, we create a placed UAV
	// that aliases them. This resource description is used to create that UAV.
	TUniquePtr<D3D12_RESOURCE_DESC> UAVAccessResourceDesc;
#endif

	FD3D12ResidencyHandle* ResidencyHandle = nullptr; // Residency handle owned by this object

	D3D12_GPU_VIRTUAL_ADDRESS GPUVirtualAddress{};
	void* ResourceBaseAddress{};

#if NV_AFTERMATH
	UE::RHICore::Nvidia::Aftermath::D3D12::FResource AftermathHandle{};
#endif
	const FD3D12ResourceDesc Desc;
	ED3D12Access InitialD3D12Access = ED3D12Access::Unknown;
	ED3D12Access DefaultD3D12Access = ED3D12Access::Unknown;
#ifdef PLATFORM_SUPPORTS_RESOURCE_COMPRESSION
	D3D12_RESOURCE_STATES CompressedState{ D3D12_RESOURCE_STATE_CORRUPT };
#endif

	D3D12_HEAP_TYPE HeapType;
	FName DebugName;

	int32 NumMapCalls = 0;
	uint16 SubresourceCount{};
	uint8 PlaneCount;
	bool bRequiresResourceStateTracking : 1;	//1st bit 
	bool bRequiresResidencyTracking : 1;		//2nd bit
	bool bDepthStencil : 1;						//3rd bit
	bool bDeferDelete : 1;						//4th bit

#if UE_BUILD_DEBUG
	static int64 TotalResourceCount;
	static int64 NoStateTrackingResourceCount;
#endif

	struct FD3D12ReservedResourceData
	{
		TArray<TRefCountPtr<FD3D12Heap>> BackingHeaps;

		// Flattened array of residency handles owned by backing heaps, used to support batched GetResidencyHandles()
		TArray<FD3D12ResidencyHandle*> ResidencyHandles;
		TArray<int32> NumResidencyHandlesPerHeap;

		// Tiles currently assigned to the resource
		uint32 NumCommittedTiles = 0;

		// Available tiles at the end of the last backing heap
		uint32 NumSlackTiles = 0;
	};
	TUniquePtr<FD3D12ReservedResourceData> ReservedResourceData;

public:
	FD3D12Resource() = delete;
	explicit FD3D12Resource(FD3D12Device* ParentDevice,
		FRHIGPUMask VisibleNodes,
		ID3D12Resource* InResource,
		ED3D12Access InInitialD3D12Access,
		FD3D12ResourceDesc const& InDesc,
		FD3D12Heap* InHeap = nullptr,
		D3D12_HEAP_TYPE InHeapType = D3D12_HEAP_TYPE_DEFAULT);

	explicit FD3D12Resource(FD3D12Device* ParentDevice,
		FRHIGPUMask VisibleNodes,
		ID3D12Resource* InResource,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		FD3D12ResourceDesc const& InDesc,
		FD3D12Heap* InHeap,
		D3D12_HEAP_TYPE InHeapType);

	virtual ~FD3D12Resource();

	operator ID3D12Resource&() { return *Resource; }

	ID3D12Resource* GetResource() const { return Resource.GetReference(); }

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	const D3D12_RESOURCE_DESC* GetUAVAccessResourceDesc() const
	{
		return UAVAccessResourceDesc.Get();
	}

	void SetUAVAccessResourceDesc(const D3D12_RESOURCE_DESC& InUAVAccessResourceDesc)
	{
		UAVAccessResourceDesc = MakeUnique<D3D12_RESOURCE_DESC>(InUAVAccessResourceDesc);
	}
#endif

	inline void* Map(const D3D12_RANGE* ReadRange = nullptr)
	{
		if (NumMapCalls == 0)
		{
			check(Resource);
			check(ResourceBaseAddress == nullptr);
			VERIFYD3D12RESULT(Resource->Map(0, ReadRange, &ResourceBaseAddress));
		}
		else
		{
			check(ResourceBaseAddress);
		}
		++NumMapCalls;

		return ResourceBaseAddress;
	}

	inline void Unmap()
	{
		check(Resource);
		check(ResourceBaseAddress);
		check(NumMapCalls > 0);

		--NumMapCalls;
		if (NumMapCalls == 0)
		{
			Resource->Unmap(0, nullptr);
			ResourceBaseAddress = nullptr;
		}
	}

	ID3D12Pageable* GetPageable();
	const FD3D12ResourceDesc& GetDesc() const { return Desc; }
	D3D12_HEAP_TYPE GetHeapType() const { return HeapType; }
	D3D12_GPU_VIRTUAL_ADDRESS GetGPUVirtualAddress() const { return GPUVirtualAddress; }
	inline void SetGPUVirtualAddress(D3D12_GPU_VIRTUAL_ADDRESS Value) { GPUVirtualAddress = Value; }
	void* GetResourceBaseAddress() const { check(ResourceBaseAddress); return ResourceBaseAddress; }
	uint16 GetMipLevels() const { return Desc.MipLevels; }
	uint16 GetArraySize() const { return (Desc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D) ? 1 : Desc.DepthOrArraySize; }
	uint8 GetPlaneCount() const { return PlaneCount; }
	uint16 GetSubresourceCount() const { return SubresourceCount; }
	ED3D12Access GetInitialAccess() const { return InitialD3D12Access; }
	ED3D12Access GetDefaultAccess() const { return DefaultD3D12Access; }
#ifdef PLATFORM_SUPPORTS_RESOURCE_COMPRESSION
	D3D12_RESOURCE_STATES GetCompressedState() const { return CompressedState; }
	void SetCompressedState(D3D12_RESOURCE_STATES State) { CompressedState = State; }
#endif
	bool RequiresResourceStateTracking() const { return bRequiresResourceStateTracking; }

	inline bool IsBackBuffer() const { return Desc.bBackBuffer; }

	void SetName(const TCHAR* Name)
	{
		// Check name before setting it.  Saves FName lookup and driver call.  Names are frequently the same for pooled buffers
		// that end up getting reused for the same purpose every frame (2/3 of calls to this function on a given frame).
		if (DebugName != Name)
		{
			DebugName = FName(Name);
			SetD3D12ObjectName(GetResource(), Name);
		}
	}

	FName GetName() const
	{
		return DebugName;
	}

	void DoNotDeferDelete()
	{
		bDeferDelete = false;
	}

	inline bool ShouldDeferDelete() const { return bDeferDelete; }
	void DeferDelete();

	inline bool IsReservedResource() const { return ReservedResourceData.IsValid(); }
	inline bool IsPlacedResource() const { return Heap.GetReference() != nullptr; }
	inline FD3D12Heap* GetHeap() const { return Heap; };
	inline bool IsDepthStencilResource() const { return bDepthStencil; }

	inline uint64 GetCommittedReservedResourceSize() const
	{
		check(IsReservedResource());
		return ReservedResourceData->NumCommittedTiles * (uint64)GRHIGlobals.ReservedResources.TileSizeInBytes;
	}

	inline bool NeedsDeferredResidencyUpdate() const { return IsReservedResource(); }

	void StartTrackingForResidency();

	bool IsResident() const
	{
#if ENABLE_RESIDENCY_MANAGEMENT

		if (NeedsDeferredResidencyUpdate())
		{
			// We don't know the state because the set of residency handles is only known on the 
			// RHI Submission Thread and may change throughout the frame.
			return true;
		}

		TConstArrayView<FD3D12ResidencyHandle*> ResidencyHandles = GetResidencyHandles();
		if (ResidencyHandles.IsEmpty())
		{
			// No residency tracking for this resource
			return true;
		}

		// Treat resource as resident if at least one backing heap is resident.
		// Technically we should return a partial residency status.
		for (FD3D12ResidencyHandle* Handle : ResidencyHandles)
		{
			if (Handle->ResidencyStatus == FD3D12ResidencyHandle::RESIDENCY_STATUS::RESIDENT)
			{
				return true;
			}
		}
		return false;
#else // ENABLE_RESIDENCY_MANAGEMENT
		return true;
#endif // ENABLE_RESIDENCY_MANAGEMENT
	}

	TConstArrayView<FD3D12ResidencyHandle*> GetResidencyHandles() const
	{
#if ENABLE_RESIDENCY_MANAGEMENT
		if (!bRequiresResidencyTracking)
		{
			return {};
		}
		else if (IsPlacedResource())
		{
			return Heap->GetResidencyHandles();
		}
		else if (IsReservedResource())
		{
			return ReservedResourceData->ResidencyHandles;
		}
		else
		{
			checkf(ResidencyHandle, TEXT("Resource requires residency tracking, but StartTrackingForResidency() was not called."));
			return MakeArrayView(&ResidencyHandle, 1);
		}
#else // ENABLE_RESIDENCY_MANAGEMENT
		return {};
#endif // ENABLE_RESIDENCY_MANAGEMENT
	}

	template<typename TLambda>
	FORCEINLINE_DEBUGGABLE void GetBackingHeapsGpuAddresses(TLambda&& Lambda) const
	{
		if (IsReservedResource())
		{
			for (TRefCountPtr<FD3D12Heap>& BackingHeap: ReservedResourceData->BackingHeaps)
			{
				Lambda(BackingHeap->GetGPUVirtualAddress(), BackingHeap->GetGPUVirtualAddress(), BackingHeap->GetHeapDesc().SizeInBytes);
			}
		}
	}

	struct FD3D12ResourceTypeHelper
	{
		FD3D12ResourceTypeHelper(const FD3D12ResourceDesc& Desc, D3D12_HEAP_TYPE HeapType) :
			bSRV(!EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE)),
			bDSV(EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL)),
			bRTV(EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET)),
			bUAV(EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS)),
			bWritable(bDSV || bRTV || bUAV),
			bSRVOnly(bSRV && !bWritable),
			bBuffer(Desc.Dimension == D3D12_RESOURCE_DIMENSION_BUFFER),
			bReadBackResource(HeapType == D3D12_HEAP_TYPE_READBACK)
		{}

		ED3D12Access GetOptimalInitialD3D12Access(ED3D12Access InD3D12Access, bool bAccurateWriteableStates) const
		{
			// Ignore the requested resource state for non tracked resource because RHI will assume it's always in default resource 
			// state then when a transition is required (will transition via scoped push/pop to requested state)
			if (!bSRVOnly && InD3D12Access != ED3D12Access::Unknown)
			{
				return InD3D12Access;
			}

			if (bSRVOnly)
			{
				return ED3D12Access::SRVGraphics;
			}
			else if (bBuffer && !bUAV)
			{
				return (bReadBackResource) ? ED3D12Access::CopyDest : ED3D12Access::GenericRead;
			}
			else if (bWritable && bAccurateWriteableStates)
			{
				if (bDSV)
				{
					return ED3D12Access::DSVWrite;
				}
				else if (bRTV)
				{
					return ED3D12Access::RTV;
				}
				else if (bUAV)
				{
					check(InD3D12Access != ED3D12Access::Unknown);
					return ED3D12Access::UAVMask;
				}
			}

			return ED3D12Access::Common;
		}

		const uint32 bSRV : 1;
		const uint32 bDSV : 1;
		const uint32 bRTV : 1;
		const uint32 bUAV : 1;
		const uint32 bWritable : 1;
		const uint32 bSRVOnly : 1;
		const uint32 bBuffer : 1;
		const uint32 bReadBackResource : 1;
	};

	// Note: RequiredCommitSizeInBytes is clamped to the maximum size of the resource.
	// Use UINT64_MAX to commit the entire resource.
	void CommitReservedResource(ID3D12CommandQueue* D3DCommandQueue, uint64 RequiredCommitSizeInBytes);

private:
	void InitializeResourceState(
		const D3D12_HEAP_PROPERTIES* InHeapProps,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access)
	{
		check(IsValidAccess(InDefaultD3D12Access));
		check(IsValidAccess(InInitialD3D12Access));

		SubresourceCount = GetMipLevels() * GetArraySize() * GetPlaneCount();

		if (InResourceStateMode == ED3D12ResourceStateMode::SingleState)
		{
			// make sure a valid default state is set
			check(InInitialD3D12Access != ED3D12Access::Unknown);
			check(InDefaultD3D12Access != ED3D12Access::Unknown);

#if UE_BUILD_DEBUG
			FPlatformAtomics::InterlockedIncrement(&NoStateTrackingResourceCount);
#endif
			InitialD3D12Access = InInitialD3D12Access;
			DefaultD3D12Access = InDefaultD3D12Access;
			bRequiresResourceStateTracking = false;
		}
		else
		{
			DetermineResourceStates(InHeapProps, InInitialD3D12Access, InResourceStateMode, InDefaultD3D12Access);
		}

		if (bRequiresResourceStateTracking)
		{
#if D3D12_RHI_RAYTRACING
			// No state tracking for acceleration structures because they can't have another state
			check(!EnumHasAnyFlags(InInitialD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite) ||
		          !EnumHasAnyFlags(InDefaultD3D12Access, ED3D12Access::BVHRead | ED3D12Access::BVHWrite));
#endif // D3D12_RHI_RAYTRACING

			check(InInitialD3D12Access != ED3D12Access::Unknown);
			InitialD3D12Access = InInitialD3D12Access;
		}
	}

	void DetermineResourceStates(
		const D3D12_HEAP_PROPERTIES* InHeapProps,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access)
	{
		const FD3D12ResourceTypeHelper Type(Desc, HeapType);

		bDepthStencil = Type.bDSV;

#ifdef PLATFORM_SUPPORTS_RESOURCE_COMPRESSION
		SetCompressedState(D3D12_RESOURCE_STATE_COMMON);
#endif

		if (!Type.bWritable && InResourceStateMode != ED3D12ResourceStateMode::MultiState)
		{
			bRequiresResourceStateTracking = false;
						
#if UE_BUILD_DEBUG
			FPlatformAtomics::InterlockedIncrement(&NoStateTrackingResourceCount);
#endif
			if (InDefaultD3D12Access != ED3D12Access::Unknown)
			{
				DefaultD3D12Access = InDefaultD3D12Access;
			}
			else if (Type.bBuffer)
			{
				DefaultD3D12Access = DetermineInitialBufferD3D12Access(HeapType, InHeapProps);
			}
			else
			{
				check(Type.bSRVOnly);
				DefaultD3D12Access = ED3D12Access::SRVGraphics;
			}

			check(InInitialD3D12Access != ED3D12Access::Unknown);
			InitialD3D12Access = InInitialD3D12Access;
		}
	}
};

typedef class FD3D12BuddyAllocator FD3D12BaseAllocatorType;

struct FD3D12BuddyAllocatorPrivateData
{
	uint32 Offset;
	uint32 Order;

	void Init()
	{
		Offset = 0;
		Order = 0;
	}
};

struct FD3D12PoolAllocatorPrivateData
{
	FRHIPoolAllocationData PoolData;

	void Init()
	{
		PoolData.Reset();
	}
};

class FD3D12ResourceAllocator;
class FD3D12BaseShaderResource;

// A very light-weight and cache friendly way of accessing a GPU resource
class FD3D12ResourceLocation : public FRHIPoolResource, public FD3D12DeviceChild, public FNoncopyable
{
	friend class FD3D12PoolAllocator;
public:
	enum class ResourceLocationType : uint8
	{
		eUndefined,
		eStandAlone,
		eSubAllocation,
		eFastAllocation,
		eMultiFrameFastAllocation,
		eAliased, // XR HMDs are the only use cases
		eNodeReference,
		eHeapAliased, 
	};

	enum EAllocatorType : uint8
	{
		AT_Default, // FD3D12BaseAllocatorType
		AT_Pool,	// FD3D12PoolAllocator
		AT_Unknown = 0xff
	};

	FD3D12ResourceLocation(FD3D12Device* Parent);
	FD3D12ResourceLocation(FD3D12ResourceLocation&& Other)
		: FD3D12ResourceLocation(Other.GetParentDevice())
	{
		TransferOwnership(*this, Other);
	}

	~FD3D12ResourceLocation();

	void Clear();

	// Transfers the contents of 1 resource location to another, destroying the original but preserving the underlying resource 
	static void TransferOwnership(FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source);

	// Setters
	void SetOwner   (FD3D12BaseShaderResource* InOwner) { Owner = InOwner; }
	void SetType    (ResourceLocationType Value)        { Type = Value;}
	void SetResource(FD3D12Resource* Value);

	void SetAllocator       (FD3D12BaseAllocatorType* Value) { Allocator        = Value; AllocatorType = AT_Default; }
	void SetPoolAllocator   (FD3D12PoolAllocator*     Value) { PoolAllocator    = Value; AllocatorType = AT_Pool;    }

	void ClearAllocator() { Allocator = nullptr; AllocatorType = AT_Unknown; }

	void SetMappedBaseAddress       (void* Value                    ) { MappedBaseAddress = Value; }
	void SetGPUVirtualAddress       (D3D12_GPU_VIRTUAL_ADDRESS Value) { GPUVirtualAddress = Value; }
	void SetOffsetFromBaseOfResource(uint64 Value                   ) { OffsetFromBaseOfResource = Value; }
	void SetSize                    (uint64 Value                   ) { Size = Value; }

	// Getters
	ResourceLocationType               GetType                       () const { return Type;                                                 }
	EAllocatorType                     GetAllocatorType              () const { return AllocatorType;                                        }
	FD3D12BaseAllocatorType*           GetAllocator                  ()       { check(AT_Default == AllocatorType); return Allocator;        }
	FD3D12PoolAllocator*               GetPoolAllocator              ()       { check(AT_Pool == AllocatorType);    return PoolAllocator;    }
	FD3D12Resource*                    GetResource                   () const { return UnderlyingResource;                                   }
	void*                              GetMappedBaseAddress          () const { return MappedBaseAddress;                                    }
	D3D12_GPU_VIRTUAL_ADDRESS          GetGPUVirtualAddress          () const { return GPUVirtualAddress;                                    }
	uint64                             GetOffsetFromBaseOfResource   () const { return OffsetFromBaseOfResource;                             }
	uint64                             GetSize                       () const { return Size;                                                 }
	FD3D12BuddyAllocatorPrivateData&   GetBuddyAllocatorPrivateData  ()       { return AllocatorData.BuddyAllocatorPrivateData;              }
	FD3D12PoolAllocatorPrivateData&    GetPoolAllocatorPrivateData   ()       { return AllocatorData.PoolAllocatorPrivateData;               }

	// Pool allocation specific functions
	bool OnAllocationMoved(FD3D12ContextArray const& Contexts, FRHIPoolAllocationData* InNewData, ED3D12Access& OutRHIAccess);
	void UnlockPoolData();

	bool IsValid() const { return Type != ResourceLocationType::eUndefined; }

	void AsStandAlone(FD3D12Resource* Resource, uint64 InSize = 0, bool bInIsTransient = false, const D3D12_HEAP_PROPERTIES* CustomHeapProperties = nullptr);
	bool IsStandaloneOrPooledPlacedResource() const;

	void AsHeapAliased(FD3D12Resource* Resource)
	{
		check(Resource->GetHeapType() != D3D12_HEAP_TYPE_READBACK);

		SetType(FD3D12ResourceLocation::ResourceLocationType::eHeapAliased);
		SetResource(Resource);
		SetSize(0);

		if (IsCPUWritable(Resource->GetHeapType()))
		{
			D3D12_RANGE range = { 0, 0 };
			SetMappedBaseAddress(Resource->Map(&range));
		}
		SetGPUVirtualAddress(Resource->GetGPUVirtualAddress());
	}


	void AsFastAllocation(FD3D12Resource* Resource, uint32 BufferSize, D3D12_GPU_VIRTUAL_ADDRESS GPUBase, void* CPUBase, uint64 ResourceOffsetBase, uint64 Offset, bool bMultiFrame = false)
	{
		if (bMultiFrame)
		{
			Resource->AddRef();
			SetType(ResourceLocationType::eMultiFrameFastAllocation);
		}
		else
		{
			SetType(ResourceLocationType::eFastAllocation);
		}
		SetResource(Resource);
		SetSize(BufferSize);
		SetOffsetFromBaseOfResource(ResourceOffsetBase + Offset);

		if (CPUBase != nullptr)
		{
			SetMappedBaseAddress((uint8*)CPUBase + Offset);
		}
		SetGPUVirtualAddress(GPUBase + Offset);
	}

#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
	// XR plugins alias textures so this allows 2+ resource locations to reference the same underlying
	// resource. We should avoid this as much as possible as it requires expensive reference counting and
	// it complicates the resource ownership model.
	static void Alias(FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source);
#endif

	static void ReferenceNode(FD3D12Device* NodeDevice, FD3D12ResourceLocation& Destination, FD3D12ResourceLocation& Source);
	bool IsAliased() const
	{
		return (Type == ResourceLocationType::eAliased) 
			|| (Type == ResourceLocationType::eHeapAliased);
	}

	void SetTransient(bool bInTransient)
	{
		bTransient = bInTransient;
	}
	bool IsTransient() const
	{
		return bTransient;
	}

	/**  Get an address used by LLM to track the GPU allocation that this location represents. */
	void* GetAddressForLLMTracking() const
	{
		return (uint8*)this + 1;
	}

private:

	template<bool bReleaseResource>
	void InternalClear();

	void ReleaseResource();
	void UpdateStandAloneStats(bool bIncrement);

	FD3D12BaseShaderResource* Owner{};
	FD3D12Resource* UnderlyingResource{};

	// Which allocator this belongs to
	union
	{
		FD3D12BaseAllocatorType* Allocator;
		FD3D12PoolAllocator* PoolAllocator;
	};

	// Union to save memory
	union PrivateAllocatorData
	{
		FD3D12BuddyAllocatorPrivateData BuddyAllocatorPrivateData;
		FD3D12PoolAllocatorPrivateData PoolAllocatorPrivateData;
	} AllocatorData;

	// Note: These values refer to the start of this location including any padding *NOT* the start of the underlying resource
	void* MappedBaseAddress{};
	D3D12_GPU_VIRTUAL_ADDRESS GPUVirtualAddress{};
	uint64 OffsetFromBaseOfResource{};

	// The size the application asked for
	uint64 Size{};

	ResourceLocationType Type{ ResourceLocationType::eUndefined };
	EAllocatorType AllocatorType{ AT_Unknown };
	bool bTransient{ false };
};

// Generic interface for every type D3D12 specific allocator
struct ID3D12ResourceAllocator
{
	// Helper function for textures to compute the correct size and alignment
	void AllocateTexture(
		uint32 GPUIndex,
		D3D12_HEAP_TYPE InHeapType,
		const FD3D12ResourceDesc& InDesc,
		EPixelFormat InUEFormat,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		const TCHAR* InName,
		FD3D12ResourceLocation& ResourceLocation);

	// Actual pure virtual resource allocation function
	virtual void AllocateResource(
		uint32 GPUIndex,
		D3D12_HEAP_TYPE InHeapType,
		const FD3D12ResourceDesc& InDesc,
		uint64 InSize,
		uint32 InAllocationAlignment,
		ED3D12Access InInitialD3D12Access,
		ED3D12ResourceStateMode InResourceStateMode,
		ED3D12Access InDefaultD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		const TCHAR* InName,
		FD3D12ResourceLocation& ResourceLocation) = 0;
};

struct FD3D12LockedResource : public FD3D12DeviceChild
{
	FD3D12LockedResource(FD3D12Device* Device)
		: FD3D12DeviceChild(Device)
		, ResourceLocation(Device)
	{}

	inline void Reset()
	{
		ResourceLocation.Clear();
		bLocked = false;
		bLockedForReadOnly = false;
		LockOffset = 0;
		LockSize = 0;
		FMemory::Memzero(Footprint);
	}

	FD3D12ResourceLocation ResourceLocation;
	D3D12_SUBRESOURCE_FOOTPRINT Footprint = {};
	uint32 LockOffset = 0;
	uint32 LockSize = 0;
	uint32 bLocked : 1 = false;
	uint32 bLockedForReadOnly : 1 = false;
	uint32 bHasNeverBeenLocked : 1 = true;
};

/** Resource which might needs to be notified about changes on dependent resources (Views, RTGeometryObject, Cached binding tables) */
struct FD3D12ShaderResourceRenameListener
{
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) = 0;
};


/** The base class of resources that may be bound as shader resources (texture or buffer). */
class FD3D12BaseShaderResource : public FD3D12DeviceChild, public IRefCountedObject
{
private:
	mutable FCriticalSection RenameListenersCS;
	TArray<FD3D12ShaderResourceRenameListener*> RenameListeners;

public:
	FD3D12Resource* GetResource() const { return ResourceLocation.GetResource(); }

	void AddRenameListener(FD3D12ShaderResourceRenameListener* InRenameListener)
	{
		FScopeLock Lock(&RenameListenersCS);
		check(!RenameListeners.Contains(InRenameListener));
		RenameListeners.Add(InRenameListener);
	}

	void RemoveRenameListener(FD3D12ShaderResourceRenameListener* InRenameListener)
	{
		FScopeLock Lock(&RenameListenersCS);
		uint32 Removed = RenameListeners.Remove(InRenameListener);

		checkf(Removed == 1, TEXT("Should have exactly one registered listener during remove (same listener shouldn't registered twice and we shouldn't call this if not registered"));
	}

	bool HasLinkedViews() const
	{
		FScopeLock Lock(&RenameListenersCS);
		return RenameListeners.Num() != 0;
	}

	void ResourceRenamed(FD3D12ContextArray const& Contexts)
	{
		FScopeLock Lock(&RenameListenersCS);
		for (FD3D12ShaderResourceRenameListener* RenameListener : RenameListeners)
		{
			RenameListener->ResourceRenamed(Contexts, this, &ResourceLocation);
		}
	}

	D3D12_GPU_VIRTUAL_ADDRESS GetGPUVirtualAddress() const
	{
		return ResourceLocation.GetGPUVirtualAddress();
	}

	FD3D12ResourceLocation ResourceLocation;

public:
	FD3D12BaseShaderResource(FD3D12Device* InParent)
		: FD3D12DeviceChild(InParent)
		, ResourceLocation(InParent)
	{
		ResourceLocation.SetOwner(this);
	}

	~FD3D12BaseShaderResource()
	{
		check(!HasLinkedViews());
	}
};

class FD3D12UniformBuffer;

/** Resource which might needs to be notified about changes on dependent resources (Views, RTGeometryObject, Cached binding tables) */
struct ID3D12UniformBufferUpdateListener
{
	virtual void RemoveListener(FD3D12UniformBuffer* InUniformBufferToRemove) = 0;
	virtual void UniformBufferUpdated(const FD3D12ContextArray& Contexts, FD3D12UniformBuffer* InUpdatedUniformBuffer) = 0;
};

/** Uniform buffer resource class. */
class FD3D12UniformBuffer : public FRHIUniformBuffer, public FD3D12DeviceChild, public FD3D12LinkedAdapterObject<FD3D12UniformBuffer>
{
private:
	mutable FCriticalSection UpdateListenersCS;
	TArray<ID3D12UniformBufferUpdateListener*> UpdateListeners;
public:
	class FD3D12ConstantBufferView* View = nullptr;

	/** The D3D12 constant buffer resource */
	FD3D12ResourceLocation ResourceLocation;

	const EUniformBufferUsage UniformBufferUsage;

	/** Initialization constructor. */
	FD3D12UniformBuffer(class FD3D12Device* InParent, const FRHIUniformBufferLayout* InLayout, EUniformBufferUsage InUniformBufferUsage)
		: FRHIUniformBuffer(InLayout)
		, FD3D12DeviceChild(InParent)
		, ResourceLocation(InParent)
		, UniformBufferUsage(InUniformBufferUsage)
	{
	}

	virtual ~FD3D12UniformBuffer();

	virtual FRHIDescriptorHandle GetBindlessHandle() const override;

	// Provides public non-const access to ResourceTable.
	// @todo refactor uniform buffers to perform updates as a member function, so this isn't necessary.
	TArray<TRefCountPtr<FRHIResource>>& GetResourceTable() { return ResourceTable; }
		
	void AddUpdateListener(ID3D12UniformBufferUpdateListener* InUpdateListener)
	{
		FScopeLock Lock(&UpdateListenersCS);
		check(!UpdateListeners.Contains(InUpdateListener));
		UpdateListeners.Add(InUpdateListener);
	}

	void RemoveUpdateListener(ID3D12UniformBufferUpdateListener* InUpdateListener)
	{
		FScopeLock Lock(&UpdateListenersCS);

		// During dtor cleanup the Remove can be called on the same element multiple times
		UpdateListeners.Remove(InUpdateListener);
	}

	bool HasListeners() const
	{
		FScopeLock Lock(&UpdateListenersCS);
		return UpdateListeners.Num() != 0;
	}

	void UniformBufferUpdated(const FD3D12ContextArray& Contexts)
	{
		FScopeLock Lock(&UpdateListenersCS);
		for (ID3D12UniformBufferUpdateListener* UpdateListener : UpdateListeners)
		{
			UpdateListener->UniformBufferUpdated(Contexts, this);
		}
	}
};

class FD3D12Buffer : public FRHIBuffer, public FD3D12BaseShaderResource, public FD3D12LinkedAdapterObject<FD3D12Buffer>
{
public:
	FD3D12Buffer(FD3D12Device* InParent, const FRHIBufferCreateDesc& InCreateDesc)
		: FRHIBuffer(InCreateDesc)
		, FD3D12BaseShaderResource(InParent)
		, LockedData(InParent)
	{
	}

	virtual ~FD3D12Buffer();

	static void UploadResourceData(
		FD3D12CommandContext& CommandContext,
		ED3D12Access DestinationD3D12Access,
		FD3D12ResourceLocation& DestinationResourceLocation,
		const FD3D12ResourceLocation& SourceResourceLocation,
		uint32 Size);

	FD3D12SyncPointRef UploadResourceDataViaCopyQueue(FD3D12CommandContext& OwningContext, FResourceArrayUploadInterface* InResourceArray);

	// FRHIResource overrides
#if RHI_ENABLE_RESOURCE_INFO
	bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override
	{
		OutResourceInfo = FRHIResourceInfo{};
		OutResourceInfo.Name = GetName();
		OutResourceInfo.Type = GetType();
		if (ResourceLocation.GetResource() && ResourceLocation.GetResource()->IsReservedResource())
		{
			OutResourceInfo.VRamAllocation.AllocationSize = ResourceLocation.GetResource()->GetCommittedReservedResourceSize();
		}
		else
		{
			OutResourceInfo.VRamAllocation.AllocationSize = ResourceLocation.GetSize();
		}
		OutResourceInfo.IsTransient = ResourceLocation.IsTransient();
		OutResourceInfo.bResident = GetResource() && GetResource()->IsResident();

		return true;
	}
#endif

	void Rename(FD3D12ContextArray const& Contexts, FD3D12ResourceLocation& NewLocation);
	void RenameLDAChain(FD3D12ContextArray const& Contexts, FD3D12ResourceLocation& NewLocation);

	void TakeOwnership(FD3D12Buffer& Other);
	void ReleaseOwnership();

	// IRefCountedObject interface.
	virtual void AddRef() const override final
	{
		FRHIResource::AddRef();
	}
	virtual FReturnedRefCountValue Release() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::Release() };
	}
	virtual FReturnedRefCountValue GetRefCount() const override final
	{
		return FReturnedRefCountValue{ FRHIResource::GetRefCount() };
	}

	static void GetResourceDescAndAlignment(const FRHIBufferCreateDesc& CreateDesc, D3D12_RESOURCE_DESC& ResourceDesc, uint32& Alignment);

	// There might be more than 1 VRAM allocation in case of reserved resources
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	void UpdateAllocationTags() const;
#endif

	FD3D12LockedResource LockedData;
	uint32 BufferAlignment = 0;
};

namespace D3D12BufferStats
{
	void UpdateBufferStats(FD3D12Buffer& Buffer, bool bAllocating);
}

class FD3D12StagingBuffer final : public FRHIStagingBuffer
{
	friend class FD3D12CommandContext;
	friend class FD3D12DynamicRHI;

public:
	FD3D12StagingBuffer(FD3D12Device* InDevice)
		: FRHIStagingBuffer()
		, ResourceLocation(InDevice)
		, ShadowBufferSize(0)
	{}
	~FD3D12StagingBuffer() override;

	void SafeRelease()
	{
		ResourceLocation.Clear();
	}

	void* Lock(uint32 Offset, uint32 NumBytes) override;
	void Unlock() override;
	uint64 GetGPUSizeBytes() const override { return ShadowBufferSize; }

private:
	FD3D12ResourceLocation ResourceLocation;
	uint32 ShadowBufferSize;
};

class FD3D12ShaderBundle : public FRHIShaderBundle, public FD3D12DeviceChild
{
	friend class FD3D12CommandContext;
	friend class FD3D12DynamicRHI;

public:
	FD3D12ShaderBundle(FD3D12Device* InDevice, const FShaderBundleCreateInfo& CreateInfo)
	: FRHIShaderBundle(CreateInfo)
	, FD3D12DeviceChild(InDevice)
	{
	}
};

class FD3D12GPUFence final : public FRHIGPUFence
{
public:
	FD3D12GPUFence(FName InName);

	virtual void Clear() override;
	virtual bool Poll() const override;
	virtual bool Poll(FRHIGPUMask GPUMask) const override;
	virtual void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override;

	TArray<FD3D12SyncPointRef, TInlineAllocator<MAX_NUM_GPUS>> SyncPoints;
};

template<>
struct TD3D12ResourceTraits<FRHIUniformBuffer>
{
	typedef FD3D12UniformBuffer TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIBuffer>
{
	typedef FD3D12Buffer TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHISamplerState>
{
	typedef FD3D12SamplerState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIRasterizerState>
{
	typedef FD3D12RasterizerState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIDepthStencilState>
{
	typedef FD3D12DepthStencilState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIBlendState>
{
	typedef FD3D12BlendState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIGraphicsPipelineState>
{
	typedef FD3D12GraphicsPipelineState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIComputePipelineState>
{
	typedef FD3D12ComputePipelineState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIWorkGraphPipelineState>
{
	typedef FD3D12WorkGraphPipelineState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIGPUFence>
{
	typedef FD3D12GPUFence TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIStagingBuffer>
{
	typedef FD3D12StagingBuffer TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIShaderBundle>
{
	typedef FD3D12ShaderBundle TConcreteType;
};

#if D3D12_RHI_RAYTRACING
template<>
struct TD3D12ResourceTraits<FRHIRayTracingScene>
{
	typedef FD3D12RayTracingScene TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIRayTracingGeometry>
{
	typedef FD3D12RayTracingGeometry TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIRayTracingPipelineState>
{
	typedef FD3D12RayTracingPipelineState TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIShaderBindingTable>
{
	typedef FD3D12RayTracingShaderBindingTable TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIRayTracingShader>
{
	typedef FD3D12RayTracingShader TConcreteType;
};
#endif // D3D12_RHI_RAYTRACING
