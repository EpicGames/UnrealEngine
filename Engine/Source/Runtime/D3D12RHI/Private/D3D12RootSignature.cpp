// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RootSignature.cpp: D3D12 Root Signatures
=============================================================================*/

#include "D3D12RootSignature.h"
#include "D3D12RHIPrivate.h"
#include "D3D12AmdExtensions.h"
#include "D3D12RootSignatureDefinitions.h"
#include "RayTracingBuiltInResources.h"
#include "D3D12RayTracing.h"

#ifndef FD3D12_ROOT_SIGNATURE_FLAG_GLOBAL_ROOT_SIGNATURE
#define FD3D12_ROOT_SIGNATURE_FLAG_GLOBAL_ROOT_SIGNATURE D3D12_ROOT_SIGNATURE_FLAG_NONE
#endif

// Allows to automatically bind UEDiagnosticBuffer UAV, available to all shaders.
// If a shader is compiled with diagnostics enabled, it will fail to load/create 
// unless D3D12_ALLOW_SHADER_DIAGNOSTIC_BUFFER is enabled.
#define D3D12_ALLOW_SHADER_DIAGNOSTIC_BUFFER 1

namespace
{
	// Root parameter costs in DWORDs as described here: https://docs.microsoft.com/en-us/windows/desktop/direct3d12/root-signature-limits
	static const uint32 RootDescriptorTableCostGlobal = 1; // Descriptor tables cost 1 DWORD
	static const uint32 RootDescriptorTableCostLocal = 2; // Local root signature descriptor tables cost 2 DWORDs -- undocumented as of 2018-11-12
	static const uint32 RootConstantCost = 1; // Each root constant is 1 DWORD
	static const uint32 RootDescriptorCost = 2; // Root descriptor is 64-bit GPU virtual address, 2 DWORDs
}

static D3D12_STATIC_SAMPLER_DESC MakeStaticSampler(D3D12_FILTER Filter, D3D12_TEXTURE_ADDRESS_MODE WrapMode, uint32 Register, uint32 Space)
{
	D3D12_STATIC_SAMPLER_DESC Result = {};
	
	Result.Filter           = Filter;
	Result.AddressU         = WrapMode;
	Result.AddressV         = WrapMode;
	Result.AddressW         = WrapMode;
	Result.MipLODBias       = 0.0f;
	Result.MaxAnisotropy    = 1;
	Result.ComparisonFunc   = D3D12_COMPARISON_FUNC_NEVER;
	Result.BorderColor      = D3D12_STATIC_BORDER_COLOR_TRANSPARENT_BLACK;
	Result.MinLOD           = 0.0f;
	Result.MaxLOD           = D3D12_FLOAT32_MAX;
	Result.ShaderRegister   = Register;
	Result.RegisterSpace    = Space;
	Result.ShaderVisibility = D3D12_SHADER_VISIBILITY_ALL;

	return Result;
}

// Static sampler table must match D3DCommon.ush
static const D3D12_STATIC_SAMPLER_DESC StaticSamplerDescs[] =
{
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_MIP_POINT,        D3D12_TEXTURE_ADDRESS_MODE_WRAP,  0, 1000),
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_MIP_POINT,        D3D12_TEXTURE_ADDRESS_MODE_CLAMP, 1, 1000),
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_LINEAR_MIP_POINT, D3D12_TEXTURE_ADDRESS_MODE_WRAP,  2, 1000),
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_LINEAR_MIP_POINT, D3D12_TEXTURE_ADDRESS_MODE_CLAMP, 3, 1000),
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_MIP_LINEAR,       D3D12_TEXTURE_ADDRESS_MODE_WRAP,  4, 1000),
	MakeStaticSampler(D3D12_FILTER_MIN_MAG_MIP_LINEAR,       D3D12_TEXTURE_ADDRESS_MODE_CLAMP, 5, 1000),
};

FORCEINLINE D3D12_SHADER_VISIBILITY GetD3D12ShaderVisibility(EShaderVisibility Visibility)
{
	switch (Visibility)
	{
	case SV_Vertex:
		return D3D12_SHADER_VISIBILITY_VERTEX;
	case SV_Geometry:
		return D3D12_SHADER_VISIBILITY_GEOMETRY;
	case SV_Pixel:
		return D3D12_SHADER_VISIBILITY_PIXEL;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SV_Mesh:
		return D3D12_SHADER_VISIBILITY_MESH;
	case SV_Amplification:
		return D3D12_SHADER_VISIBILITY_AMPLIFICATION;
#endif
	case SV_All:
		return D3D12_SHADER_VISIBILITY_ALL;

	default:
		check(false);
		return static_cast<D3D12_SHADER_VISIBILITY>(-1);
	};
}

FORCEINLINE D3D12_ROOT_SIGNATURE_FLAGS GetD3D12RootSignatureDenyFlag(EShaderVisibility Visibility)
{
	switch (Visibility)
	{
	case SV_Vertex:
		return D3D12_ROOT_SIGNATURE_FLAG_DENY_VERTEX_SHADER_ROOT_ACCESS;
	case SV_Geometry:
		return D3D12_ROOT_SIGNATURE_FLAG_DENY_GEOMETRY_SHADER_ROOT_ACCESS;
	case SV_Pixel:
		return D3D12_ROOT_SIGNATURE_FLAG_DENY_PIXEL_SHADER_ROOT_ACCESS;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SV_Mesh:
		return GRHISupportsMeshShadersTier0 ? D3D12_ROOT_SIGNATURE_FLAG_DENY_MESH_SHADER_ROOT_ACCESS : D3D12_ROOT_SIGNATURE_FLAG_NONE;
	case SV_Amplification:
		return GRHISupportsMeshShadersTier0 ? D3D12_ROOT_SIGNATURE_FLAG_DENY_AMPLIFICATION_SHADER_ROOT_ACCESS : D3D12_ROOT_SIGNATURE_FLAG_NONE;
#endif
	case SV_All:
		return D3D12_ROOT_SIGNATURE_FLAG_NONE;

	default:
		check(false);
		return static_cast<D3D12_ROOT_SIGNATURE_FLAGS>(-1);
	};
}

static bool IsLocalRootSignature(ERootSignatureType InRootSignatureType)
{
	switch (InRootSignatureType)
	{
	case RS_RayTracingLocal:
	case RS_WorkGraphLocalCompute:
	case RS_WorkGraphLocalRaster:
		return true;
	default:
		return false;
	}
}

static uint32 GetBindingSpace(ERootSignatureType InRootSignatureType, bool bInIsPixelShader)
{
	switch (InRootSignatureType)
	{
	case RS_RayTracingGlobal:
		return UE_HLSL_SPACE_RAY_TRACING_GLOBAL;
	case RS_RayTracingLocal:
		return UE_HLSL_SPACE_RAY_TRACING_LOCAL;
	case RS_WorkGraphGlobal:
		return UE_HLSL_SPACE_WORK_GRAPH_GLOBAL;
	case RS_WorkGraphLocalCompute:
		return UE_HLSL_SPACE_WORK_GRAPH_LOCAL;
	case RS_WorkGraphLocalRaster:
		// We use separate binding spaces for the mesh and pixel stages.
		// If we add support for work graph vertex shaders then we will need to add a new binding space for the additional stage.
		return bInIsPixelShader ? UE_HLSL_SPACE_WORK_GRAPH_LOCAL_PIXEL : UE_HLSL_SPACE_WORK_GRAPH_LOCAL;
	default:
		break;
	}

	// Default binding space for D3D 11 & 12 shaders
	return UE_HLSL_SPACE_DEFAULT;
}

static uint32 GetBindingSpace(ERootSignatureType InRootSignatureType, EShaderVisibility InVisibility)
{
	return GetBindingSpace(InRootSignatureType, InVisibility == SV_Pixel);
}

static uint32 GetBindingSpace(ERootSignatureType InRootSignatureType, EShaderFrequency InFrequency)
{
	return GetBindingSpace(InRootSignatureType, InFrequency == SF_Pixel);
}


FD3D12RootSignatureDesc::FD3D12RootSignatureDesc(const FD3D12QuantizedBoundShaderState& QBSS, const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier)
	: RootParametersSize(0)
{
	const EShaderVisibility ShaderVisibilityPriorityOrder[] =
	{
		SV_Pixel,
		SV_Vertex,
		SV_Geometry,
#if PLATFORM_SUPPORTS_MESH_SHADERS
		SV_Mesh,
		SV_Amplification,
#endif
		SV_All
	};
	const D3D12_ROOT_PARAMETER_TYPE RootParameterTypePriorityOrder[] = { D3D12_ROOT_PARAMETER_TYPE_DESCRIPTOR_TABLE, D3D12_ROOT_PARAMETER_TYPE_CBV };
	uint32 RootParameterCount = 0;

	// Determine if our descriptors or their data is static based on the resource binding tier.
	// We do this because sometimes (based on binding tier) our descriptor tables are bigger than the # of descriptors we copy. See FD3D12QuantizedBoundShaderState::InitShaderRegisterCounts().
	const D3D12_DESCRIPTOR_RANGE_FLAGS SRVDescriptorRangeFlags = (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_1) ?
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE :
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;

	const D3D12_DESCRIPTOR_RANGE_FLAGS CBVDescriptorRangeFlags = (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_2) ?
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE :
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;

	const D3D12_DESCRIPTOR_RANGE_FLAGS UAVDescriptorRangeFlags = (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_2) ?
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_VOLATILE :
		D3D12_DESCRIPTOR_RANGE_FLAG_DATA_VOLATILE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;

	const D3D12_DESCRIPTOR_RANGE_FLAGS SamplerDescriptorRangeFlags = (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_1) ?
		D3D12_DESCRIPTOR_RANGE_FLAG_NONE :
		D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;

	const D3D12_ROOT_DESCRIPTOR_FLAGS CBVRootDescriptorFlags = D3D12_ROOT_DESCRIPTOR_FLAG_DATA_STATIC;	// We always set the data in an upload heap before calling Set*RootConstantBufferView.

	const bool bUseShaderDiagnosticBuffer = D3D12_ALLOW_SHADER_DIAGNOSTIC_BUFFER
		&& QBSS.bUseDiagnosticBuffer
		&& !IsLocalRootSignature(QBSS.RootSignatureType);

#if D3D12_RHI_RAYTRACING
	if (QBSS.RootSignatureType == RS_RayTracingLocal)
	{
		// Add standard root parameters for hit groups, as per FD3D12HitGroupSystemParameters declaration in D3D12Util.h 
		// and RayTracingHitGroupCommon.ush (Non-Bindless) and D3DCommon.ush (Bindless):
		//          FHitGroupSystemRootConstants:
		// 4 bytes: index/vertex fetch configuration as root constant (bitfield defining index and vertex formats)
		// 4 bytes: index buffer offset in bytes
		// 4 bytes: first primitive of the segment (as set in FRayTracingGeometrySegment)
		// 4 bytes: hit group user data
		//          Non-Bindless Resources:
		// 8 bytes: index buffer as root SRV (raw buffer)
		// 8 bytes: vertex buffer as root SRV (raw buffer)
		//          Bindless Resources:
		// 4 bytes: index buffer bindless resource index (raw buffer)
		// 4 bytes: vertex buffer bindless resource index (raw buffer)
		// 8 bytes: union padding with non-bindless data

		// -----------
		// 32 bytes

		check(RootParameterCount == 0 && RootParametersSize == 0); // We expect system RT parameters to come first
		
		// Bindless FD3D12HitGroupSystemParameters structure
		if (QBSS.bUseDirectlyIndexedResourceHeap)
		{			
			check(RootParameterCount < MaxRootParameters);
			static_assert(sizeof(FD3D12HitGroupSystemParameters) % 8 == 0, "FD3D12HitGroupSystemParameters structure must be 8-byte aligned");

			// Num constants could be 2 smaller if non-bindless data is removed (24 bytes instead of 32)
			const uint32 NumConstants = sizeof(FD3D12HitGroupSystemParameters) / sizeof(uint32);
			TableSlots[RootParameterCount].InitAsConstants(NumConstants, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
			RootParameterCount++;
			RootParametersSize += NumConstants * RootConstantCost;
		}
		else
		{
			// FHitGroupSystemRootConstants structure
			{
				check(RootParameterCount < MaxRootParameters);
				static_assert(sizeof(FHitGroupSystemRootConstants) % 8 == 0, "FHitGroupSystemRootConstants structure must be 8-byte aligned");
				const uint32 NumConstants = sizeof(FHitGroupSystemRootConstants) / sizeof(uint32);
				TableSlots[RootParameterCount].InitAsConstants(NumConstants, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
				RootParameterCount++;
				RootParametersSize += NumConstants * RootConstantCost;
			}

			// Index buffer descriptor
			{
				check(RootParameterCount < MaxRootParameters);
				TableSlots[RootParameterCount].InitAsShaderResourceView(RAY_TRACING_SYSTEM_INDEXBUFFER_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
				RootParameterCount++;
				RootParametersSize += RootDescriptorCost;
			}

			// Vertex buffer descriptor
			{
				check(RootParameterCount < MaxRootParameters);
				TableSlots[RootParameterCount].InitAsShaderResourceView(RAY_TRACING_SYSTEM_VERTEXBUFFER_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
				RootParameterCount++;
				RootParametersSize += RootDescriptorCost;
			}
		}	
	}
#endif //D3D12_RHI_RAYTRACING

	const uint32 RootDescriptorTableCost = IsLocalRootSignature(QBSS.RootSignatureType) ? RootDescriptorTableCostLocal : RootDescriptorTableCostGlobal;

	// For each root parameter type...
	for (uint32 RootParameterTypeIndex = 0; RootParameterTypeIndex < UE_ARRAY_COUNT(RootParameterTypePriorityOrder); RootParameterTypeIndex++)
	{
		const D3D12_ROOT_PARAMETER_TYPE& RootParameterType = RootParameterTypePriorityOrder[RootParameterTypeIndex];

		// ... and each shader stage visibility ...
		for (uint32 ShaderVisibilityIndex = 0; ShaderVisibilityIndex < UE_ARRAY_COUNT(ShaderVisibilityPriorityOrder); ShaderVisibilityIndex++)
		{
			const EShaderVisibility StageVisibility = ShaderVisibilityPriorityOrder[ShaderVisibilityIndex];
			const FShaderRegisterCounts& Shader = QBSS.RegisterCounts[StageVisibility];
			// Work graphs must use SV_All and we use the per frequency binding space instead.
			const EShaderVisibility Visibility = QBSS.RootSignatureType == RS_WorkGraphLocalRaster ? SV_All : StageVisibility;
			const uint32 BindingSpace = GetBindingSpace(QBSS.RootSignatureType, StageVisibility);

			switch (RootParameterType)
			{
			case D3D12_ROOT_PARAMETER_TYPE_DESCRIPTOR_TABLE:
			{
				if (Shader.ShaderResourceCount > 0)
				{
					check(RootParameterCount < MaxRootParameters);
					DescriptorRanges[RootParameterCount].Init(D3D12_DESCRIPTOR_RANGE_TYPE_SRV, Shader.ShaderResourceCount, 0u, BindingSpace, SRVDescriptorRangeFlags);
					TableSlots[RootParameterCount].InitAsDescriptorTable(1, &DescriptorRanges[RootParameterCount], GetD3D12ShaderVisibility(Visibility));
					RootParameterCount++;
					RootParametersSize += RootDescriptorTableCost;
				}

				if (Shader.ConstantBufferCount > MAX_ROOT_CBVS)
				{
					checkf(!IsLocalRootSignature(QBSS.RootSignatureType), TEXT("CBV descriptor tables are not implemented for local root signatures"));

					// Use a descriptor table for the 'excess' CBVs
					check(RootParameterCount < MaxRootParameters);
					DescriptorRanges[RootParameterCount].Init(D3D12_DESCRIPTOR_RANGE_TYPE_CBV, Shader.ConstantBufferCount - MAX_ROOT_CBVS, MAX_ROOT_CBVS, BindingSpace, CBVDescriptorRangeFlags);
					TableSlots[RootParameterCount].InitAsDescriptorTable(1, &DescriptorRanges[RootParameterCount], GetD3D12ShaderVisibility(Visibility));
					RootParameterCount++;
					RootParametersSize += RootDescriptorTableCost;
				}

				if (Shader.SamplerCount > 0)
				{
					check(RootParameterCount < MaxRootParameters);
					DescriptorRanges[RootParameterCount].Init(D3D12_DESCRIPTOR_RANGE_TYPE_SAMPLER, Shader.SamplerCount, 0u, BindingSpace, SamplerDescriptorRangeFlags);
					TableSlots[RootParameterCount].InitAsDescriptorTable(1, &DescriptorRanges[RootParameterCount], GetD3D12ShaderVisibility(Visibility));
					RootParameterCount++;
					RootParametersSize += RootDescriptorTableCost;
				}

				if (Shader.UnorderedAccessCount > 0)
				{
					check(RootParameterCount < MaxRootParameters);
					DescriptorRanges[RootParameterCount].Init(D3D12_DESCRIPTOR_RANGE_TYPE_UAV, Shader.UnorderedAccessCount, 0u, BindingSpace, UAVDescriptorRangeFlags);
					TableSlots[RootParameterCount].InitAsDescriptorTable(1, &DescriptorRanges[RootParameterCount], GetD3D12ShaderVisibility(Visibility));
					RootParameterCount++;
					RootParametersSize += RootDescriptorTableCost;
				}
				break;
			}

			case D3D12_ROOT_PARAMETER_TYPE_CBV:
			{
				for (uint32 ShaderRegister = 0; (ShaderRegister < Shader.ConstantBufferCount) && (ShaderRegister < MAX_ROOT_CBVS); ShaderRegister++)
				{
					check(RootParameterCount < MaxRootParameters);
					TableSlots[RootParameterCount].InitAsConstantBufferView(ShaderRegister, BindingSpace, CBVRootDescriptorFlags, GetD3D12ShaderVisibility(Visibility));
					RootParameterCount++;
					RootParametersSize += RootDescriptorCost;
				}
				break;
			}

			default:
				check(false);
				break;
			}
		}
	}

	D3D12_ROOT_SIGNATURE_FLAGS Flags = D3D12_ROOT_SIGNATURE_FLAG_NONE;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (QBSS.bUseDirectlyIndexedResourceHeap)
	{
		Flags |= D3D12_ROOT_SIGNATURE_FLAG_CBV_SRV_UAV_HEAP_DIRECTLY_INDEXED;
	}

	if (QBSS.bUseDirectlyIndexedSamplerHeap)
	{
		Flags |= D3D12_ROOT_SIGNATURE_FLAG_SAMPLER_HEAP_DIRECTLY_INDEXED;
	}
#endif

#if D3D12_RHI_RAYTRACING
	if (IsLocalRootSignature(QBSS.RootSignatureType))
	{
		Flags = D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE;
	}
	else if(QBSS.RootSignatureType == RS_RayTracingGlobal)
	{
		Flags |= FD3D12_ROOT_SIGNATURE_FLAG_GLOBAL_ROOT_SIGNATURE;
	}
	else if (QBSS.RootSignatureType == RS_Raster)
#endif // D3D12_RHI_RAYTRACING
	{
		// Determine what shader stages need access in the root signature.

		if (QBSS.bAllowIAInputLayout)
		{
			Flags |= D3D12_ROOT_SIGNATURE_FLAG_ALLOW_INPUT_ASSEMBLER_INPUT_LAYOUT;
		}

		// don't set deny flags when static shader resource table is used
		if (QBSS.ShaderBindingLayout.GetNumUniformBufferEntries() == 0)
		{
			for (uint32 ShaderVisibilityIndex = 0; ShaderVisibilityIndex < UE_ARRAY_COUNT(ShaderVisibilityPriorityOrder); ShaderVisibilityIndex++)
			{
				const EShaderVisibility Visibility = ShaderVisibilityPriorityOrder[ShaderVisibilityIndex];
				const FShaderRegisterCounts& Shader = QBSS.RegisterCounts[Visibility];
				if ((Shader.ShaderResourceCount == 0) &&
					(Shader.ConstantBufferCount == 0) &&
					(Shader.UnorderedAccessCount == 0) &&
					(Shader.SamplerCount == 0))
				{
					// This shader stage doesn't use any descriptors, deny access to the shader stage in the root signature.
					Flags = (Flags | GetD3D12RootSignatureDenyFlag(Visibility));
				}
			}
		}
	}

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS && WITH_AMD_AGS
	if (QBSS.bNeedsAgsIntrinsicsSpace)
	{
		check(RootParameterCount < MaxRootParameters);
		TableSlots[RootParameterCount].InitAsUnorderedAccessView(0, AGS_DX12_SHADER_INSTRINSICS_SPACE_ID, D3D12_ROOT_DESCRIPTOR_FLAG_DATA_VOLATILE, D3D12_SHADER_VISIBILITY_ALL);
		RootParameterCount++;
		RootParametersSize += RootDescriptorCost;
	}
#endif
	
	// Add all the static defined uniform buffers
	if (QBSS.ShaderBindingLayout.GetNumUniformBufferEntries() > 0)
	{
		StaticShaderBindingSlot = int8(RootParameterCount);
		StaticShaderBindingCount = 0;
		for (uint32 Index = 0; Index < QBSS.ShaderBindingLayout.GetNumUniformBufferEntries(); ++Index)
		{
			const FRHIUniformBufferShaderBindingLayout& UniformBufferSBLayout = QBSS.ShaderBindingLayout.GetUniformBufferEntry(Index);
			
			check(RootParameterCount < MaxRootParameters);
			check(UniformBufferSBLayout.RegisterSpace > 0);
			TableSlots[RootParameterCount].InitAsConstantBufferView(UniformBufferSBLayout.CBVResourceIndex, UniformBufferSBLayout.RegisterSpace, CBVRootDescriptorFlags, D3D12_SHADER_VISIBILITY_ALL);
			StaticShaderBindingCount = FMath::Max(StaticShaderBindingCount, int8(UniformBufferSBLayout.CBVResourceIndex + 1));

			RootParameterCount++;
			RootParametersSize += RootDescriptorCost;
		}
	}

	if (QBSS.bUseRootConstants)
	{
		check(RootParameterCount < MaxRootParameters);
		RootConstantsSlot = int8(RootParameterCount);
		TableSlots[RootParameterCount].InitAsConstants(4, 0, UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS, D3D12_SHADER_VISIBILITY_ALL);
		RootParameterCount++;
		RootParametersSize += RootDescriptorCost;
	}

#if WITH_NVAPI
	if (QBSS.bNeedsNvIntrinsicsSpace)
	{
		check(RootParameterCount < MaxRootParameters);

		// Must use a DescriptorTable since structured buffers with counters can't be bound via root descriptors
		DescriptorRanges[RootParameterCount].Init(D3D12_DESCRIPTOR_RANGE_TYPE_UAV, 1, UE_HLSL_SLOT_NV_SHADER_EXTN, UE_HLSL_SPACE_NV_SHADER_EXTN, D3D12_DESCRIPTOR_RANGE_FLAG_NONE, 0);
		TableSlots[RootParameterCount].InitAsDescriptorTable(1, &DescriptorRanges[RootParameterCount]);
		RootParameterCount++;
		RootParametersSize += RootDescriptorCost;
	}
#endif

	if (bUseShaderDiagnosticBuffer)
	{
		check(RootParameterCount < MaxRootParameters);
		DiagnosticBufferSlot = int8(RootParameterCount);
		TableSlots[RootParameterCount].InitAsUnorderedAccessView(0, UE_HLSL_SPACE_DIAGNOSTIC, D3D12_ROOT_DESCRIPTOR_FLAG_DATA_VOLATILE, D3D12_SHADER_VISIBILITY_ALL);
		RootParameterCount++;
		RootParametersSize += RootDescriptorCost;
	}

	// Init the desc (warn about the size if necessary).
#if !NO_LOGGING
	const uint32 SizeWarningThreshold = 12;
	if (RootParametersSize > SizeWarningThreshold && QBSS.RootSignatureType == RS_Raster)
	{
		UE_LOGF(LogD3D12RHI, Verbose, "Root signature created where the root parameters take up %u DWORDS. Using more than %u DWORDs can negatively impact performance depending on the hardware and root parameter usage.", RootParametersSize, SizeWarningThreshold);
	}
#endif

#if D3D12_RHI_RAYTRACING
	if (IsLocalRootSignature(QBSS.RootSignatureType))
	{
		// Local root signatures don't need to provide static samplers as they are provided by global RS already.
		// Providing static sampler bindings in global and local RS simultaneously is invalid due to overlapping register ranges.
		RootDesc.Init_1_1(RootParameterCount, TableSlots, 0, nullptr, Flags);
	}
	else
#endif
	{
		// Only use static samplers for binding tier higher than 1 otherwise root signature only supports 16 samplers
		// Only use by DXR shaders and validated that DXR has at least Tier 2 support
		if (ResourceBindingTier > D3D12_RESOURCE_BINDING_TIER_1)
		{
			RootDesc.Init_1_1(RootParameterCount, TableSlots, UE_ARRAY_COUNT(StaticSamplerDescs), StaticSamplerDescs, Flags);
		}
		else
		{
			RootDesc.Init_1_1(RootParameterCount, TableSlots, 0, nullptr, Flags);
		}
	}
}

void FD3D12RootSignature::InitStaticGraphicsRootSignature(EShaderBindingLayoutFlags InFlags)
{
	D3D12ShaderUtils::FBinaryRootSignatureCreator Creator;
	D3D12ShaderUtils::CreateGfxRootSignature(Creator, InFlags);
	Init(Creator.Finalize(), RS_Raster);

	for (int32 ParameterSlot = 0; ParameterSlot < Creator.Parameters.Num(); ++ParameterSlot)
	{
		const CD3DX12_ROOT_PARAMETER1 RootParameter = Creator.Parameters[ParameterSlot];

		if (RootParameter.ParameterType == D3D12_ROOT_PARAMETER_TYPE_32BIT_CONSTANTS
			&& RootParameter.Constants.RegisterSpace == UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS
			&& RootParameter.Constants.ShaderRegister == 0)
		{
			RootConstantsSlot = int8(ParameterSlot);
		}

		if (RootParameter.ParameterType == D3D12_ROOT_PARAMETER_TYPE_UAV
			&& RootParameter.Descriptor.RegisterSpace == UE_HLSL_SPACE_DIAGNOSTIC
			&& RootParameter.Descriptor.ShaderRegister == 0)
		{
			DiagnosticBufferSlot = int8(ParameterSlot);
		}
	}
}

void FD3D12RootSignature::InitStaticComputeRootSignatureDesc(EShaderBindingLayoutFlags InFlags)
{
	D3D12ShaderUtils::FBinaryRootSignatureCreator Creator;
	D3D12ShaderUtils::CreateComputeRootSignature(Creator, InFlags);
	Init(Creator.Finalize(), RS_Raster);

	for (int32 ParameterSlot = 0; ParameterSlot < Creator.Parameters.Num(); ++ParameterSlot)
	{
		const CD3DX12_ROOT_PARAMETER1 RootParameter = Creator.Parameters[ParameterSlot];

		if (RootParameter.ParameterType == D3D12_ROOT_PARAMETER_TYPE_32BIT_CONSTANTS
			&& RootParameter.Constants.RegisterSpace == UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS
			&& RootParameter.Constants.ShaderRegister == 0)
		{
			RootConstantsSlot = int8(ParameterSlot);
		}

		if (RootParameter.ParameterType == D3D12_ROOT_PARAMETER_TYPE_UAV
			&& RootParameter.Descriptor.RegisterSpace == UE_HLSL_SPACE_DIAGNOSTIC
			&& RootParameter.Descriptor.ShaderRegister == 0)
		{
			DiagnosticBufferSlot = int8(ParameterSlot);
		}
	}
}

#if D3D12_RHI_RAYTRACING
void FD3D12RootSignature::InitStaticRayTracingGlobalRootSignatureDesc(EShaderBindingLayoutFlags InFlags)
{
	D3D12ShaderUtils::FBinaryRootSignatureCreator Creator;
	D3D12ShaderUtils::CreateRayTracingSignature(Creator, false, FD3D12_ROOT_SIGNATURE_FLAG_GLOBAL_ROOT_SIGNATURE, InFlags);
	Init(Creator.Finalize(), RS_RayTracingGlobal);
}

void FD3D12RootSignature::InitStaticRayTracingLocalRootSignatureDesc(EShaderBindingLayoutFlags InFlags)
{
	D3D12ShaderUtils::FBinaryRootSignatureCreator Creator;
	D3D12ShaderUtils::CreateRayTracingSignature(Creator, true, D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE, InFlags);
	Init(Creator.Finalize(), RS_RayTracingLocal);
}
#endif // D3D12_RHI_RAYTRACING

void FD3D12RootSignature::Init(const FD3D12QuantizedBoundShaderState& InQBSS)
{
	// Create a root signature desc from the quantized bound shader state.
	const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier = GetParentAdapter()->GetResourceBindingTier();

	FD3D12RootSignatureDesc Desc(InQBSS, ResourceBindingTier);

	RootConstantsSlot = Desc.GetRootConstantsSlot();
	StaticShaderBindingSlot = Desc.GetStaticShaderBindingSlot();
	StaticShaderBindingCount = Desc.GetStaticShaderBindingCount();
	DiagnosticBufferSlot = Desc.GetDiagnosticBufferSlot();

	Init(Desc.GetDesc(), InQBSS.RootSignatureType);
}

void FD3D12RootSignature::Init(const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& InDesc, ERootSignatureType InRootSignatureType)
{
	ID3D12Device* Device = GetParentAdapter()->GetD3DDevice();
	
	// Serialize the desc.
	TRefCountPtr<ID3DBlob> Error;
	const D3D_ROOT_SIGNATURE_VERSION MaxRootSignatureVersion = GetParentAdapter()->GetRootSignatureVersion();
	const HRESULT SerializeHR = D3DX12SerializeVersionedRootSignature(&InDesc, MaxRootSignatureVersion, RootSignatureBlob.GetInitReference(), Error.GetInitReference());
	if (Error.GetReference())
	{
		UE_LOGF(LogD3D12RHI, Fatal, "D3DX12SerializeVersionedRootSignature failed with error %ls", ANSI_TO_TCHAR(Error->GetBufferPointer()));
	}
	VERIFYD3D12RESULT(SerializeHR);

	// Create and analyze the root signature.
	VERIFYD3D12RESULT(Device->CreateRootSignature(FRHIGPUMask::All().GetNative(),
		RootSignatureBlob->GetBufferPointer(),
		RootSignatureBlob->GetBufferSize(),
		IID_PPV_ARGS(RootSignature.GetInitReference())));

	AnalyzeSignature(InDesc, InRootSignatureType);
	// TODO: Analyze vendor extension space?
}

void FD3D12RootSignature::AnalyzeSignature(const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& Desc, ERootSignatureType InRootSignatureType)
{
	switch (Desc.Version)
	{
	case D3D_ROOT_SIGNATURE_VERSION_1_0:
		InternalAnalyzeSignature(Desc.Desc_1_0, InRootSignatureType);
		break;

	case D3D_ROOT_SIGNATURE_VERSION_1_1:
		InternalAnalyzeSignature(Desc.Desc_1_1, InRootSignatureType);
		break;

	default:
		ensureMsgf(false, TEXT("Invalid root signature version %u"), Desc.Version);
		break;
	}
}

template<typename RootSignatureDescType>
void FD3D12RootSignature::InternalAnalyzeSignature(const RootSignatureDescType& Desc, ERootSignatureType InRootSignatureType)
{
	// Reset members to default values.
	{
		FMemory::Memset(BindSlotMap, InvalidBindSlotMapIndex, sizeof(BindSlotMap));
		bHasUAVs = false;
		bHasSRVs = false;
		bHasCBVs = false;
		bHasRootCBs = false;
		bHasSamplers = false;

		FMemory::Memset(BindSlotOffsetsInDWORDs, 0, sizeof(BindSlotOffsetsInDWORDs));
		TotalRootSignatureSizeInDWORDs = 0;
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	bUsesDynamicResources = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_CBV_SRV_UAV_HEAP_DIRECTLY_INDEXED) != 0;
	bUsesDynamicSamplers = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_SAMPLER_HEAP_DIRECTLY_INDEXED) != 0;
#endif

	const bool bDenyVS = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_DENY_VERTEX_SHADER_ROOT_ACCESS) != 0;
	const bool bDenyGS = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_DENY_GEOMETRY_SHADER_ROOT_ACCESS) != 0;
	const bool bDenyPS = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_DENY_PIXEL_SHADER_ROOT_ACCESS) != 0;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	const bool bDenyMS = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_DENY_MESH_SHADER_ROOT_ACCESS) != 0;
	const bool bDenyAS = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_DENY_AMPLIFICATION_SHADER_ROOT_ACCESS) != 0;
#endif

#if D3D12_RHI_RAYTRACING
	const uint32 RootDescriptorTableCost = (Desc.Flags & D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE) ? RootDescriptorTableCostLocal : RootDescriptorTableCostGlobal;
#else
	const uint32 RootDescriptorTableCost = RootDescriptorTableCostGlobal;
#endif

	// Go through each root parameter.
	for (uint32 i = 0; i < Desc.NumParameters; i++)
	{
		const auto& CurrentParameter = Desc.pParameters[i];

		uint32 ParameterBindingSpace = ~0u;

		switch (CurrentParameter.ParameterType)
		{
		case D3D12_ROOT_PARAMETER_TYPE_DESCRIPTOR_TABLE:
			check(CurrentParameter.DescriptorTable.NumDescriptorRanges == 1); // Code currently assumes a single descriptor range.
			ParameterBindingSpace = CurrentParameter.DescriptorTable.pDescriptorRanges[0].RegisterSpace;
			BindSlotOffsetsInDWORDs[i] = TotalRootSignatureSizeInDWORDs;
			TotalRootSignatureSizeInDWORDs += RootDescriptorTableCost;
			break;
		case D3D12_ROOT_PARAMETER_TYPE_32BIT_CONSTANTS:
			ParameterBindingSpace = CurrentParameter.Constants.RegisterSpace;
			BindSlotOffsetsInDWORDs[i] = TotalRootSignatureSizeInDWORDs;
			TotalRootSignatureSizeInDWORDs += RootConstantCost * CurrentParameter.Constants.Num32BitValues;
			break;
		case D3D12_ROOT_PARAMETER_TYPE_CBV:
		case D3D12_ROOT_PARAMETER_TYPE_SRV:
		case D3D12_ROOT_PARAMETER_TYPE_UAV:
			ParameterBindingSpace = CurrentParameter.Descriptor.RegisterSpace;
			BindSlotOffsetsInDWORDs[i] = TotalRootSignatureSizeInDWORDs;
			TotalRootSignatureSizeInDWORDs += RootDescriptorCost;
			break;
		default:
			checkNoEntry();
			break;
		}

		EShaderFrequency CurrentVisibleSF = SF_NumFrequencies;
		switch (CurrentParameter.ShaderVisibility)
		{
		case D3D12_SHADER_VISIBILITY_ALL:
			if (InRootSignatureType == RS_WorkGraphLocalRaster)
			{
				// Handle work graph graphics node case where shader visibility will be SV_All.
				// We need to use the stored parameter binding space to determine which stage this is.
				if (ParameterBindingSpace == UE_HLSL_SPACE_WORK_GRAPH_LOCAL)
				{
					CurrentVisibleSF = SF_Mesh;
				}
				else if (ParameterBindingSpace == UE_HLSL_SPACE_WORK_GRAPH_LOCAL_PIXEL)
				{
					CurrentVisibleSF = SF_Pixel;
				}
			}
			break;

		case D3D12_SHADER_VISIBILITY_VERTEX:
			CurrentVisibleSF = SF_Vertex;
			break;
		case D3D12_SHADER_VISIBILITY_GEOMETRY:
			CurrentVisibleSF = SF_Geometry;
			break;
		case D3D12_SHADER_VISIBILITY_PIXEL:
			CurrentVisibleSF = SF_Pixel;
			break;

#if PLATFORM_SUPPORTS_MESH_SHADERS
		case D3D12_SHADER_VISIBILITY_MESH:
			CurrentVisibleSF = SF_Mesh;
			break;
		case D3D12_SHADER_VISIBILITY_AMPLIFICATION:
			CurrentVisibleSF = SF_Amplification;
			break;
#endif

		default:
			check(false);
			break;
		}

		const uint32 BindingSpace = GetBindingSpace(InRootSignatureType, CurrentVisibleSF);

		// Track parameters bound to the Static Shader Binding layout slot independent of the regular parameter binding space
		if (ParameterBindingSpace == UE_HLSL_SPACE_STATIC_SHADER_BINDINGS)
		{
			if (StaticShaderBindingSlot < 0)
			{
				// This is the first CBV for this shader binding layout, save it's root parameter index (other CBVs will be indexed using this base root parameter index).
				StaticShaderBindingSlot = i;
				StaticShaderBindingCount = 0;
			}
			StaticShaderBindingCount++;

			// All shader binding layout CBVs should be contiguous registers.
			check(i == StaticShaderBindingSlot + CurrentParameter.Descriptor.ShaderRegister);

			continue;
		}
		else if (BindingSpace != ParameterBindingSpace)
		{
			// Only consider parameters in the requested binding space.
			continue;
		}

		// Determine shader stage visibility.
		{
			Stage[SF_Vertex].bVisible = Stage[SF_Vertex].bVisible || (!bDenyVS && HasVisibility(CurrentParameter.ShaderVisibility, D3D12_SHADER_VISIBILITY_VERTEX));
			Stage[SF_Geometry].bVisible = Stage[SF_Geometry].bVisible || (!bDenyGS && HasVisibility(CurrentParameter.ShaderVisibility, D3D12_SHADER_VISIBILITY_GEOMETRY));
			Stage[SF_Pixel].bVisible = Stage[SF_Pixel].bVisible || (!bDenyPS && HasVisibility(CurrentParameter.ShaderVisibility, D3D12_SHADER_VISIBILITY_PIXEL));

#if PLATFORM_SUPPORTS_MESH_SHADERS
			Stage[SF_Mesh].bVisible = Stage[SF_Mesh].bVisible || (!bDenyMS && HasVisibility(CurrentParameter.ShaderVisibility, D3D12_SHADER_VISIBILITY_MESH));
			Stage[SF_Amplification].bVisible = Stage[SF_Amplification].bVisible || (!bDenyAS && HasVisibility(CurrentParameter.ShaderVisibility, D3D12_SHADER_VISIBILITY_AMPLIFICATION));
#endif

			// Compute is a special case, it must have visibility all.
			Stage[SF_Compute].bVisible = Stage[SF_Compute].bVisible || (CurrentParameter.ShaderVisibility == D3D12_SHADER_VISIBILITY_ALL);
		}

		// Determine shader resource counts.
		{
			switch (CurrentParameter.ParameterType)
			{
			case D3D12_ROOT_PARAMETER_TYPE_DESCRIPTOR_TABLE:
				check(CurrentParameter.DescriptorTable.NumDescriptorRanges == 1);	// Code currently assumes a single descriptor range.
				{
					const auto& CurrentRange = CurrentParameter.DescriptorTable.pDescriptorRanges[0];
					check(CurrentRange.BaseShaderRegister == 0);	// Code currently assumes always starting at register 0.
					check(CurrentRange.RegisterSpace == BindingSpace); // Parameters in other binding spaces are expected to be filtered out at this point

					switch (CurrentRange.RangeType)
					{
					case D3D12_DESCRIPTOR_RANGE_TYPE_SRV:
						SetMaxSRVCount(CurrentVisibleSF, CurrentRange.NumDescriptors);
						SetSRVRDTBindSlot(CurrentVisibleSF, i);
						break;
					case D3D12_DESCRIPTOR_RANGE_TYPE_UAV:
						SetMaxUAVCount(CurrentVisibleSF, CurrentRange.NumDescriptors);
						SetUAVRDTBindSlot(CurrentVisibleSF, i);
						break;
					case D3D12_DESCRIPTOR_RANGE_TYPE_CBV:
						IncrementMaxCBVCount(CurrentVisibleSF, CurrentRange.NumDescriptors);
						SetCBVRDTBindSlot(CurrentVisibleSF, i);
						UpdateCBVRegisterMaskWithDescriptorRange(CurrentVisibleSF, CurrentRange);
						break;
					case D3D12_DESCRIPTOR_RANGE_TYPE_SAMPLER:
						SetMaxSamplerCount(CurrentVisibleSF, CurrentRange.NumDescriptors);
						SetSamplersRDTBindSlot(CurrentVisibleSF, i);
						break;

					default: check(false); break;
					}
				}
				break;

			case D3D12_ROOT_PARAMETER_TYPE_CBV:
			{
				check(CurrentParameter.Descriptor.RegisterSpace == BindingSpace); // Parameters in other binding spaces are expected to be filtered out at this point

				IncrementMaxCBVCount(CurrentVisibleSF, 1);
				if (CurrentParameter.Descriptor.ShaderRegister == 0)
				{
					// This is the first CBV for this stage, save it's root parameter index (other CBVs will be indexed using this base root parameter index).
					SetCBVRDBindSlot(CurrentVisibleSF, i);
				}

				UpdateCBVRegisterMaskWithDescriptor(CurrentVisibleSF, CurrentParameter.Descriptor);

				// The first CBV for this stage must come first in the root signature, and subsequent root CBVs for this stage must be contiguous.
				check(InvalidBindSlotMapIndex != CBVRDBindSlot(CurrentVisibleSF, 0));
				check(i == CBVRDBindSlot(CurrentVisibleSF, 0) + CurrentParameter.Descriptor.ShaderRegister);
			}
			break;

			default:
				// Need to update this for the other types. Currently we only use descriptor tables in the root signature.
				check(false);
				break;
			}
		}
	}
}

void FD3D12RootSignatureManager::Destroy()
{
	for (auto Iter = RootSignatureMap.CreateIterator(); Iter; ++Iter)
	{
		FD3D12RootSignature* pRootSignature = Iter.Value();
		delete pRootSignature;
	}
	RootSignatureMap.Reset();
}

FD3D12RootSignature* FD3D12RootSignatureManager::GetRootSignature(const FD3D12QuantizedBoundShaderState& QBSS)
{
	// Creating bound shader states happens in parallel, so this must be thread safe.
	FScopeLock Lock(&CS);

	FD3D12RootSignature** ppRootSignature = RootSignatureMap.Find(QBSS);
	if (ppRootSignature == nullptr)
	{
		// Create a new root signature and return it.
		return CreateRootSignature(QBSS);
	}

	check(*ppRootSignature);
	return *ppRootSignature;
}

FD3D12RootSignature* FD3D12RootSignatureManager::CreateRootSignature(const FD3D12QuantizedBoundShaderState& QBSS)
{
	// Create a desc and the root signature.
	FD3D12RootSignature* pNewRootSignature = new FD3D12RootSignature(GetParentAdapter(), QBSS);
	check(pNewRootSignature);

	// Add the index to the map.
	RootSignatureMap.Add(QBSS, pNewRootSignature);

	return pNewRootSignature;
}

FD3D12QuantizedBoundShaderState FD3D12RootSignatureManager::GetQuantizedBoundShaderState(const FD3D12RootSignature* const RootSignature)
{
	FScopeLock Lock(&CS);

	const FD3D12QuantizedBoundShaderState* QBSS = RootSignatureMap.FindKey(const_cast<FD3D12RootSignature*>(RootSignature));
	check(QBSS);

	return *QBSS;
}