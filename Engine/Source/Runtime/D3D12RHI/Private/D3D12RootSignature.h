// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RootSignature.h: D3D12 Root Signatures
=============================================================================*/

#pragma once

#include "D3D12RootSignatureDefinitions.h"
#include "D3D12Util.h"

// Root parameter keys grouped by visibility.
enum ERootParameterKeys
{
	PS_SRVs,
	PS_CBVs,
	PS_RootCBVs,
	PS_Samplers,
	VS_SRVs,
	VS_CBVs,
	VS_RootCBVs,
	VS_Samplers,
	VS_UAVs,
	GS_SRVs,
	GS_CBVs,
	GS_RootCBVs,
	GS_Samplers,
#if PLATFORM_SUPPORTS_MESH_SHADERS
	MS_SRVs,
	MS_CBVs,
	MS_RootCBVs,
	MS_Samplers,
	MS_UAVs,
	AS_SRVs,
	AS_CBVs,
	AS_RootCBVs,
	AS_Samplers,
	AS_UAVs,
#endif
	ALL_SRVs,
	ALL_CBVs,
	ALL_RootCBVs,
	ALL_Samplers,
	ALL_UAVs, // non-VS stages (PS, CS, etc.)
	RPK_RootParameterKeyCount,
};

class FD3D12RootSignature;

class FD3D12RootSignatureDesc
{
public:
	explicit FD3D12RootSignatureDesc(const FD3D12QuantizedBoundShaderState& QBSS, const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier);

	inline const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& GetDesc() const { return RootDesc; }

	static constexpr uint32 MaxRootParameters = 32;	// Arbitrary max, increase as needed.

	inline int8 GetRootConstantsSlot() const { return RootConstantsSlot; }
	inline int8 GetDiagnosticBufferSlot() const { return DiagnosticBufferSlot; }
	inline int8 GetStaticShaderBindingSlot() const { return StaticShaderBindingSlot; }
	inline int8 GetStaticShaderBindingCount() const { return StaticShaderBindingCount; }

private:

	uint32 RootParametersSize;	// The size of all root parameters in the root signature. Size in DWORDs, the limit is 64.
	int8 RootConstantsSlot = -1;
	int8 DiagnosticBufferSlot = -1;
	int8 StaticShaderBindingSlot = -1;
	int8 StaticShaderBindingCount = -1;
	CD3DX12_ROOT_PARAMETER1 TableSlots[MaxRootParameters];
	CD3DX12_DESCRIPTOR_RANGE1 DescriptorRanges[MaxRootParameters];
	CD3DX12_VERSIONED_ROOT_SIGNATURE_DESC RootDesc;
};

class FD3D12RootSignature : public FD3D12AdapterChild
{
private:
	// Struct for all the useful info we want per shader stage.
	struct ShaderStage
	{
		// TODO: Make these arrays and index into them by type instead of individual variables.
		uint8 MaxCBVCount = 0;
		uint8 MaxSRVCount = 0;
		uint8 MaxSamplerCount = 0;
		uint8 MaxUAVCount = 0;
		CBVSlotMask CBVRegisterMask = 0;
		bool bVisible = false;
	};

public:
	explicit FD3D12RootSignature(FD3D12Adapter* InParent)
		: FD3D12AdapterChild(InParent)
	{}
	explicit FD3D12RootSignature(FD3D12Adapter* InParent, const FD3D12QuantizedBoundShaderState& InQBSS)
		: FD3D12AdapterChild(InParent)
	{
		Init(InQBSS);
	}

	void Init(const FD3D12QuantizedBoundShaderState& InQBSS);
	void Init(const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& InDesc, ERootSignatureType InRootSignatureType);

	void InitStaticGraphicsRootSignature(EShaderBindingLayoutFlags InFlags);
	void InitStaticComputeRootSignatureDesc(EShaderBindingLayoutFlags InFlags);
#if D3D12_RHI_RAYTRACING
	void InitStaticRayTracingGlobalRootSignatureDesc(EShaderBindingLayoutFlags InFlags);
	void InitStaticRayTracingLocalRootSignatureDesc(EShaderBindingLayoutFlags InFlags);
#endif

	ID3D12RootSignature* GetRootSignature() const { return RootSignature.GetReference(); }
	ID3DBlob* GetRootSignatureBlob() const { return RootSignatureBlob.GetReference(); }

	inline uint32 SamplerRDTBindSlot(EShaderFrequency ShaderStage) const
	{
		switch (ShaderStage)
		{
		case SF_Vertex: return BindSlotMap[VS_Samplers];
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return BindSlotMap[MS_Samplers];
		case SF_Amplification: return BindSlotMap[AS_Samplers];
#endif
		case SF_Pixel: return BindSlotMap[PS_Samplers];
		case SF_Geometry: return BindSlotMap[GS_Samplers];
		case SF_Compute: return BindSlotMap[ALL_Samplers];

		default: check(false);
			return UINT_MAX;
		}
	}

	inline uint32 SRVRDTBindSlot(EShaderFrequency ShaderStage) const
	{
		switch (ShaderStage)
		{
		case SF_Vertex: return BindSlotMap[VS_SRVs];
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return BindSlotMap[MS_SRVs];
		case SF_Amplification: return BindSlotMap[AS_SRVs];
#endif
		case SF_Pixel: return BindSlotMap[PS_SRVs];
		case SF_Geometry: return BindSlotMap[GS_SRVs];
		case SF_Compute: return BindSlotMap[ALL_SRVs];

		default: check(false);
			return UINT_MAX;
		}
	}

	inline uint32 CBVRDTBindSlot(EShaderFrequency ShaderStage) const
	{
		switch (ShaderStage)
		{
		case SF_Vertex: return BindSlotMap[VS_CBVs];
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return BindSlotMap[MS_CBVs];
		case SF_Amplification: return BindSlotMap[AS_CBVs];
#endif
		case SF_Pixel: return BindSlotMap[PS_CBVs];
		case SF_Geometry: return BindSlotMap[GS_CBVs];
		case SF_Compute: return BindSlotMap[ALL_CBVs];

		default: check(false);
			return UINT_MAX;
		}
	}

	inline uint32 CBVRDBaseBindSlot(EShaderFrequency ShaderStage) const
	{
		switch (ShaderStage)
		{
		case SF_Vertex: return BindSlotMap[VS_RootCBVs];
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return BindSlotMap[MS_RootCBVs];
		case SF_Amplification: return BindSlotMap[AS_RootCBVs];
#endif
		case SF_Pixel: return BindSlotMap[PS_RootCBVs];
		case SF_Geometry: return BindSlotMap[GS_RootCBVs];

		case SF_NumFrequencies:
		case SF_Compute: return BindSlotMap[ALL_RootCBVs];

		default: check(false);
			return UINT_MAX;
		}
	}

	inline uint32 CBVRDBindSlot(EShaderFrequency ShaderStage, uint32 BufferIndex) const
	{
		// This code assumes that all Root CBVs for a particular stage are contiguous in the root signature (thus indexable by the buffer index).
		return CBVRDBaseBindSlot(ShaderStage) + BufferIndex;
	}

	inline uint32 UAVRDTBindSlot(EShaderFrequency ShaderStage) const
	{
		switch (ShaderStage)
		{
		case SF_Vertex: return BindSlotMap[VS_UAVs];
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: return BindSlotMap[MS_UAVs];
		case SF_Amplification: return BindSlotMap[AS_UAVs];
#endif
		case SF_Pixel: return BindSlotMap[ALL_UAVs];

		case SF_NumFrequencies:
		case SF_Compute: return BindSlotMap[ALL_UAVs];

		default: check(false);
			return UINT_MAX;
		}
	}

	inline static bool IsValidBindSlot(uint32 BindSlotMapIndex)
	{
		return BindSlotMapIndex < InvalidBindSlotMapIndex;
	}

	inline bool HasUAVs() const { return bHasUAVs; }
	inline bool HasSRVs() const { return bHasSRVs; }
	inline bool HasCBVs() const { return bHasCBVs; }
	inline bool HasRootCBs() const { return bHasRootCBs; }
	inline bool HasTableResources() const { return bHasUAVs || bHasSRVs; }
	inline bool HasTableConstants() const { return bHasCBVs; }
	inline bool HasSamplers() const { return bHasSamplers; }

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	bool UsesDynamicResources() const { return bUsesDynamicResources; }
	bool UsesDynamicSamplers() const { return bUsesDynamicSamplers; }
#else
	constexpr bool UsesDynamicResources() const { return false; }
	constexpr bool UsesDynamicSamplers() const { return false; }
#endif

	inline bool HasVS() const { return Stage[SF_Vertex].bVisible; }
	inline bool HasMS() const { return Stage[SF_Mesh].bVisible; }
	inline bool HasAS() const { return Stage[SF_Amplification].bVisible; }
	inline bool HasGS() const { return Stage[SF_Geometry].bVisible; }
	inline bool HasPS() const { return Stage[SF_Pixel].bVisible; }
	inline bool HasCS() const { return Stage[SF_Compute].bVisible; }	// Root signatures can be used for Graphics and/or Compute because they exist in separate bind spaces.
	inline uint32 MaxSamplerCount(uint32 ShaderStage) const { check(ShaderStage != SF_NumFrequencies); return Stage[ShaderStage].MaxSamplerCount; }
	inline uint32 MaxSRVCount(uint32 ShaderStage) const { check(ShaderStage != SF_NumFrequencies); return Stage[ShaderStage].MaxSRVCount; }
	inline uint32 MaxCBVCount(uint32 ShaderStage) const { check(ShaderStage != SF_NumFrequencies); return Stage[ShaderStage].MaxCBVCount; }
	inline uint32 MaxUAVCount(uint32 ShaderStage) const { check(ShaderStage != SF_NumFrequencies); return Stage[ShaderStage].MaxUAVCount; }
	inline CBVSlotMask CBVRegisterMask(uint32 ShaderStage) const { check(ShaderStage != SF_NumFrequencies); return Stage[ShaderStage].CBVRegisterMask; }

	uint32 GetBindSlotOffsetInBytes(uint8 BindSlotIndex) const { check(BindSlotIndex < UE_ARRAY_COUNT(BindSlotOffsetsInDWORDs)); return 4 * BindSlotOffsetsInDWORDs[BindSlotIndex]; }
	uint32 GetTotalRootSignatureSizeInBytes() const { return 4 * TotalRootSignatureSizeInDWORDs; }

	inline int8 GetRootConstantsSlot() const { return RootConstantsSlot; }
	inline int8 GetStaticShaderBindingSlot() const { return StaticShaderBindingSlot; }
	inline int8 GetStaticShaderBindingCount() const { return StaticShaderBindingCount; }

	// Returns root parameter slot for the internal shader diagnostic buffer (used for asserts, etc.) or -1 if not available.
	inline int8 GetDiagnosticBufferSlot() const { return DiagnosticBufferSlot; }

private:
	void AnalyzeSignature(const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& Desc, ERootSignatureType InRootSignatureType);

	template<typename RootSignatureDescType>
	void InternalAnalyzeSignature(const RootSignatureDescType& Desc, ERootSignatureType InRootSignatureType);

	inline bool HasVisibility(const D3D12_SHADER_VISIBILITY& ParameterVisibility, const D3D12_SHADER_VISIBILITY& Visibility) const
	{
		return ParameterVisibility == D3D12_SHADER_VISIBILITY_ALL || ParameterVisibility == Visibility;
	}

	inline void SetSamplersRDTBindSlot(EShaderFrequency SF, uint8 RootParameterIndex)
	{
		uint8* pBindSlot = nullptr;
		switch (SF)
		{
		case SF_Vertex: pBindSlot = &BindSlotMap[VS_Samplers]; break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: pBindSlot = &BindSlotMap[MS_Samplers]; break;
		case SF_Amplification: pBindSlot = &BindSlotMap[AS_Samplers]; break;
#endif
		case SF_Pixel: pBindSlot = &BindSlotMap[PS_Samplers]; break;
		case SF_Geometry: pBindSlot = &BindSlotMap[GS_Samplers]; break;

		case SF_Compute:
		case SF_NumFrequencies: pBindSlot = &BindSlotMap[ALL_Samplers]; break;

		default: check(false);
			return;
		}

		check(*pBindSlot == InvalidBindSlotMapIndex);
		*pBindSlot = RootParameterIndex;

		bHasSamplers = true;
	}

	inline void SetSRVRDTBindSlot(EShaderFrequency SF, uint8 RootParameterIndex)
	{
		uint8* pBindSlot = nullptr;
		switch (SF)
		{
		case SF_Vertex: pBindSlot = &BindSlotMap[VS_SRVs]; break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: pBindSlot = &BindSlotMap[MS_SRVs]; break;
		case SF_Amplification: pBindSlot = &BindSlotMap[AS_SRVs]; break;
#endif
		case SF_Pixel: pBindSlot = &BindSlotMap[PS_SRVs]; break;
		case SF_Geometry: pBindSlot = &BindSlotMap[GS_SRVs]; break;

		case SF_Compute:
		case SF_NumFrequencies: pBindSlot = &BindSlotMap[ALL_SRVs]; break;

		default: check(false);
			return;
		}

		check(*pBindSlot == InvalidBindSlotMapIndex);
		*pBindSlot = RootParameterIndex;

		bHasSRVs = true;
	}

	inline void SetCBVRDTBindSlot(EShaderFrequency SF, uint8 RootParameterIndex)
	{
		uint8* pBindSlot = nullptr;
		switch (SF)
		{
		case SF_Vertex: pBindSlot = &BindSlotMap[VS_CBVs]; break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: pBindSlot = &BindSlotMap[MS_CBVs]; break;
		case SF_Amplification: pBindSlot = &BindSlotMap[AS_CBVs]; break;
#endif
		case SF_Pixel: pBindSlot = &BindSlotMap[PS_CBVs]; break;
		case SF_Geometry: pBindSlot = &BindSlotMap[GS_CBVs]; break;

		case SF_Compute:
		case SF_NumFrequencies: pBindSlot = &BindSlotMap[ALL_CBVs]; break;

		default: check(false);
			return;
		}

		check(*pBindSlot == InvalidBindSlotMapIndex);
		*pBindSlot = RootParameterIndex;

		bHasCBVs = true;
	}

	inline void SetCBVRDBindSlot(EShaderFrequency SF, uint8 RootParameterIndex)
	{
		uint8* pBindSlot = nullptr;
		switch (SF)
		{
		case SF_Vertex: pBindSlot = &BindSlotMap[VS_RootCBVs]; break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: pBindSlot = &BindSlotMap[MS_RootCBVs]; break;
		case SF_Amplification: pBindSlot = &BindSlotMap[AS_RootCBVs]; break;
#endif
		case SF_Pixel: pBindSlot = &BindSlotMap[PS_RootCBVs]; break;
		case SF_Geometry: pBindSlot = &BindSlotMap[GS_RootCBVs]; break;

		case SF_Compute:
		case SF_NumFrequencies: pBindSlot = &BindSlotMap[ALL_RootCBVs]; break;

		default: check(false);
			return;
		}

		check(*pBindSlot == InvalidBindSlotMapIndex);
		*pBindSlot = RootParameterIndex;

		bHasRootCBs = true;
	}

	inline void SetUAVRDTBindSlot(EShaderFrequency SF, uint8 RootParameterIndex)
	{
		uint8* pBindSlot = nullptr;
		switch (SF)
		{
		case SF_Vertex: pBindSlot = &BindSlotMap[VS_UAVs]; break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh: pBindSlot = &BindSlotMap[MS_UAVs]; break;
		case SF_Amplification: pBindSlot = &BindSlotMap[AS_UAVs]; break;
#endif
		case SF_Pixel: pBindSlot = &BindSlotMap[ALL_UAVs]; break;

		case SF_Compute:
		case SF_NumFrequencies: pBindSlot = &BindSlotMap[ALL_UAVs]; break;

		default: check(false);
			return;
		}

		check(*pBindSlot == InvalidBindSlotMapIndex);
		*pBindSlot = RootParameterIndex;

		bHasUAVs = true;
	}

	inline void SetMaxSamplerCount(EShaderFrequency SF, uint8 Count)
	{
		if (SF == SF_NumFrequencies)
		{
			// Update all counts for all stages.
			for (uint32 s = SF_Vertex; s <= SF_Compute; s++)
			{
				Stage[s].MaxSamplerCount = Count;
			}
		}
		else
		{
			Stage[SF].MaxSamplerCount = Count;
		}
	}

	inline void SetMaxSRVCount(EShaderFrequency SF, uint8 Count)
	{
		if (SF == SF_NumFrequencies)
		{
			// Update all counts for all stages.
			for (uint32 s = SF_Vertex; s <= SF_Compute; s++)
			{
				Stage[s].MaxSRVCount = Count;
			}
		}
		else
		{
			Stage[SF].MaxSRVCount = Count;
		}
	}

	// Update the mask that indicates what shader registers are used in the descriptor table.
	template<typename DescriptorRangeType>
	inline void UpdateCBVRegisterMaskWithDescriptorRange(EShaderFrequency SF, const DescriptorRangeType& Range)
	{
		const uint32 StartRegister = Range.BaseShaderRegister;
		const uint32 EndRegister = StartRegister + Range.NumDescriptors;
		const uint32 StartStage = (SF == SF_NumFrequencies) ? SF_Vertex : SF;
		const uint32 EndStage = (SF == SF_NumFrequencies) ? SF_Compute : SF;
		for (uint32 CurrentStage = StartStage; CurrentStage <= EndStage; CurrentStage++)
		{
			for (uint32 Register = StartRegister; Register < EndRegister; Register++)
			{
				// The bit shouldn't already be set for the current register.
				check((Stage[CurrentStage].CBVRegisterMask & (1 << Register)) == 0); 
				Stage[CurrentStage].CBVRegisterMask |= (1 << Register);
			}
		}
	}

	// Update the mask that indicates what shader registers are used in the root descriptor.
	template<typename DescriptorType>
	inline void UpdateCBVRegisterMaskWithDescriptor(EShaderFrequency SF, const DescriptorType& Descriptor)
	{
		const uint32 StartStage = (SF == SF_NumFrequencies) ? SF_Vertex : SF;
		const uint32 EndStage = (SF == SF_NumFrequencies) ? SF_Compute : SF;
		const uint32& Register = Descriptor.ShaderRegister;
		for (uint32 CurrentStage = StartStage; CurrentStage <= EndStage; CurrentStage++)
		{
			// The bit shouldn't already be set for the current register.
			check((Stage[CurrentStage].CBVRegisterMask & (1 << Register)) == 0); 
			Stage[CurrentStage].CBVRegisterMask |= (1 << Register);
		}
	}

	inline void SetMaxCBVCount(EShaderFrequency SF, uint8 Count)
	{
		if (SF == SF_NumFrequencies)
		{
			// Update all counts for all stages.
			for (uint32 s = SF_Vertex; s <= SF_Compute; s++)
			{
				Stage[s].MaxCBVCount = Count;
			}
		}
		else
		{
			Stage[SF].MaxCBVCount = Count;
		}
	}

	inline void IncrementMaxCBVCount(EShaderFrequency SF, uint8 Count)
	{
		if (SF == SF_NumFrequencies)
		{
			// Update all counts for all stages.
			for (uint32 s = SF_Vertex; s <= SF_Compute; s++)
			{
				Stage[s].MaxCBVCount += Count;
			}
		}
		else
		{
			Stage[SF].MaxCBVCount += Count;
		}
	}

	inline void SetMaxUAVCount(EShaderFrequency SF, uint8 Count)
	{
		if (SF == SF_NumFrequencies)
		{
			// Update all counts for all stages.
			for (uint32 s = SF_Vertex; s <= SF_Compute; s++)
			{
				Stage[s].MaxUAVCount = Count;
			}
		}
		else
		{
			Stage[SF].MaxUAVCount = Count;
		}
	}

	TRefCountPtr<ID3D12RootSignature> RootSignature;
	uint8 BindSlotMap[RPK_RootParameterKeyCount];	// This map uses an enum as a key to lookup the root parameter index
	static constexpr uint8 InvalidBindSlotMapIndex = 0xFF;
	ShaderStage Stage[SF_NumFrequencies];
	TRefCountPtr<ID3DBlob> RootSignatureBlob;

	uint8 BindSlotOffsetsInDWORDs[FD3D12RootSignatureDesc::MaxRootParameters] = {};
	uint8 TotalRootSignatureSizeInDWORDs = 0;
	int8 RootConstantsSlot = -1;
	int8 DiagnosticBufferSlot = -1;
	int8 StaticShaderBindingSlot = -1;
	int8 StaticShaderBindingCount = -1;

	uint8 bHasUAVs : 1;
	uint8 bHasSRVs : 1;
	uint8 bHasCBVs : 1;
	uint8 bHasRootCBs : 1;
	uint8 bHasSamplers : 1;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	uint8 bUsesDynamicResources : 1;
	uint8 bUsesDynamicSamplers : 1;
#endif
};

class FD3D12RootSignatureManager : public FD3D12AdapterChild
{
public:
	explicit FD3D12RootSignatureManager(FD3D12Adapter* InParent)
		: FD3D12AdapterChild(InParent)
	{
	}
	~FD3D12RootSignatureManager()
	{
		check(RootSignatureMap.Num() == 0);
	}
	void Destroy();

	FD3D12RootSignature* GetRootSignature(const FD3D12QuantizedBoundShaderState& QBSS);
	FD3D12QuantizedBoundShaderState GetQuantizedBoundShaderState(const FD3D12RootSignature* const RootSignature);

private:
	FCriticalSection CS;
	FD3D12RootSignature* CreateRootSignature(const FD3D12QuantizedBoundShaderState& QBSS);

	TMap<FD3D12QuantizedBoundShaderState, FD3D12RootSignature*> RootSignatureMap;
};