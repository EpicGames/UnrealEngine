// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RHIPrivate.h"
#include "D3D12RayTracing.h"

FD3D12ShaderResourceView::FD3D12ShaderResourceView(FD3D12Device* InDevice, FD3D12ShaderResourceView* FirstLinkedObject, ERHIDescriptorType InDescriptorType, FD3D12RayTracingScene* InRayTracingScene)
	: TD3D12View(InDevice, InDescriptorType, ERHIDescriptorHeapType::Standard, FirstLinkedObject)
	, RayTracingScene(InRayTracingScene)
{
}

void FD3D12ShaderResourceView::UpdateResourceInfo(const FResourceInfo& InResource, const D3D12_SHADER_RESOURCE_VIEW_DESC& InD3DViewDesc, EFlags InFlags)
{
	OffsetInBytes = 0;
	StrideInBytes = 0;
	Flags = InFlags;

	//
	// Buffer / acceleration structure views can apply an offset in bytes from the start of the logical resource.
	//
	// Reconstruct this value and store it for later. We'll need it if the view is renamed, to determine where
	// the view should exist within the bounds of the new resource location.
	//
	if (InD3DViewDesc.ViewDimension == D3D12_SRV_DIMENSION_BUFFER)
	{
		StrideInBytes = InD3DViewDesc.Format == DXGI_FORMAT_UNKNOWN
			? InD3DViewDesc.Buffer.StructureByteStride
			: UE::DXGIUtilities::GetFormatSizeInBytes(InD3DViewDesc.Format);

		check(StrideInBytes > 0);

		OffsetInBytes = (InD3DViewDesc.Buffer.FirstElement * StrideInBytes) - InResource.ResourceLocation->GetOffsetFromBaseOfResource();
		check((OffsetInBytes % StrideInBytes) == 0);
	}
#if D3D12_RHI_RAYTRACING
	else if (InD3DViewDesc.ViewDimension == D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE)
	{
		OffsetInBytes = InD3DViewDesc.RaytracingAccelerationStructure.Location - InResource.ResourceLocation->GetGPUVirtualAddress();
		StrideInBytes = 1;
	}
#endif
}

void FD3D12ShaderResourceView::CreateView(FResourceInfo const& InResource, D3D12_SHADER_RESOURCE_VIEW_DESC const& InD3DViewDesc, EFlags InFlags)
{
	UpdateResourceInfo(InResource, InD3DViewDesc, InFlags);
	TD3D12View::CreateView(InResource, InD3DViewDesc);
}

void FD3D12ShaderResourceView::UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, const D3D12_SHADER_RESOURCE_VIEW_DESC& InD3DViewDesc, EFlags InFlags)
{
	UpdateResourceInfo(InResource, InD3DViewDesc, InFlags);
	TD3D12View::UpdateView(Contexts, InResource, InD3DViewDesc);
}

void FD3D12ShaderResourceView::ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation)
{
	check(IsInitialized());

	//
	// Buffer SRV descriptors contain offsets / GPU virtual addresses which need to be updated to match the new resource location.
	// Use the values we saved during intialization to find the start of the viewed data at the new location.
	//

	if (D3DViewDesc.ViewDimension == D3D12_SRV_DIMENSION_BUFFER)
	{
		D3DViewDesc.Buffer.FirstElement = (InNewResourceLocation->GetOffsetFromBaseOfResource() + OffsetInBytes) / StrideInBytes;
	}
#if D3D12_RHI_RAYTRACING
	else if (D3DViewDesc.ViewDimension == D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE)
	{
		D3DViewDesc.RaytracingAccelerationStructure.Location = InNewResourceLocation->GetOffsetFromBaseOfResource() + OffsetInBytes;
	}
#endif

	TD3D12View::ResourceRenamed(Contexts, InRenamedResource, InNewResourceLocation);
}

void FD3D12ShaderResourceView::UpdateMinLODClamp(FD3D12ContextArray const& Contexts, float MinLODClamp)
{
	check(IsInitialized());

	FLOAT* pResourceMinLODClamp = nullptr;

	switch (D3DViewDesc.ViewDimension)
	{
	default: checkNoEntry(); return; // not supported
	case D3D12_SRV_DIMENSION_TEXTURE2D       : pResourceMinLODClamp = &D3DViewDesc.Texture2D       .ResourceMinLODClamp; break;
	case D3D12_SRV_DIMENSION_TEXTURE2DARRAY  : pResourceMinLODClamp = &D3DViewDesc.Texture2DArray  .ResourceMinLODClamp; break;
	case D3D12_SRV_DIMENSION_TEXTURE3D       : pResourceMinLODClamp = &D3DViewDesc.Texture3D       .ResourceMinLODClamp; break;
	case D3D12_SRV_DIMENSION_TEXTURECUBE     : pResourceMinLODClamp = &D3DViewDesc.TextureCube     .ResourceMinLODClamp; break;
	case D3D12_SRV_DIMENSION_TEXTURECUBEARRAY: pResourceMinLODClamp = &D3DViewDesc.TextureCubeArray.ResourceMinLODClamp; break;
	}

	if (pResourceMinLODClamp && *pResourceMinLODClamp != MinLODClamp)
	{
		*pResourceMinLODClamp = MinLODClamp;

		UpdateDescriptor();
		UpdateBindlessSlot(Contexts);
	}
}

void FD3D12ShaderResourceView::UpdateDescriptor()
{
#if D3D12_RHI_RAYTRACING
	// NOTE (from D3D Debug runtime): pResource must be NULL for acceleration structures, since the resource location comes from a GPUVA in pDesc.
	ID3D12Resource* TargetResource = D3DViewDesc.ViewDimension != D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE
		? GetResource()->GetResource()
		: nullptr;
#else
	ID3D12Resource* TargetResource = GetResource()->GetResource();
#endif

	GetParentDevice()->GetDevice()->CreateShaderResourceView(
		TargetResource,
		&D3DViewDesc,
		OfflineCpuHandle
	);

	OfflineCpuHandle.IncrementVersion();
}

static FD3D12ShaderResourceView::EFlags TranslateDesc(D3D12_SHADER_RESOURCE_VIEW_DESC& SRVDesc, FD3D12Buffer* Buffer, const FRHIViewDesc::FBufferSRV::FViewInfo& Info)
{
	if (!Info.bNullView)
	{
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;

		if (Info.BufferType == FRHIViewDesc::EBufferType::AccelerationStructure)
		{
#if D3D12_RHI_RAYTRACING
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE;
			SRVDesc.Format = DXGI_FORMAT_UNKNOWN;

			SRVDesc.RaytracingAccelerationStructure.Location = Info.OffsetInBytes + Buffer->ResourceLocation.GetGPUVirtualAddress();
#else
			UE_LOGF(LogD3D12RHI, Fatal, "Raytracing not implemented.");
#endif
		}
		else
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
			SRVDesc.Format = UE::DXGIUtilities::FindShaderResourceFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat), false);
			SRVDesc.Buffer.FirstElement = (Info.OffsetInBytes + Buffer->ResourceLocation.GetOffsetFromBaseOfResource()) / Info.StrideInBytes;
			SRVDesc.Buffer.NumElements = Info.NumElements;

			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Raw:
				SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
				SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;
				break;

			case FRHIViewDesc::EBufferType::Structured:
				SRVDesc.Buffer.StructureByteStride = Info.StrideInBytes;
				break;

			case FRHIViewDesc::EBufferType::Typed:
				// Nothing more to specify
				break;
			}
		}
	}

	return FD3D12ShaderResourceView::EFlags::None;
}

static FD3D12ShaderResourceView::EFlags TranslateDesc(D3D12_SHADER_RESOURCE_VIEW_DESC& SRVDesc, FD3D12Texture* Texture, const FRHIViewDesc::FTextureSRV::FViewInfo& Info)
{
	FRHITextureDesc const& TextureDesc = Texture->GetDesc();

	DXGI_FORMAT const ViewFormat = UE::DXGIUtilities::FindShaderResourceFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat), Info.bSRGB);
	DXGI_FORMAT const BaseFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat(DXGI_FORMAT(GPixelFormats[TextureDesc.Format].PlatformFormat), TextureDesc.Flags);

	SRVDesc.Format = ViewFormat;
	SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;

	uint32 const PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(BaseFormat, ViewFormat);
	FRHIRange8 const PlaneRange(PlaneSlice, 1);
	FRHIViewDesc::EDimension ViewDimension = UE::RHICore::AdjustViewInfoDimensionForNarrowing(Info, TextureDesc);
	switch (ViewDimension)
	{
	case FRHIViewDesc::EDimension::Texture2D:
		if (TextureDesc.NumSamples > 1)
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DMS;
		}
		else
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2D;
			SRVDesc.Texture2D.MostDetailedMip = Info.MipRange.First;
			SRVDesc.Texture2D.MipLevels = Info.MipRange.Num;
			SRVDesc.Texture2D.PlaneSlice = PlaneSlice;
		}
		break;

	case FRHIViewDesc::EDimension::Texture2DArray:
		if (TextureDesc.NumSamples > 1)
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DMSARRAY;
			SRVDesc.Texture2DMSArray.FirstArraySlice = Info.ArrayRange.First;
			SRVDesc.Texture2DMSArray.ArraySize = Info.ArrayRange.Num;
		}
		else
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DARRAY;
			SRVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First;
			SRVDesc.Texture2DArray.ArraySize = Info.ArrayRange.Num;
			SRVDesc.Texture2DArray.MostDetailedMip = Info.MipRange.First;
			SRVDesc.Texture2DArray.MipLevels = Info.MipRange.Num;
			SRVDesc.Texture2DArray.PlaneSlice = PlaneSlice;
		}
		break;

	case FRHIViewDesc::EDimension::Texture3D:
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE3D;
		SRVDesc.Texture3D.MostDetailedMip = Info.MipRange.First;
		SRVDesc.Texture3D.MipLevels = Info.MipRange.Num;
		break;

	case FRHIViewDesc::EDimension::TextureCube:
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURECUBE;
		SRVDesc.TextureCube.MostDetailedMip = Info.MipRange.First;
		SRVDesc.TextureCube.MipLevels = Info.MipRange.Num;
		break;

	case FRHIViewDesc::EDimension::TextureCubeArray:
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURECUBEARRAY;
		SRVDesc.TextureCubeArray.MostDetailedMip = Info.MipRange.First;
		SRVDesc.TextureCubeArray.MipLevels = Info.MipRange.Num;
		SRVDesc.TextureCubeArray.First2DArrayFace = Info.ArrayRange.First * 6;
		SRVDesc.TextureCubeArray.NumCubes = Info.ArrayRange.Num;
		break;

	default:
		checkNoEntry();
		break;
	}

	FD3D12ShaderResourceView::EFlags Flags = FD3D12ShaderResourceView::EFlags::None;
	if (Texture->SkipsFastClearFinalize())
	{
		Flags |= FD3D12ShaderResourceView::EFlags::SkipFastClearFinalize;
	}

	return Flags;
}

void FD3D12ShaderResourceView_RHI::CreateView()
{
	if (IsBuffer())
	{
		FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(GetBuffer());

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
		const EFlags CreateFlags = TranslateDesc(SRVDesc, Buffer, ViewDesc.Buffer.SRV.GetViewInfo(Buffer));

		FD3D12ShaderResourceView::CreateView(Buffer, SRVDesc, CreateFlags);
	}
	else
	{
		FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(GetTexture());

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
		const EFlags CreateFlags = TranslateDesc(SRVDesc, Texture, ViewDesc.Texture.SRV.GetViewInfo(Texture));

		FD3D12ShaderResourceView::CreateView(Texture, SRVDesc, CreateFlags);
	}
}

void FD3D12ShaderResourceView_RHI::UpdateView(FD3D12ContextArray const& Contexts)
{
	if (IsBuffer())
	{
		FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(GetBuffer());

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
		const EFlags CreateFlags = TranslateDesc(SRVDesc, Buffer, ViewDesc.Buffer.SRV.GetViewInfo(Buffer));

		FD3D12ShaderResourceView::UpdateView(Contexts, Buffer, SRVDesc, CreateFlags);
	}
	else
	{
		FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(GetTexture());

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc{};
		const EFlags CreateFlags = TranslateDesc(SRVDesc, Texture, ViewDesc.Texture.SRV.GetViewInfo(Texture));

		FD3D12ShaderResourceView::UpdateView(Contexts, Texture, SRVDesc, CreateFlags);
	}
}

inline FD3D12RayTracingScene* GetRayTracingSceneFromViewDesc(const FRHIViewDesc& InViewDesc)
{
#if RHI_RAYTRACING
	if (InViewDesc.Buffer.SRV.BufferType == FRHIViewDesc::EBufferType::AccelerationStructure)
	{
		return FD3D12DynamicRHI::ResourceCast(InViewDesc.Buffer.SRV.RayTracingScene);
	}
#endif

	return nullptr;
}

FD3D12ShaderResourceView_RHI::FD3D12ShaderResourceView_RHI(FD3D12Device* InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc, FD3D12ShaderResourceView_RHI* FirstLinkedObject)
	: FRHIShaderResourceView(InResource, InViewDesc)
	, FD3D12ShaderResourceView(InDevice, FirstLinkedObject, RHIDescriptorTypeFromViewDesc(InViewDesc), GetRayTracingSceneFromViewDesc(InViewDesc))
{
}

// -----------------------------------------------------------------------------------------------------
//
//                                            RHI Functions                                             
//
// -----------------------------------------------------------------------------------------------------

FShaderResourceViewRHIRef FD3D12DynamicRHI::RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	FRHIGPUMask RelevantGPUs = ViewDesc.IsBuffer()
		? FD3D12DynamicRHI::ResourceCast(static_cast<FRHIBuffer* >(Resource))->GetLinkedObjectsGPUMask()
		: FD3D12DynamicRHI::ResourceCast(static_cast<FRHITexture*>(Resource))->GetLinkedObjectsGPUMask();

	FD3D12ShaderResourceView_RHI* View = GetAdapter().CreateLinkedObject<FD3D12ShaderResourceView_RHI>(RelevantGPUs, [&](FD3D12Device* Device, FD3D12ShaderResourceView_RHI* FirstLinkedObject)
	{
		FRHIViewableResource* TargetResource = ViewDesc.IsBuffer()
			? static_cast<FRHIViewableResource*>(FD3D12DynamicRHI::ResourceCast(static_cast<FRHIBuffer* >(Resource), Device->GetGPUIndex()))
			: static_cast<FRHIViewableResource*>(FD3D12DynamicRHI::ResourceCast(static_cast<FRHITexture*>(Resource), Device->GetGPUIndex()));

		return new FD3D12ShaderResourceView_RHI(Device, TargetResource, ViewDesc, FirstLinkedObject);
	});

	View->CreateViews(RHICmdList);
	return View;
}
