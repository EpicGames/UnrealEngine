// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Shader.h: D3D12 Shaders
=============================================================================*/

#pragma once

#include "D3D12RHICommon.h"
#include "RHIResources.h"
#include "ShaderCore.h"
#include "Templates/UnrealTypeTraits.h"

class FD3D12RootSignature;

template <>
struct TTypeTraits<D3D12_INPUT_ELEMENT_DESC> : public TTypeTraitsBase < D3D12_INPUT_ELEMENT_DESC >
{
	enum { IsBytewiseComparable = true };
};

/** Convenience typedef: preallocated array of D3D12 input element descriptions. */
typedef TArray<D3D12_INPUT_ELEMENT_DESC, TFixedAllocator<MaxVertexElementCount> > FD3D12VertexElements;

/** This represents a vertex declaration that hasn't been combined with a specific shader to create a bound shader. */
class FD3D12VertexDeclaration : public FRHIVertexDeclaration
{
public:
	/** Elements of the vertex declaration. */
	FD3D12VertexElements VertexElements;

	TStaticArray<uint16, MaxVertexElementCount> StreamStrides;
	uint32 Hash;
	uint32 HashNoStrides;

	/** Initialization constructor. */
	explicit FD3D12VertexDeclaration(const FD3D12VertexElements& InElements, const uint16* InStrides, const uint32 InHash, const uint32 InHashNoStrides)
		: VertexElements(InElements)
		, Hash(InHash)
		, HashNoStrides(InHashNoStrides)
	{
		FMemory::Memcpy(StreamStrides.GetData(), InStrides, StreamStrides.Num() * sizeof(StreamStrides[0]));
	}

	virtual bool GetInitializer(FVertexDeclarationElementList& Init) final override;
	virtual uint32 GetPrecachePSOHash() const final override { return HashNoStrides;  }
};

//==================================================================================================================================
// FD3D12ShaderBytecode
// Encapsulates D3D12 shader bytecode and creates a hash for the shader bytecode
//==================================================================================================================================
struct ShaderBytecodeHash
{
	uint64 Hash[2];

	bool operator ==(const ShaderBytecodeHash& b) const
	{
		return (Hash[0] == b.Hash[0] && Hash[1] == b.Hash[1]);
	}

	bool operator !=(const ShaderBytecodeHash& b) const
	{
		return (Hash[0] != b.Hash[0] || Hash[1] != b.Hash[1]);
	}
};

struct FD3D12ShaderData
{
	/** The shader's bytecode, with custom data in the last byte. */
	TArray<uint8> Code;

	FShaderCodePackedResourceCounts ResourceCounts{};
		
	// Hash of the static shader binding layout used during shader gen
	uint32 ShaderBindingLayoutHash = 0;

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	TArray<FShaderCodeVendorExtension> VendorExtensions;
#endif

#if D3D12RHI_NEEDS_SHADER_FEATURE_CHECKS
	EShaderCodeFeatures Features = EShaderCodeFeatures::None;
#endif

	D3D12_SHADER_BYTECODE GetShaderBytecode() const
	{
		return CD3DX12_SHADER_BYTECODE(Code.GetData(), Code.Num());
	}

	ShaderBytecodeHash GetBytecodeHash() const
	{
		ShaderBytecodeHash Hash;
		if (Code.Num() == 0)
		{
			Hash.Hash[0] = Hash.Hash[1] = 0;
		}
		else
		{
			// D3D shader bytecode contains a 128bit checksum in DWORD 1-4. We can just use that directly instead of hashing the whole shader bytecode ourselves.
			const uint8* pData = Code.GetData() + 4;
			Hash = *reinterpret_cast<const ShaderBytecodeHash*>(pData);
		}
		return Hash;
	}

#if D3D12RHI_NEEDS_SHADER_FEATURE_CHECKS
	FORCEINLINE EShaderCodeFeatures GetFeatures() const { return Features; }
#else
	FORCEINLINE EShaderCodeFeatures GetFeatures() const { return EShaderCodeFeatures::None; }
#endif

	FORCEINLINE bool UsesDiagnosticBuffer() const { return EnumHasAnyFlags(ResourceCounts.UsageFlags, EShaderResourceUsageFlags::DiagnosticBuffer); }
	FORCEINLINE bool UsesGlobalUniformBuffer() const { return EnumHasAnyFlags(ResourceCounts.UsageFlags, EShaderResourceUsageFlags::GlobalUniformBuffer); }
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FORCEINLINE bool UsesBindlessResources() const { return EnumHasAnyFlags(ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources); }
	FORCEINLINE bool UsesBindlessSamplers() const { return EnumHasAnyFlags(ResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessSamplers); }
#endif
	FORCEINLINE bool UsesRootConstants() const { return EnumHasAnyFlags(ResourceCounts.UsageFlags, EShaderResourceUsageFlags::RootConstants); }

	bool InitCommon(TArrayView<const uint8> InCode);
};

/** This represents a vertex shader that hasn't been combined with a specific declaration to create a bound shader. */
class FD3D12VertexShader : public FRHIVertexShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Vertex };
};

class FD3D12MeshShader : public FRHIMeshShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Mesh };
};

class FD3D12AmplificationShader : public FRHIAmplificationShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Amplification };
};

class FD3D12GeometryShader : public FRHIGeometryShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Geometry };
};

class FD3D12PixelShader : public FRHIPixelShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Pixel };

#if D3D12_RHI_WORKGRAPHS_GRAPHICS
	FString EntryPoint;
#endif
};

class FD3D12ComputeShader : public FRHIComputeShader, public FD3D12ShaderData
{
public:
	enum { StaticFrequency = SF_Compute };

	const FD3D12RootSignature* RootSignature = nullptr;
};

class FD3D12WorkGraphShader : public FRHIWorkGraphShader, public FD3D12ShaderData
{
public:
	explicit FD3D12WorkGraphShader(EShaderFrequency InFrequency) : FRHIWorkGraphShader(InFrequency) {}

	const FD3D12RootSignature* RootSignature = nullptr;

	FString EntryPoint;
};

#if D3D12_RHI_RAYTRACING

class FD3D12RayTracingShader : public FRHIRayTracingShader, public FD3D12ShaderData
{
public:
	explicit FD3D12RayTracingShader(EShaderFrequency InFrequency) : FRHIRayTracingShader(InFrequency) {}

	const FD3D12RootSignature* LocalRootSignature = nullptr;

	/** The shader's DXIL entrypoint & base export name for DXR (required for RTPSO creation) */
	FString EntryPoint; // Primary entry point for all ray tracing shaders. Assumed to be closest hit shader for SF_RayHitGroup.
	FString AnyHitEntryPoint; // Optional any-hit shader entry point for SF_RayHitGroup.
	FString IntersectionEntryPoint; // Optional intersection shader entry point for SF_RayHitGroup.
	bool bPrecompiledPSO = false;
};

#endif // D3D12_RHI_RAYTRACING

template<>
struct TD3D12ResourceTraits<FRHIVertexShader>
{
	typedef FD3D12VertexShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIMeshShader>
{
	typedef FD3D12MeshShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIAmplificationShader>
{
	typedef FD3D12AmplificationShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIGeometryShader>
{
	typedef FD3D12GeometryShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIPixelShader>
{
	typedef FD3D12PixelShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIComputeShader>
{
	typedef FD3D12ComputeShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIWorkGraphShader>
{
	typedef FD3D12WorkGraphShader TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIVertexDeclaration>
{
	typedef FD3D12VertexDeclaration TConcreteType;
};
