// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Shaders.cpp: D3D shader RHI implementation.
=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "RHICoreShader.h"

// We don't store EntryPoint name in all shader types to avoid memory bloat.
// Use template specialization to define which shader types need this.
template <typename TShaderType>
constexpr bool CanReadEntryPoint(const TShaderType& InShader) { return false; }
template <typename TShaderType>
static void ReadEntryPoint(FBufferReader& InReader, TShaderType& OutShader) {}

// Work graph shaders need to store EntryPoint for state object linking.
constexpr bool CanReadEntryPoint(const FD3D12WorkGraphShader& InShader) { return true; }
static void ReadEntryPoint(FBufferReader& InReader, FD3D12WorkGraphShader& OutShader) { InReader << OutShader.EntryPoint; }

// Pixels shaders need to store EntryPoint whenever work graphs are supported because they can be used in work graph graphics nodes.
#if D3D12_RHI_WORKGRAPHS_GRAPHICS
constexpr bool CanReadEntryPoint(const FD3D12PixelShader& InShader) { return true; }
static void ReadEntryPoint(FBufferReader& InReader, FD3D12PixelShader& OutShader) { InReader << OutShader.EntryPoint; }
#endif

template <typename TShaderType>
static inline bool ReadShaderOptionalData(FShaderCodeReader& InShaderCode, TShaderType& OutShader)
{
	const FShaderCodePackedResourceCounts* PackedResourceCounts = InShaderCode.FindOptionalData<FShaderCodePackedResourceCounts>();
	if (!PackedResourceCounts)
	{
		return false;
	}
	OutShader.ResourceCounts = *PackedResourceCounts;

#if RHI_INCLUDE_SHADER_DEBUG_DATA
	OutShader.Debug.ShaderName = InShaderCode.FindOptionalData(FShaderCodeName::Key);

	int32 UniformBufferTableSize = 0;
	const uint8* UniformBufferData = InShaderCode.FindOptionalDataAndSize(FShaderCodeUniformBuffers::Key, UniformBufferTableSize);
	if (UniformBufferData && UniformBufferTableSize > 0)
	{
		FBufferReader UBReader((void*)UniformBufferData, UniformBufferTableSize, false);
		TArray<FString> Names;
		UBReader << Names;
		check(OutShader.Debug.UniformBufferNames.Num() == 0);
		for (int32 Index = 0; Index < Names.Num(); ++Index)
		{
			OutShader.Debug.UniformBufferNames.Add(FName(*Names[Index]));
		}
	}
#endif

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	int32 VendorExtensionTableSize = 0;
	auto* VendorExtensionData = InShaderCode.FindOptionalDataAndSize(FShaderCodeVendorExtension::Key, VendorExtensionTableSize);
	if (VendorExtensionData && VendorExtensionTableSize > 0)
	{
		FBufferReader Ar((void*)VendorExtensionData, VendorExtensionTableSize, false);
		Ar << OutShader.VendorExtensions;

		// Reconstruct bIsOptional for NVIDIA extensions.
		for (FShaderCodeVendorExtension& Extension : OutShader.VendorExtensions)
		{
			if (Extension.VendorId == EGpuVendorId::Nvidia && Extension.Parameter.Type == EShaderParameterType::UAV)
			{
				Extension.bIsOptional = true;
			}
		}
	}
#endif

#if D3D12RHI_NEEDS_SHADER_FEATURE_CHECKS
	if (const FShaderCodeFeatures* CodeFeatures = InShaderCode.FindOptionalData<FShaderCodeFeatures>())
	{
		OutShader.Features = CodeFeatures->CodeFeatures;
	}
#endif

	int32 ShaderBindingLayoutSize = 0;
	auto* ShaderBindingLayoutData = InShaderCode.FindOptionalDataAndSize(FShaderCodeShaderResourceTableDataDesc::Key, ShaderBindingLayoutSize);
	if (ShaderBindingLayoutData && ShaderBindingLayoutSize > 0)
	{
		check(ShaderBindingLayoutSize == sizeof(OutShader.ShaderBindingLayoutHash));
		FBufferReader Ar((void*)ShaderBindingLayoutData, ShaderBindingLayoutSize, false);
		Ar << OutShader.ShaderBindingLayoutHash;
	}

	if (CanReadEntryPoint(OutShader))
	{
		int32 NameSize = 0;
		uint8 const* NameData = InShaderCode.FindOptionalDataAndSize(EShaderOptionalDataKey::EntryPoint, NameSize);
		if (NameData && NameSize > 0)
		{
			FBufferReader Reader((void*)NameData, NameSize, false);
			ReadEntryPoint(Reader, OutShader);
		}
	}

	UE::RHICore::SetupShaderCodeValidationData(&OutShader, InShaderCode);
	UE::RHICore::SetupShaderDiagnosticData(&OutShader, InShaderCode);

	return true;
}

static bool ValidateShaderIsUsable(FD3D12ShaderData* InShader, EShaderFrequency InFrequency)
{
#if D3D12RHI_NEEDS_SHADER_FEATURE_CHECKS
	if ((InFrequency == SF_Mesh || InFrequency == SF_Amplification) && !GRHISupportsMeshShadersTier0)
	{
		UE_LOGF(LogD3D12RHI, Log, "Trying to create Mesh or Amplication shader but RHI doesn't support MeshShaders");
		return false;
	}

	// When we're using the SM5 shader library plus raytracing, GRHISupportsWaveOperations is false because DXBC shaders can't do wave ops,
	// but RT shaders are compiled to DXIL and can use wave ops (HW support is guaranteed too).
	if (EnumHasAnyFlags(InShader->Features, EShaderCodeFeatures::WaveOps) && !GRHISupportsWaveOperations && !IsRayTracingShaderFrequency(InFrequency))
	{
		UE_LOGF(LogD3D12RHI, Log, "Trying to create shader with WaveOps but RHI doesn't support WaveOperations");
		return false;
	}

	if (EnumHasAnyFlags(InShader->Features, EShaderCodeFeatures::BindlessResources | EShaderCodeFeatures::BindlessSamplers))
	{
		if (!GRHIGlobals.bSupportsBindless)
		{
			UE_LOGF(LogD3D12RHI, Log, "Trying to create shader with bindless resources or samplers but RHI doesn't support Bindless");
			return false;
		}
	}

	if (InFrequency == SF_Pixel && EnumHasAnyFlags(InShader->Features, EShaderCodeFeatures::StencilRef) && !GRHISupportsStencilRefFromPixelShader)
	{
		UE_LOGF(LogD3D12RHI, Log, "Trying to create pixel shader with stencil ref but RHI doesn't support StencilRefFromPixelShader");
		return false;
	}

	if (EnumHasAnyFlags(InShader->Features, EShaderCodeFeatures::BarycentricsSemantic) && !GRHIGlobals.SupportsBarycentricsSemantic)
	{
		UE_LOGF(LogD3D12RHI, Log, "Trying to create shader with BarycentricsSemantic but RHI doesn't support BarycentricsSemantic");
		return false;
	}
#endif

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	// At the time that this code was added, we only have AMD, Nvidia and Intel extensions and GetRHIDeviceVendorId handles all these vendors.
	const EGpuVendorId CurrentVendorId = GetRHIDeviceVendorId();

	for (const FShaderCodeVendorExtension& Extension : InShader->VendorExtensions)
	{
		if (Extension.VendorId != CurrentVendorId)
		{
			if (Extension.bIsOptional)
			{
				continue;
			}

			return false;
		}
	}
#endif

	return true;
}

template <typename TShaderType>
bool InitShaderCommon(FShaderCodeReader& ShaderCode, int32 Offset, TShaderType* InShader)
{
	if (!ReadShaderOptionalData(ShaderCode, *InShader))
	{
		return false;
	}

	if (!ValidateShaderIsUsable(InShader, InShader->GetFrequency()))
	{
		return false;
	}

	// Copy the native shader data only, skipping any of our own headers.
	InShader->Code = ShaderCode.GetOffsetShaderCode(Offset);
	InShader->SetShaderBundleUsage(EnumHasAnyFlags(InShader->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::ShaderBundle));

	return true;
}

template <typename TShaderType, typename LambdaType>
TShaderType* InitStandardShaderWithCustomSerialization(TShaderType* InShader, TArrayView<const uint8> InCode, const LambdaType& CustomSerializationLambda)
{
	FShaderCodeReader ShaderCode(InCode);

	FMemoryReaderView Ar(InCode, true);
	InShader->SerializeShaderResourceTable(Ar);

	int32 Offset = Ar.Tell();

	CustomSerializationLambda(Ar, InShader, Offset);

	if (!InitShaderCommon(ShaderCode, Offset, InShader))
	{
		InShader->AddRef();
		InShader->Release();
		return nullptr;
	}

	UE::RHICore::InitStaticUniformBufferSlots(InShader);

	return InShader;
}

template <typename TShaderType>
TShaderType* InitStandardShader(TShaderType* InShader, TArrayView<const uint8> InCode)
{
	auto CustomSerialization = [](FMemoryReaderView&, TShaderType*, int32&) {};
	return InitStandardShaderWithCustomSerialization<TShaderType>(InShader, InCode, CustomSerialization);
}

FVertexShaderRHIRef FD3D12DynamicRHI::RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return InitStandardShader(new FD3D12VertexShader(), CreateShaderDesc.Code);
}

FMeshShaderRHIRef FD3D12DynamicRHI::RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return InitStandardShader(new FD3D12MeshShader(), CreateShaderDesc.Code);
}

FAmplificationShaderRHIRef FD3D12DynamicRHI::RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return InitStandardShader(new FD3D12AmplificationShader(), CreateShaderDesc.Code);
}

FPixelShaderRHIRef FD3D12DynamicRHI::RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return InitStandardShader(new FD3D12PixelShader(), CreateShaderDesc.Code);
}

FGeometryShaderRHIRef FD3D12DynamicRHI::RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	return InitStandardShader(new FD3D12GeometryShader(), CreateShaderDesc.Code);
}

FComputeShaderRHIRef FD3D12DynamicRHI::RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
	FD3D12ComputeShader* Shader = InitStandardShader(new FD3D12ComputeShader(), CreateShaderDesc.Code);
	if (Shader)
	{
		Shader->RootSignature = GetAdapter().GetRootSignature(Shader);
		Shader->SetNoDerivativeOps(EnumHasAnyFlags(Shader->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::NoDerivativeOps));
	}

	return Shader;
}

FWorkGraphShaderRHIRef FD3D12DynamicRHI::RHICreateWorkGraphShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	FD3D12WorkGraphShader* Shader = InitStandardShader(new FD3D12WorkGraphShader(ShaderFrequency), CreateShaderDesc.Code);
	if (Shader)
	{
		Shader->RootSignature = GetAdapter().GetRootSignature(Shader);
		Shader->SetNoDerivativeOps(EnumHasAnyFlags(Shader->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::NoDerivativeOps));
	}

	return Shader;
}

#if D3D12_RHI_RAYTRACING

FRayTracingShaderRHIRef FD3D12DynamicRHI::RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	checkf(GRHISupportsRayTracing && GRHISupportsRayTracingShaders, TEXT("Tried to create RayTracing shader but RHI doesn't support it!"));

	auto CustomSerialization = [this, ShaderFrequency](FMemoryReaderView& Ar, FD3D12RayTracingShader* Shader, int32& Offset)
	{
		Ar << Shader->EntryPoint;
		Ar << Shader->AnyHitEntryPoint;
		Ar << Shader->IntersectionEntryPoint;
		Ar << Shader->RayTracingPayloadType;
		Ar << Shader->RayTracingPayloadSize;
	
		checkf(Shader->RayTracingPayloadType != 0, TEXT("Ray Tracing Shader must not have an empty payload type!"));
		checkf((FMath::CountBits(Shader->RayTracingPayloadType) == 1 && (ShaderFrequency == SF_RayHitGroup || ShaderFrequency == SF_RayMiss || ShaderFrequency == SF_RayCallable)) ||
			(FMath::CountBits(Shader->RayTracingPayloadType) >= 1 && (ShaderFrequency == SF_RayGen)),
			TEXT("Ray Tracing Shader has %d bits set, which is not the expected count for shader frequency %d"), FMath::CountBits(Shader->RayTracingPayloadType), int(ShaderFrequency)
		);

		Offset = Ar.Tell();

		int32 PrecompiledKey = 0;
		Ar << PrecompiledKey;
		if (PrecompiledKey == RayTracingPrecompiledPSOKey)
		{
			Offset += sizeof(PrecompiledKey); // Skip the precompiled PSO marker if it's present
			Shader->bPrecompiledPSO = true;
		}
	};

	FD3D12RayTracingShader* Shader = InitStandardShaderWithCustomSerialization(new FD3D12RayTracingShader(ShaderFrequency), CreateShaderDesc.Code, CustomSerialization);

	// Setup local root signature (RayGen only has global root signature)
	if (Shader && ShaderFrequency != SF_RayGen)
	{
		Shader->LocalRootSignature = GetAdapter().GetLocalRootSignature(Shader);
		Shader->LocalBindingDataSize = Shader->LocalRootSignature->GetTotalRootSignatureSizeInBytes();
	}

	return Shader;
}

#endif // D3D12_RHI_RAYTRACING

FShaderBundleRHIRef FD3D12DynamicRHI::RHICreateShaderBundle(const FShaderBundleCreateInfo& CreateInfo)
{
	FD3D12ShaderBundle* ShaderBundle = new FD3D12ShaderBundle(GetRHIDevice(0), CreateInfo);
	return ShaderBundle;
}

void FD3D12CommandContext::RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data)
{
	// Structures are chosen to be directly mappable
	StateCache.SetViewports(Count, reinterpret_cast<const D3D12_VIEWPORT*>(Data));
}

FBoundShaderStateRHIRef FD3D12DynamicRHI::RHICreateBoundShaderState(
	FRHIVertexDeclaration* VertexDeclarationRHI,
	FRHIVertexShader* VertexShaderRHI,
	FRHIPixelShader* PixelShaderRHI,
	FRHIGeometryShader* GeometryShaderRHI
	)
{
	checkNoEntry();
	return nullptr;
}
