// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12State.cpp: D3D state implementation.
	=============================================================================*/

#include "D3D12State.h"
#include "D3D12RHIPrivate.h"
#include "RHIUtilities.h"

// MSFT: Need to make sure sampler state is thread safe
// Cache of Sampler States; we store pointers to both as we don't want the TMap to be artificially
// modifying ref counts if not needed; so we manage that ourselves
FCriticalSection GD3D12SamplerStateCacheLock;

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Graphics: Find or Create time"), STAT_PSOGraphicsFindOrCreateTime, STATGROUP_D3D12PipelineState, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Compute: Find or Create time"), STAT_PSOComputeFindOrCreateTime, STATGROUP_D3D12PipelineState, EStatFlags::Verbose);

static D3D12_TEXTURE_ADDRESS_MODE TranslateAddressMode(ESamplerAddressMode AddressMode)
{
	switch (AddressMode)
	{
	case AM_Clamp: return D3D12_TEXTURE_ADDRESS_MODE_CLAMP;
	case AM_Mirror: return D3D12_TEXTURE_ADDRESS_MODE_MIRROR;
	case AM_Border: return D3D12_TEXTURE_ADDRESS_MODE_BORDER;
	default: return D3D12_TEXTURE_ADDRESS_MODE_WRAP;
	};
}

static D3D12_CULL_MODE TranslateCullMode(ERasterizerCullMode CullMode)
{
	switch (CullMode)
	{
	case CM_CW: return D3D12_CULL_MODE_BACK;
	case CM_CCW: return D3D12_CULL_MODE_FRONT;
	default: return D3D12_CULL_MODE_NONE;
	};
}

static ERasterizerCullMode ReverseTranslateCullMode(D3D12_CULL_MODE CullMode)
{
	switch (CullMode)
	{
	case D3D12_CULL_MODE_BACK: return CM_CW;
	case D3D12_CULL_MODE_FRONT: return CM_CCW;
	default: return CM_None;
	}
}

static D3D12_FILL_MODE TranslateFillMode(ERasterizerFillMode FillMode)
{
	switch (FillMode)
	{
	case FM_Wireframe: return D3D12_FILL_MODE_WIREFRAME;
	default: return D3D12_FILL_MODE_SOLID;
	};
}

static ERasterizerFillMode ReverseTranslateFillMode(D3D12_FILL_MODE FillMode)
{
	switch (FillMode)
	{
	case D3D12_FILL_MODE_WIREFRAME: return FM_Wireframe;
	default: return FM_Solid;
	}
}

static D3D12_COMPARISON_FUNC TranslateCompareFunction(ECompareFunction CompareFunction)
{
	switch (CompareFunction)
	{
	case CF_Less: return D3D12_COMPARISON_FUNC_LESS;
	case CF_LessEqual: return D3D12_COMPARISON_FUNC_LESS_EQUAL;
	case CF_Greater: return D3D12_COMPARISON_FUNC_GREATER;
	case CF_GreaterEqual: return D3D12_COMPARISON_FUNC_GREATER_EQUAL;
	case CF_Equal: return D3D12_COMPARISON_FUNC_EQUAL;
	case CF_NotEqual: return D3D12_COMPARISON_FUNC_NOT_EQUAL;
	case CF_Never: return D3D12_COMPARISON_FUNC_NEVER;
	default: return D3D12_COMPARISON_FUNC_ALWAYS;
	};
}

static ECompareFunction ReverseTranslateCompareFunction(D3D12_COMPARISON_FUNC CompareFunction)
{
	switch (CompareFunction)
	{
	case D3D12_COMPARISON_FUNC_LESS: return CF_Less;
	case D3D12_COMPARISON_FUNC_LESS_EQUAL: return CF_LessEqual;
	case D3D12_COMPARISON_FUNC_GREATER: return CF_Greater;
	case D3D12_COMPARISON_FUNC_GREATER_EQUAL: return CF_GreaterEqual;
	case D3D12_COMPARISON_FUNC_EQUAL: return CF_Equal;
	case D3D12_COMPARISON_FUNC_NOT_EQUAL: return CF_NotEqual;
	case D3D12_COMPARISON_FUNC_NEVER: return CF_Never;
	default: return CF_Always;
	}
}

static D3D12_COMPARISON_FUNC TranslateSamplerCompareFunction(ESamplerCompareFunction SamplerComparisonFunction)
{
	switch (SamplerComparisonFunction)
	{
	case SCF_Less: return D3D12_COMPARISON_FUNC_LESS;
	case SCF_Never:
	default: return D3D12_COMPARISON_FUNC_NEVER;
	};
}

static D3D12_STENCIL_OP TranslateStencilOp(EStencilOp StencilOp)
{
	switch (StencilOp)
	{
	case SO_Zero: return D3D12_STENCIL_OP_ZERO;
	case SO_Replace: return D3D12_STENCIL_OP_REPLACE;
	case SO_SaturatedIncrement: return D3D12_STENCIL_OP_INCR_SAT;
	case SO_SaturatedDecrement: return D3D12_STENCIL_OP_DECR_SAT;
	case SO_Invert: return D3D12_STENCIL_OP_INVERT;
	case SO_Increment: return D3D12_STENCIL_OP_INCR;
	case SO_Decrement: return D3D12_STENCIL_OP_DECR;
	default: return D3D12_STENCIL_OP_KEEP;
	};
}

static EStencilOp ReverseTranslateStencilOp(D3D12_STENCIL_OP StencilOp)
{
	switch (StencilOp)
	{
	case D3D12_STENCIL_OP_ZERO: return SO_Zero;
	case D3D12_STENCIL_OP_REPLACE: return SO_Replace;
	case D3D12_STENCIL_OP_INCR_SAT: return SO_SaturatedIncrement;
	case D3D12_STENCIL_OP_DECR_SAT: return SO_SaturatedDecrement;
	case D3D12_STENCIL_OP_INVERT: return SO_Invert;
	case D3D12_STENCIL_OP_INCR: return SO_Increment;
	case D3D12_STENCIL_OP_DECR: return SO_Decrement;
	default: return SO_Keep;
	};
}

static D3D12_BLEND_OP TranslateBlendOp(EBlendOperation BlendOp)
{
	switch (BlendOp)
	{
	case BO_Subtract: return D3D12_BLEND_OP_SUBTRACT;
	case BO_Min: return D3D12_BLEND_OP_MIN;
	case BO_Max: return D3D12_BLEND_OP_MAX;
	case BO_ReverseSubtract: return D3D12_BLEND_OP_REV_SUBTRACT;
	default: return D3D12_BLEND_OP_ADD;
	};
}

static EBlendOperation ReverseTranslateBlendOp(D3D12_BLEND_OP BlendOp)
{
	switch (BlendOp)
	{
	case D3D12_BLEND_OP_SUBTRACT: return BO_Subtract;
	case D3D12_BLEND_OP_MIN: return BO_Min;
	case D3D12_BLEND_OP_MAX: return BO_Max;
	case D3D12_BLEND_OP_REV_SUBTRACT: return BO_ReverseSubtract;
	default: return BO_Add;
	};
}

static D3D12_BLEND TranslateBlendFactor(EBlendFactor BlendFactor)
{
	switch (BlendFactor)
	{
	case BF_One: return D3D12_BLEND_ONE;
	case BF_SourceColor: return D3D12_BLEND_SRC_COLOR;
	case BF_InverseSourceColor: return D3D12_BLEND_INV_SRC_COLOR;
	case BF_SourceAlpha: return D3D12_BLEND_SRC_ALPHA;
	case BF_InverseSourceAlpha: return D3D12_BLEND_INV_SRC_ALPHA;
	case BF_DestAlpha: return D3D12_BLEND_DEST_ALPHA;
	case BF_InverseDestAlpha: return D3D12_BLEND_INV_DEST_ALPHA;
	case BF_DestColor: return D3D12_BLEND_DEST_COLOR;
	case BF_InverseDestColor: return D3D12_BLEND_INV_DEST_COLOR;
	case BF_ConstantBlendFactor: return D3D12_BLEND_BLEND_FACTOR;
	case BF_InverseConstantBlendFactor: return D3D12_BLEND_INV_BLEND_FACTOR;
	case BF_Source1Color: return D3D12_BLEND_SRC1_COLOR;
	case BF_InverseSource1Color: return D3D12_BLEND_INV_SRC1_COLOR;
	case BF_Source1Alpha: return D3D12_BLEND_SRC1_ALPHA;
	case BF_InverseSource1Alpha: return D3D12_BLEND_INV_SRC1_ALPHA;
	default: return D3D12_BLEND_ZERO;
	};
}

static EBlendFactor ReverseTranslateBlendFactor(D3D12_BLEND BlendFactor)
{
	switch (BlendFactor)
	{
	case D3D12_BLEND_ONE: return BF_One;
	case D3D12_BLEND_SRC_COLOR: return BF_SourceColor;
	case D3D12_BLEND_INV_SRC_COLOR: return BF_InverseSourceColor;
	case D3D12_BLEND_SRC_ALPHA: return BF_SourceAlpha;
	case D3D12_BLEND_INV_SRC_ALPHA: return BF_InverseSourceAlpha;
	case D3D12_BLEND_DEST_ALPHA: return BF_DestAlpha;
	case D3D12_BLEND_INV_DEST_ALPHA: return BF_InverseDestAlpha;
	case D3D12_BLEND_DEST_COLOR: return BF_DestColor;
	case D3D12_BLEND_INV_DEST_COLOR: return BF_InverseDestColor;
	case D3D12_BLEND_BLEND_FACTOR: return BF_ConstantBlendFactor;
	case D3D12_BLEND_INV_BLEND_FACTOR: return BF_InverseConstantBlendFactor;
	case D3D12_BLEND_SRC1_COLOR: return BF_Source1Color;
	case D3D12_BLEND_INV_SRC1_COLOR: return BF_InverseSource1Color;
	case D3D12_BLEND_SRC1_ALPHA: return BF_Source1Alpha;
	case D3D12_BLEND_INV_SRC1_ALPHA: return BF_InverseSource1Alpha;
	default: return BF_Zero;
	};
}

bool operator==(const D3D12_SAMPLER_DESC& lhs, const D3D12_SAMPLER_DESC& rhs)
{
	return 0 == memcmp(&lhs, &rhs, sizeof(lhs));
}

uint32 GetTypeHash(const D3D12_SAMPLER_DESC& Desc)
{
	return Desc.Filter;
}

FSamplerStateRHIRef FD3D12DynamicRHI::RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	FD3D12Adapter* Adapter = &GetAdapter();

	return Adapter->CreateLinkedObject<FD3D12SamplerState>(FRHIGPUMask::All(), [&](FD3D12Device* Device, FD3D12SamplerState* FirstLinkedObject)
	{
		return Device->CreateSampler(Initializer, FirstLinkedObject);
	});
}

static void LogSamplerStateWarning(const FSamplerStateInitializerRHI& Initializer)
{
	UE_LOGF(LogD3D12RHI, Warning,
		"New SamplerState would exceed cache limit: FSamplerStateInitializerRHI(Filter: %d, AddressU: %d, AddressV: %d, AddressW: %d, MipBias: %f, MinMipLevel: %f, MaxMipLevel: %f, MaxAnisotropy: %d, BorderColor: %d, SamplerComparisonFunction: %d)." "An unreferenced SamplerState will be replaced. Try reducing r.ViewTextureMipBias.Quantization and reloading your project to reduce the number of unique samplers when Dynamic Resoluton is active.",
		Initializer.Filter.GetIntValue(),
		Initializer.AddressU.GetIntValue(),
		Initializer.AddressV.GetIntValue(),
		Initializer.AddressW.GetIntValue(),
		Initializer.MipBias,
		Initializer.MinMipLevel,
		Initializer.MaxMipLevel,
		Initializer.MaxAnisotropy,
		Initializer.BorderColor,
		Initializer.SamplerComparisonFunction.GetIntValue()
	);
}

FD3D12SamplerState* FD3D12Device::CreateSampler(const FSamplerStateInitializerRHI& Initializer, FD3D12SamplerState* FirstLinkedObject)
{
	D3D12_SAMPLER_DESC SamplerDesc;
	FMemory::Memzero(&SamplerDesc, sizeof(D3D12_SAMPLER_DESC));

	SamplerDesc.AddressU = TranslateAddressMode(Initializer.AddressU);
	SamplerDesc.AddressV = TranslateAddressMode(Initializer.AddressV);
	SamplerDesc.AddressW = TranslateAddressMode(Initializer.AddressW);
	SamplerDesc.MipLODBias = Initializer.MipBias;
	SamplerDesc.MaxAnisotropy = ComputeAnisotropyRT(Initializer.MaxAnisotropy);
	SamplerDesc.MinLOD = Initializer.MinMipLevel;
	SamplerDesc.MaxLOD = Initializer.MaxMipLevel;

	// Determine whether we should use one of the comparison modes
	const bool bComparisonEnabled = Initializer.SamplerComparisonFunction != SCF_Never;
	switch (Initializer.Filter)
	{
	case SF_AnisotropicLinear:
	case SF_AnisotropicPoint:
		if (SamplerDesc.MaxAnisotropy == 1)
		{
			SamplerDesc.Filter = bComparisonEnabled ? D3D12_FILTER_COMPARISON_MIN_MAG_MIP_LINEAR : D3D12_FILTER_MIN_MAG_MIP_LINEAR;
		}
		else
		{
			// D3D12  doesn't allow using point filtering for mip filter when using anisotropic filtering
			SamplerDesc.Filter = bComparisonEnabled ? D3D12_FILTER_COMPARISON_ANISOTROPIC : D3D12_FILTER_ANISOTROPIC;
		}

		break;
	case SF_Trilinear:
		SamplerDesc.Filter = bComparisonEnabled ? D3D12_FILTER_COMPARISON_MIN_MAG_MIP_LINEAR : D3D12_FILTER_MIN_MAG_MIP_LINEAR;
		break;
	case SF_Bilinear:
		SamplerDesc.Filter = bComparisonEnabled ? D3D12_FILTER_COMPARISON_MIN_MAG_LINEAR_MIP_POINT : D3D12_FILTER_MIN_MAG_LINEAR_MIP_POINT;
		break;
	case SF_Point:
		SamplerDesc.Filter = bComparisonEnabled ? D3D12_FILTER_COMPARISON_MIN_MAG_MIP_POINT : D3D12_FILTER_MIN_MAG_MIP_POINT;
		break;
	}
	const FLinearColor LinearBorderColor = FColor(Initializer.BorderColor);
	SamplerDesc.BorderColor[0] = LinearBorderColor.R;
	SamplerDesc.BorderColor[1] = LinearBorderColor.G;
	SamplerDesc.BorderColor[2] = LinearBorderColor.B;
	SamplerDesc.BorderColor[3] = LinearBorderColor.A;
	SamplerDesc.ComparisonFunc = TranslateSamplerCompareFunction(Initializer.SamplerComparisonFunction);

	QUICK_SCOPE_CYCLE_COUNTER(FD3D12DynamicRHI_RHICreateSamplerState_LockAndCreate);
	FScopeLock Lock(&GD3D12SamplerStateCacheLock);

	// Check to see if the sampler has already been created
	// This is done to reduce cache misses accessing sampler objects
	const TRefCountPtr<FD3D12SamplerState>* PreviouslyCreated = SamplerCache.FindAndTouch(SamplerDesc);
	if (PreviouslyCreated)
	{
		return PreviouslyCreated->GetReference();
	}
	else
	{
		// 16-bit IDs are used for faster hashing
		check(SamplerID < 0xffff);

		// If we're full and the least recent entry is still referenced, touch all referenced keys starting from the least recent until we find an unreferenced one
		// Samplers are generally added in the following order:
		//    1. Perennial samplers that will be used in many places and are never recreated (2-100+ references)
		//    2. Older material samplers that are no longer being used (1 reference, from the map itself)
		//    3. Recently created material samplers that are in use (2 references)
		// This process will move perennial samplers that were created before any extant material samplers to the top of the recently used list and ensure we have a sampler in the LRU slot that's safe to remove
		if (SamplerCache.Num() >= D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE)
		{
			LogSamplerStateWarning(Initializer);

			int32 EntriesTouched = 0;
			D3D12_SAMPLER_DESC CurrentKey = SamplerCache.GetLeastRecentKey();

			while (SamplerCache.Find(CurrentKey)->GetReference()->GetRefCount() > 1)
			{
				static_cast<void>(SamplerCache.FindAndTouch(CurrentKey)); // We don't need the return value, just to mark it as recently used
				CurrentKey = SamplerCache.GetLeastRecentKey();
				
				EntriesTouched++;
				checkf(EntriesTouched < D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE, TEXT("Attempted to create new SamplerState when cache is full, and no unreferenced entries are available for replacement."));
			}
			
			// Once we have the least recently used unreferenced entry, remove it
			// We need to free the descriptor (and possibly the bindless handle) immediately so we don't overshoot the limit while waiting for it to be deleted
			SamplerCache.Find(CurrentKey)->GetReference()->FreeDescriptor();
			SamplerCache.Remove(CurrentKey);
		}

		FD3D12SamplerState* NewSampler = new FD3D12SamplerState(this, SamplerDesc, static_cast<uint16>(SamplerID), FirstLinkedObject);
		SamplerCache.Add(SamplerDesc, NewSampler);

		SamplerID++;

		INC_DWORD_STAT(STAT_UniqueSamplers);

		return NewSampler;
	}
}

FRasterizerStateRHIRef FD3D12DynamicRHI::RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	FD3D12RasterizerState* RasterizerState = new FD3D12RasterizerState;

	D3D12_RASTERIZER_DESC& RasterizerDesc = RasterizerState->Desc;
	FMemory::Memzero(&RasterizerDesc, sizeof(D3D12_RASTERIZER_DESC));

	RasterizerDesc.CullMode = TranslateCullMode(Initializer.CullMode);
	RasterizerDesc.FillMode = TranslateFillMode(Initializer.FillMode);
	RasterizerDesc.SlopeScaledDepthBias = Initializer.SlopeScaleDepthBias;
	RasterizerDesc.FrontCounterClockwise = true;
	RasterizerDesc.DepthBias = FMath::FloorToInt(Initializer.DepthBias * (float)(1 << 24));
	RasterizerDesc.DepthClipEnable = Initializer.DepthClipMode == ERasterizerDepthClipMode::DepthClip;
	RasterizerDesc.MultisampleEnable = Initializer.bAllowMSAA;

	return RasterizerState;
}

bool FD3D12RasterizerState::GetInitializer(struct FRasterizerStateInitializerRHI& Init)
{
	Init.FillMode = ReverseTranslateFillMode(Desc.FillMode);
	Init.CullMode = ReverseTranslateCullMode(Desc.CullMode);
	Init.DepthBias = Desc.DepthBias / static_cast<float>(1 << 24);
	check(Desc.DepthBias == FMath::FloorToInt(Init.DepthBias * static_cast<float>(1 << 24)));
	Init.SlopeScaleDepthBias = Desc.SlopeScaledDepthBias;
	Init.bAllowMSAA = !!Desc.MultisampleEnable;
	return true;
}

FDepthStencilStateRHIRef FD3D12DynamicRHI::RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
	FD3D12DepthStencilState* DepthStencilState = new FD3D12DepthStencilState;

	D3D12_DEPTH_STENCIL_DESC1 &DepthStencilDesc = DepthStencilState->Desc;
	FMemory::Memzero(&DepthStencilDesc, sizeof(D3D12_DEPTH_STENCIL_DESC1));

	// depth part
	DepthStencilDesc.DepthEnable = Initializer.DepthTest != CF_Always || Initializer.bEnableDepthWrite;
	DepthStencilDesc.DepthWriteMask = Initializer.bEnableDepthWrite ? D3D12_DEPTH_WRITE_MASK_ALL : D3D12_DEPTH_WRITE_MASK_ZERO;
	DepthStencilDesc.DepthFunc = TranslateCompareFunction(Initializer.DepthTest);

	// stencil part
	DepthStencilDesc.StencilEnable = Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil;
	DepthStencilDesc.StencilReadMask = Initializer.StencilReadMask;
	DepthStencilDesc.StencilWriteMask = Initializer.StencilWriteMask;
	DepthStencilDesc.FrontFace.StencilFunc = TranslateCompareFunction(Initializer.FrontFaceStencilTest);
	DepthStencilDesc.FrontFace.StencilFailOp = TranslateStencilOp(Initializer.FrontFaceStencilFailStencilOp);
	DepthStencilDesc.FrontFace.StencilDepthFailOp = TranslateStencilOp(Initializer.FrontFaceDepthFailStencilOp);
	DepthStencilDesc.FrontFace.StencilPassOp = TranslateStencilOp(Initializer.FrontFacePassStencilOp);
	if (Initializer.bEnableBackFaceStencil)
	{
		DepthStencilDesc.BackFace.StencilFunc = TranslateCompareFunction(Initializer.BackFaceStencilTest);
		DepthStencilDesc.BackFace.StencilFailOp = TranslateStencilOp(Initializer.BackFaceStencilFailStencilOp);
		DepthStencilDesc.BackFace.StencilDepthFailOp = TranslateStencilOp(Initializer.BackFaceDepthFailStencilOp);
		DepthStencilDesc.BackFace.StencilPassOp = TranslateStencilOp(Initializer.BackFacePassStencilOp);
	}
	else
	{
		DepthStencilDesc.BackFace = DepthStencilDesc.FrontFace;
	}

	const bool bStencilOpIsKeep =
		Initializer.FrontFaceStencilFailStencilOp == SO_Keep
		&& Initializer.FrontFaceDepthFailStencilOp == SO_Keep
		&& Initializer.FrontFacePassStencilOp == SO_Keep
		&& Initializer.BackFaceStencilFailStencilOp == SO_Keep
		&& Initializer.BackFaceDepthFailStencilOp == SO_Keep
		&& Initializer.BackFacePassStencilOp == SO_Keep;

	const bool bMayWriteStencil = Initializer.StencilWriteMask != 0 && !bStencilOpIsKeep;
	DepthStencilState->AccessType.SetDepthStencilWrite(Initializer.bEnableDepthWrite, bMayWriteStencil);

	return DepthStencilState;
}

bool FD3D12DepthStencilState::GetInitializer(struct FDepthStencilStateInitializerRHI& Init)
{
	Init.bEnableDepthWrite = Desc.DepthWriteMask != D3D12_DEPTH_WRITE_MASK_ZERO;
	Init.DepthTest = ReverseTranslateCompareFunction(Desc.DepthFunc);
	Init.bEnableFrontFaceStencil = !!Desc.StencilEnable;
	Init.FrontFaceStencilTest = ReverseTranslateCompareFunction(Desc.FrontFace.StencilFunc);
	Init.FrontFaceStencilFailStencilOp = ReverseTranslateStencilOp(Desc.FrontFace.StencilFailOp);
	Init.FrontFaceDepthFailStencilOp = ReverseTranslateStencilOp(Desc.FrontFace.StencilDepthFailOp);
	Init.FrontFacePassStencilOp = ReverseTranslateStencilOp(Desc.FrontFace.StencilPassOp);
	Init.bEnableBackFaceStencil =
		Desc.StencilEnable &&
		(Desc.FrontFace.StencilFunc != Desc.BackFace.StencilFunc ||
			Desc.FrontFace.StencilFailOp != Desc.BackFace.StencilFailOp ||
			Desc.FrontFace.StencilDepthFailOp != Desc.BackFace.StencilDepthFailOp ||
			Desc.FrontFace.StencilPassOp != Desc.BackFace.StencilPassOp);
	Init.BackFaceStencilTest = ReverseTranslateCompareFunction(Desc.BackFace.StencilFunc);
	Init.BackFaceStencilFailStencilOp = ReverseTranslateStencilOp(Desc.BackFace.StencilFailOp);
	Init.BackFaceDepthFailStencilOp = ReverseTranslateStencilOp(Desc.BackFace.StencilDepthFailOp);
	Init.BackFacePassStencilOp = ReverseTranslateStencilOp(Desc.BackFace.StencilPassOp);
	Init.StencilReadMask = Desc.StencilReadMask;
	Init.StencilWriteMask = Desc.StencilWriteMask;
	return true;
}

FBlendStateRHIRef FD3D12DynamicRHI::RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
	FD3D12BlendState* BlendState = new FD3D12BlendState;
	D3D12_BLEND_DESC &BlendDesc = BlendState->Desc;
	FMemory::Memzero(&BlendDesc, sizeof(D3D12_BLEND_DESC));

	BlendDesc.AlphaToCoverageEnable = Initializer.bUseAlphaToCoverage;
	BlendDesc.IndependentBlendEnable = Initializer.bUseIndependentRenderTargetBlendStates;

	static_assert(MaxSimultaneousRenderTargets <= D3D12_SIMULTANEOUS_RENDER_TARGET_COUNT, "Too many MRTs.");
	for (uint32 RenderTargetIndex = 0; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		const FBlendStateInitializerRHI::FRenderTarget& RenderTargetInitializer = Initializer.RenderTargets[RenderTargetIndex];
		D3D12_RENDER_TARGET_BLEND_DESC& RenderTarget = BlendDesc.RenderTarget[RenderTargetIndex];
		RenderTarget.BlendEnable =
			RenderTargetInitializer.ColorBlendOp != BO_Add || RenderTargetInitializer.ColorDestBlend != BF_Zero || RenderTargetInitializer.ColorSrcBlend != BF_One ||
			RenderTargetInitializer.AlphaBlendOp != BO_Add || RenderTargetInitializer.AlphaDestBlend != BF_Zero || RenderTargetInitializer.AlphaSrcBlend != BF_One;
		RenderTarget.BlendOp = TranslateBlendOp(RenderTargetInitializer.ColorBlendOp);
		RenderTarget.SrcBlend = TranslateBlendFactor(RenderTargetInitializer.ColorSrcBlend);
		RenderTarget.DestBlend = TranslateBlendFactor(RenderTargetInitializer.ColorDestBlend);
		RenderTarget.BlendOpAlpha = TranslateBlendOp(RenderTargetInitializer.AlphaBlendOp);
		RenderTarget.SrcBlendAlpha = TranslateBlendFactor(RenderTargetInitializer.AlphaSrcBlend);
		RenderTarget.DestBlendAlpha = TranslateBlendFactor(RenderTargetInitializer.AlphaDestBlend);
		RenderTarget.RenderTargetWriteMask =
			((RenderTargetInitializer.ColorWriteMask & CW_RED) ? D3D12_COLOR_WRITE_ENABLE_RED : 0)
			| ((RenderTargetInitializer.ColorWriteMask & CW_GREEN) ? D3D12_COLOR_WRITE_ENABLE_GREEN : 0)
			| ((RenderTargetInitializer.ColorWriteMask & CW_BLUE) ? D3D12_COLOR_WRITE_ENABLE_BLUE : 0)
			| ((RenderTargetInitializer.ColorWriteMask & CW_ALPHA) ? D3D12_COLOR_WRITE_ENABLE_ALPHA : 0);
	}

	return BlendState;
}

bool FD3D12BlendState::GetInitializer(class FBlendStateInitializerRHI& Init)
{
	for (int32 Idx = 0; Idx < MaxSimultaneousRenderTargets; ++Idx)
	{
		const D3D12_RENDER_TARGET_BLEND_DESC& Src = Desc.RenderTarget[Idx];
		FBlendStateInitializerRHI::FRenderTarget& Dst = Init.RenderTargets[Idx];

		Dst.ColorBlendOp = ReverseTranslateBlendOp(Src.BlendOp);
		Dst.ColorSrcBlend = ReverseTranslateBlendFactor(Src.SrcBlend);
		Dst.ColorDestBlend = ReverseTranslateBlendFactor(Src.DestBlend);
		Dst.AlphaBlendOp = ReverseTranslateBlendOp(Src.BlendOpAlpha);
		Dst.AlphaSrcBlend = ReverseTranslateBlendFactor(Src.SrcBlendAlpha);
		Dst.AlphaDestBlend = ReverseTranslateBlendFactor(Src.DestBlendAlpha);
		Dst.ColorWriteMask = TEnumAsByte<EColorWriteMask>(
			((Src.RenderTargetWriteMask & D3D12_COLOR_WRITE_ENABLE_RED) ? CW_RED : 0)
			| ((Src.RenderTargetWriteMask & D3D12_COLOR_WRITE_ENABLE_GREEN) ? CW_GREEN : 0)
			| ((Src.RenderTargetWriteMask & D3D12_COLOR_WRITE_ENABLE_BLUE) ? CW_BLUE : 0)
			| ((Src.RenderTargetWriteMask & D3D12_COLOR_WRITE_ENABLE_ALPHA) ? CW_ALPHA : 0));
	}
	Init.bUseIndependentRenderTargetBlendStates = !!Desc.IndependentBlendEnable;
	Init.bUseAlphaToCoverage = !!Desc.AlphaToCoverageEnable;
	return true;
}

uint64 FD3D12DynamicRHI::RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	if (bDriverCacheAwarePSOPrecaching)
	{
		if (IsRHIDeviceNVIDIA())
		{
			// NVIDIA drivers only care about the shaders for PSO caching.
			struct FHashKey
			{
				uint32 VertexShader;
				uint32 PixelShader;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				uint32 GeometryShader;
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#if PLATFORM_SUPPORTS_MESH_SHADERS
				uint32 MeshShader;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
			} HashKey;

			FMemory::Memzero(&HashKey, sizeof(FHashKey));

			HashKey.VertexShader = Initializer.BoundShaderState.GetVertexShader() ? GetTypeHash(Initializer.BoundShaderState.GetVertexShader()->GetHash()) : 0;
			HashKey.PixelShader = Initializer.BoundShaderState.GetPixelShader() ? GetTypeHash(Initializer.BoundShaderState.GetPixelShader()->GetHash()) : 0;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			HashKey.GeometryShader = Initializer.BoundShaderState.GetGeometryShader() ? GetTypeHash(Initializer.BoundShaderState.GetGeometryShader()->GetHash()) : 0;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
			HashKey.MeshShader = Initializer.BoundShaderState.GetMeshShader() ? GetTypeHash(Initializer.BoundShaderState.GetMeshShader()->GetHash()) : 0;
#endif
			return CityHash64((const char*)&HashKey, sizeof(FHashKey));
		}
		else if (IsRHIDeviceIntel() && GMaxRHIFeatureLevel == ERHIFeatureLevel::SM6)
		{
			// Intel drivers have a few elements on top of the shaders that can cause PSO recompilation.
			struct FHashKey
			{
				uint32 VertexShader;
				uint32 PixelShader;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				uint32 GeometryShader;
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#if PLATFORM_SUPPORTS_MESH_SHADERS
				uint32 MeshShader;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS

				uint32 BlendState;
				uint8  MultisamplingEnabled;
			} HashKey;

			FMemory::Memzero(&HashKey, sizeof(FHashKey));

			HashKey.VertexShader = Initializer.BoundShaderState.GetVertexShader() ? GetTypeHash(Initializer.BoundShaderState.GetVertexShader()->GetHash()) : 0;
			HashKey.PixelShader = Initializer.BoundShaderState.GetPixelShader() ? GetTypeHash(Initializer.BoundShaderState.GetPixelShader()->GetHash()) : 0;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			HashKey.GeometryShader = Initializer.BoundShaderState.GetGeometryShader() ? GetTypeHash(Initializer.BoundShaderState.GetGeometryShader()->GetHash()) : 0;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
			HashKey.MeshShader = Initializer.BoundShaderState.GetMeshShader() ? GetTypeHash(Initializer.BoundShaderState.GetMeshShader()->GetHash()) : 0;
#endif
			FBlendStateInitializerRHI BlendStateInitializerRHI;
			if (Initializer.BlendState && Initializer.BlendState->GetInitializer(BlendStateInitializerRHI))
			{
				HashKey.BlendState = GetTypeHash(BlendStateInitializerRHI);
			}

			// The only rasterizer state that matters is whether multisampling is enabled.
			FRasterizerStateInitializerRHI RasterizerStateInitializerRHI;
			if (Initializer.RasterizerState && Initializer.RasterizerState->GetInitializer(RasterizerStateInitializerRHI))
			{
				HashKey.MultisamplingEnabled = RasterizerStateInitializerRHI.bAllowMSAA;
			}

			return CityHash64((const char*)&HashKey, sizeof(FHashKey));
		}
	}

	return FDynamicRHI::RHIComputeStatePrecachePSOHash(Initializer);
}

uint64 FD3D12DynamicRHI::RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	// When compute precache PSO hash we assume a valid state precache PSO hash is already provided
	checkf(Initializer.StatePrecachePSOHash != 0, TEXT("Initializer should have a valid state precache PSO hash set when computing the full initializer PSO hash"));

	if (bDriverCacheAwarePSOPrecaching)
	{
		if (IsRHIDeviceNVIDIA())
		{
			// We already hashed everything we needed.
			return Initializer.StatePrecachePSOHash;
		}
		else if (IsRHIDeviceIntel() && GMaxRHIFeatureLevel == ERHIFeatureLevel::SM6)
		{
			// On top of the state already hashed, Intel drivers care about multisampling and render target count/format.
			struct FHashKey
			{
				uint64 StatePrecachePSOHash;

				uint8 NumSamples;
				uint8 NumRenderTargets;
				FGraphicsPipelineStateInitializer::TRenderTargetFormats	RenderTargetFormats;
				FGraphicsPipelineStateInitializer::TRenderTargetFlags   RenderTargetFlags;
			} HashKey;

			FMemory::Memzero(&HashKey, sizeof(FHashKey));

			HashKey.StatePrecachePSOHash = Initializer.StatePrecachePSOHash;
			HashKey.NumSamples = Initializer.NumSamples;
			HashKey.NumRenderTargets = Initializer.RenderTargetsEnabled;
			HashKey.RenderTargetFormats = Initializer.RenderTargetFormats;
			for (uint32 Index = 0; Index < HashKey.NumRenderTargets; ++Index)
			{
				HashKey.RenderTargetFlags[Index] = Initializer.RenderTargetFlags[Index] & FGraphicsPipelineStateInitializer::RelevantRenderTargetFlagMask;
			}

			return CityHash64((const char*)&HashKey, sizeof(FHashKey));
		}
	}

	// All members which are not part of the state objects and influence the PSO on D3D12
	struct FNonStateHashKey
	{
		uint64							StatePrecachePSOHash;

		EPrimitiveType					PrimitiveType;
		uint32							RenderTargetsEnabled;
		FGraphicsPipelineStateInitializer::TRenderTargetFormats RenderTargetFormats;
		EPixelFormat					DepthStencilTargetFormat;
		uint16							NumSamples;
		EConservativeRasterization		ConservativeRasterization;
		bool							bDepthBounds;
		EVRSShadingRate					ShadingRate;
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FNonStateHashKey));

	HashKey.StatePrecachePSOHash			= Initializer.StatePrecachePSOHash;

	HashKey.PrimitiveType					= Initializer.PrimitiveType;
	HashKey.RenderTargetsEnabled			= Initializer.RenderTargetsEnabled;
	HashKey.RenderTargetFormats				= Initializer.RenderTargetFormats;
	HashKey.DepthStencilTargetFormat		= Initializer.DepthStencilTargetFormat;
	HashKey.NumSamples						= Initializer.NumSamples;
	HashKey.ConservativeRasterization		= Initializer.ConservativeRasterization;
	HashKey.bDepthBounds					= Initializer.bDepthBounds;
	HashKey.ShadingRate						= Initializer.ShadingRate;

	return CityHash64((const char*)&HashKey, sizeof(FNonStateHashKey));
}

bool FD3D12DynamicRHI::RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	// first check non pointer objects
	if (LHS.PrimitiveType != RHS.PrimitiveType ||
		LHS.bDepthBounds != RHS.bDepthBounds ||
		LHS.MultiViewCount != RHS.MultiViewCount ||
		LHS.ShadingRate != RHS.ShadingRate ||
		LHS.bHasFragmentDensityAttachment != RHS.bHasFragmentDensityAttachment ||
		LHS.RenderTargetsEnabled != RHS.RenderTargetsEnabled ||
		LHS.RenderTargetFormats != RHS.RenderTargetFormats ||
		LHS.DepthStencilTargetFormat != RHS.DepthStencilTargetFormat ||
		LHS.NumSamples != RHS.NumSamples ||
		LHS.ConservativeRasterization != RHS.ConservativeRasterization)
	{
		return false;
	}

	// check the RHI shaders (pointer check for shaders should be fine)
	if (LHS.BoundShaderState.GetVertexShader() != RHS.BoundShaderState.GetVertexShader() ||
		LHS.BoundShaderState.GetPixelShader() != RHS.BoundShaderState.GetPixelShader() ||
		LHS.BoundShaderState.GetMeshShader() != RHS.BoundShaderState.GetMeshShader() ||
		LHS.BoundShaderState.GetAmplificationShader() != RHS.BoundShaderState.GetAmplificationShader() ||
		LHS.BoundShaderState.GetGeometryShader() != RHS.BoundShaderState.GetGeometryShader())
	{
		return false;
	}

	// Compare the d3d12 vertex elements without the stride
	FD3D12VertexElements LHSVertexElements;
	if (LHS.BoundShaderState.VertexDeclarationRHI)
	{
		LHSVertexElements = ((FD3D12VertexDeclaration*)LHS.BoundShaderState.VertexDeclarationRHI)->VertexElements;
	}
	FD3D12VertexElements RHSVertexElements;
	if (RHS.BoundShaderState.VertexDeclarationRHI)
	{
		RHSVertexElements = ((FD3D12VertexDeclaration*)RHS.BoundShaderState.VertexDeclarationRHI)->VertexElements;
	}
	if (LHSVertexElements != RHSVertexElements)
	{
		return false;
	}

	// Check actual state content (each initializer can have it's own state and not going through a factory)
	if (!MatchRHIState<FRHIBlendState, FBlendStateInitializerRHI>(LHS.BlendState, RHS.BlendState) ||
		!MatchRHIState<FRHIRasterizerState, FRasterizerStateInitializerRHI>(LHS.RasterizerState, RHS.RasterizerState) ||
		!MatchRHIState<FRHIDepthStencilState, FDepthStencilStateInitializerRHI>(LHS.DepthStencilState, RHS.DepthStencilState))
	{
		return false;
	}

	return true;
}

FGraphicsPipelineStateRHIRef FD3D12DynamicRHI::RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer)
{
	SCOPE_CYCLE_COUNTER(STAT_PSOGraphicsFindOrCreateTime);

	FD3D12PipelineStateCache& PSOCache = GetAdapter().GetPSOCache();
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	// First try to find the PSO based on the hash of runtime objects.
	uint32 InitializerHash;
	FD3D12GraphicsPipelineState* Found = PSOCache.FindInRuntimeCache(Initializer, InitializerHash);
	if (Found)
	{
#if DO_CHECK
		ensure(FMemory::Memcmp(&Found->PipelineStateInitializer, &Initializer, sizeof(Initializer)) == 0);
#endif // DO_CHECK
		return Found;
	}
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12DynamicRHI::RHICreateGraphicsPipelineState);

	const FD3D12RootSignature* RootSignature = GetAdapter().GetRootSignature(Initializer.BoundShaderState);

	if (!RootSignature || !RootSignature->GetRootSignature())
	{
		UE_LOGF(LogD3D12RHI, Error, "Unexpected null root signature at graphics pipeline creation time");
		return nullptr;
	}

	// Next try to find the PSO based on the hash of its desc.

	FD3D12LowLevelGraphicsPipelineStateDesc LowLevelDesc;
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	Found = PSOCache.FindInLoadedCache(Initializer, InitializerHash, RootSignature, LowLevelDesc);
#else
	FD3D12GraphicsPipelineState* Found = PSOCache.FindInLoadedCache(Initializer, RootSignature, LowLevelDesc);
#endif
	if (Found)
	{
		return Found;
	}

	// We need to actually create a PSO.
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	return PSOCache.CreateAndAdd(Initializer, InitializerHash, RootSignature, LowLevelDesc);
#else
	return PSOCache.CreateAndAdd(Initializer, RootSignature, LowLevelDesc);
#endif
}

TRefCountPtr<FRHIComputePipelineState> FD3D12DynamicRHI::RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer)
{
	SCOPE_CYCLE_COUNTER(STAT_PSOComputeFindOrCreateTime);

	check(Initializer.ComputeShader);
	FD3D12PipelineStateCache& PSOCache = GetAdapter().GetPSOCache();
	FD3D12ComputeShader* ComputeShader = FD3D12DynamicRHI::ResourceCast(Initializer.ComputeShader);

	// First try to find the PSO based on Compute Shader pointer.
	FD3D12ComputePipelineState* Found;
#if D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	Found = PSOCache.FindInRuntimeCache(Initializer.ComputeShader);
	if (Found)
	{
		return Found;
	}
#endif

	const FD3D12RootSignature* RootSignature = ComputeShader->RootSignature;

	if (!RootSignature || !RootSignature->GetRootSignature())
	{
		UE_LOGF(LogD3D12RHI, Error, "Unexpected null root signature at compute pipeline creation time (shader hash %ls)", *ComputeShader->GetHash().ToString());
		return nullptr;
	}

	// Next try to find the PSO based on the hash of its desc.
	FD3D12ComputePipelineStateDesc LowLevelDesc;
	Found = PSOCache.FindInLoadedCache(Initializer, RootSignature, LowLevelDesc);
	if (Found)
	{
		return Found;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12DynamicRHI::RHICreateComputePipelineState);

	// We need to actually create a PSO.
	return PSOCache.CreateAndAdd(Initializer, RootSignature, LowLevelDesc);
}

FD3D12SamplerState::FD3D12SamplerState(FD3D12Device* InParent, const D3D12_SAMPLER_DESC& Desc, uint16 SamplerID, FD3D12SamplerState* FirstLinkedObject)
	: FD3D12DeviceChild(InParent)
	, ID(SamplerID)
{
	FD3D12OfflineDescriptorManager& OfflineAllocator = GetParentDevice()->GetOfflineDescriptorManager(ERHIDescriptorHeapType::Sampler);
	OfflineDescriptor = OfflineAllocator.AllocateHeapSlot();

	GetParentDevice()->CreateSamplerInternal(Desc, OfflineDescriptor);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (InParent->GetBindlessDescriptorAllocator().GetConfiguration() != ERHIBindlessConfiguration::Disabled)
	{
		BindlessHandle = FirstLinkedObject ? FirstLinkedObject->BindlessHandle : InParent->GetBindlessDescriptorAllocator().AllocateDescriptor(ERHIDescriptorType::Sampler);

		InParent->GetBindlessDescriptorManager().InitializeDescriptor(BindlessHandle, this);
	}
#endif
}

void FD3D12SamplerState::FreeDescriptor()
{
	if (OfflineDescriptor)
	{
		FD3D12OfflineDescriptorManager& OfflineAllocator = GetParentDevice()->GetOfflineDescriptorManager(ERHIDescriptorHeapType::Sampler);
		OfflineAllocator.FreeHeapSlot(OfflineDescriptor);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		// Handle is shared -- freeing is handled by the head link in the FD3D12LinkedAdapterObject
		if (BindlessHandle.IsValid() && IsHeadLink())
		{
			GetParentDevice()->GetBindlessDescriptorManager().ImmediateFree(BindlessHandle);
		}
#endif
	}
}

FD3D12SamplerState::~FD3D12SamplerState()
{
	FreeDescriptor();
}
