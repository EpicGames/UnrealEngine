// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12State.h: D3D state definitions.
	=============================================================================*/

#pragma once

#include "D3D12Descriptors.h"
#include "RHIResources.h"

class FD3D12SamplerState : public FRHISamplerState, public FD3D12DeviceChild, public FD3D12LinkedAdapterObject<FD3D12SamplerState>
{
public:
	FD3D12OfflineDescriptor OfflineDescriptor;
	FRHIDescriptorHandle BindlessHandle;
	const uint16 ID;

	FD3D12SamplerState() = delete;
	FD3D12SamplerState(FD3D12Device* InParent, const D3D12_SAMPLER_DESC& Desc, uint16 SamplerID, FD3D12SamplerState* FirstLinkedObject);
	~FD3D12SamplerState();

	void FreeDescriptor();

	virtual FRHIDescriptorHandle GetBindlessHandle() const final { return BindlessHandle; }
};

class FD3D12RasterizerState : public FRHIRasterizerState
{
public:
	D3D12_RASTERIZER_DESC Desc;

	virtual bool GetInitializer(struct FRasterizerStateInitializerRHI& Init) final override;
};

class FD3D12DepthStencilState : public FRHIDepthStencilState
{
public:

	D3D12_DEPTH_STENCIL_DESC1 Desc;

	/* Describes the read/write state of the separate depth and stencil components of the DSV. */
	FExclusiveDepthStencil AccessType;

	virtual bool GetInitializer(struct FDepthStencilStateInitializerRHI& Init) final override;
};

class FD3D12BlendState : public FRHIBlendState
{
public:

	D3D12_BLEND_DESC Desc;

	virtual bool GetInitializer(class FBlendStateInitializerRHI& Init) final override;
};

