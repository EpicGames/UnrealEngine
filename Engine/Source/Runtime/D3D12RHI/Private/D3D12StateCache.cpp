// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Device Context State Caching to improve draw
//	thread performance by removing redundant device context calls.

#include "D3D12RHIPrivate.h"
#include "D3D12ExplicitDescriptorCache.h"

// This value defines how many descriptors will be in the device global descriptor heap. This heap contains all shader visible view descriptors.
// Other shader visible descriptor heaps (e.g. OnlineViewHeap) are allocated from this pool. Non-visible heaps (e.g. LocalViewHeap) are allocated as standalone.
int32 GGlobalResourceDescriptorHeapSize = 1000 * 1000;
static FAutoConsoleVariableRef CVarGlobalResourceDescriptorHeapSize(
	TEXT("D3D12.GlobalResourceDescriptorHeapSize"),
	GGlobalResourceDescriptorHeapSize,
	TEXT("Global resource descriptor heap size"),
	ECVF_ReadOnly
);

int32 GGlobalSamplerDescriptorHeapSize = 2048;
static FAutoConsoleVariableRef CVarGlobalSamplerDescriptorHeapSize(
	TEXT("D3D12.GlobalSamplerDescriptorHeapSize"),
	GGlobalSamplerDescriptorHeapSize,
	TEXT("Global sampler descriptor heap size"),
	ECVF_ReadOnly
);

// This value defines how many descriptors will be in the device local view heap which
// This should be tweaked for each title as heaps require VRAM. The default value of 512k takes up ~16MB
int32 GLocalViewHeapSize = 500 * 1000;
static FAutoConsoleVariableRef CVarLocalViewHeapSize(
	TEXT("D3D12.LocalViewHeapSize"),
	GLocalViewHeapSize,
	TEXT("Local view heap size"),
	ECVF_ReadOnly
);

int32 GGlobalSamplerHeapSize = 2048;
static FAutoConsoleVariableRef CVarGlobalSamplerHeapSize(
	TEXT("D3D12.GlobalSamplerHeapSize"),
	GGlobalSamplerHeapSize,
	TEXT("Global sampler descriptor heap size"),
	ECVF_ReadOnly
);

// This value defines how many descriptors will be in the device online view heap which
// is shared across contexts to allow the driver to eliminate redundant descriptor heap sets.
// This should be tweaked for each title as heaps require VRAM. The default value of 512k takes up ~16MB
int32 GOnlineDescriptorHeapSize = 500 * 1000;
static FAutoConsoleVariableRef CVarOnlineDescriptorHeapSize(
	TEXT("D3D12.OnlineDescriptorHeapSize"),
	GOnlineDescriptorHeapSize,
	TEXT("Online descriptor heap size"),
	ECVF_ReadOnly
);

int32 GOnlineDescriptorHeapBlockSize = 2000;
static FAutoConsoleVariableRef CVarOnlineDescriptorHeapBlockSize(
	TEXT("D3D12.OnlineDescriptorHeapBlockSize"),
	GOnlineDescriptorHeapBlockSize,
	TEXT("Block size for sub allocations on the global view descriptor heap."),
	ECVF_ReadOnly
);

int32 GBindlessOnlineDescriptorHeapSize = 500 * 1000;
static FAutoConsoleVariableRef CVarBindlessOnlineDescriptorHeapSize(
	TEXT("D3D12.BindlessOnlineDescriptorHeapSize"),
	GBindlessOnlineDescriptorHeapSize,
	TEXT("Online descriptor heap size"),
	ECVF_ReadOnly
);

int32 GBindlessOnlineDescriptorHeapBlockSize = 2000;
static FAutoConsoleVariableRef CVarBindlessOnlineDescriptorHeapBlockSize(
	TEXT("D3D12.BindlessOnlineDescriptorHeapBlockSize"),
	GBindlessOnlineDescriptorHeapBlockSize,
	TEXT("Block size for sub allocations on the global view descriptor heap."),
	ECVF_ReadOnly
);

inline bool operator!=(D3D12_CPU_DESCRIPTOR_HANDLE lhs, D3D12_CPU_DESCRIPTOR_HANDLE rhs)
{
	return lhs.ptr != rhs.ptr;
}

#if D3D12_STATE_CACHE_RUNTIME_TOGGLE

// Default the state caching system to on.
bool GD3D12SkipStateCaching = false;

// A self registering exec helper to check for the TOGGLESTATECACHE command.
class FD3D12ToggleStateCacheExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		if (FParse::Command(&Cmd, TEXT("TOGGLESTATECACHE")))
		{
			GD3D12SkipStateCaching = !GD3D12SkipStateCaching;
			Ar.Log(FString::Printf(TEXT("D3D12 State Caching: %s"), GD3D12SkipStateCaching ? TEXT("OFF") : TEXT("ON")));
			return true;
		}
		return false;
	}
};
static FD3D12ToggleStateCacheExecHelper GD3D12ToggleStateCacheExecHelper;

#endif	// D3D12_STATE_CACHE_RUNTIME_TOGGLE

FD3D12StateCache::FD3D12StateCache(FD3D12CommandContext& Context, FRHIGPUMask Node)
	: FD3D12DeviceChild(Context.Device)
	, FD3D12SingleNodeGPUObject(Node)
	, CmdContext(Context)
	, DescriptorCache(Context, Node)
{
	FD3D12Adapter* Adapter = Parent->GetParentAdapter();

	// Cache the resource binding tier
	ResourceBindingTier = Adapter->GetResourceBindingTier();

	const uint32 NumSamplerDescriptors = NUM_SAMPLER_DESCRIPTORS;

	checkCode(
		const int32 MaximumResourceHeapSize = Adapter->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Standard);
		const int32 MaximumSamplerHeapSize = Adapter->GetMaxDescriptorsForHeapType(ERHIDescriptorHeapType::Sampler);

		check(GLocalViewHeapSize <= MaximumResourceHeapSize || MaximumResourceHeapSize < 0);
		check(GOnlineDescriptorHeapSize <= MaximumResourceHeapSize || MaximumResourceHeapSize < 0);

		check(NumSamplerDescriptors <= MaximumSamplerHeapSize);
	);

	DescriptorCache.Init(GLocalViewHeapSize, NumSamplerDescriptors);

	ClearState();
}

void FD3D12StateCache::ClearState()
{
	PipelineState = {};
	DirtyState();
}

void FD3D12StateCache::ClearSRVs()
{
	if (bSRVSCleared)
	{
		return;
	}

	PipelineState.Common.SRVCache.Clear();

	bSRVSCleared = true;
}

void FD3D12StateCache::ClearResourceViewCaches(EShaderFrequency ShaderFrequency, FD3D12ResourceLocation*& ResourceLocation, EShaderParameterTypeMask ShaderParameterTypeMask)
{
	//SCOPE_CYCLE_COUNTER(STAT_D3D12ClearShaderResourceViewsTime);

	if (EnumHasAnyFlags(ShaderParameterTypeMask, EShaderParameterTypeMask::SRVMask))
	{
		if (PipelineState.Common.SRVCache.MaxBoundIndex[ShaderFrequency] >= 0)
		{
			auto& CurrentShaderResourceViews = PipelineState.Common.SRVCache.Views[ShaderFrequency];
			for (int32 i = 0; i <= PipelineState.Common.SRVCache.MaxBoundIndex[ShaderFrequency]; ++i)
			{
				if (CurrentShaderResourceViews[i] && CurrentShaderResourceViews[i]->GetResourceLocation() == ResourceLocation)
				{
					SetShaderResourceView(ShaderFrequency, nullptr, i);
				}
			}
		}
	}

	if (EnumHasAnyFlags(ShaderParameterTypeMask, EShaderParameterTypeMask::UAVMask))
	{
		auto& CurrentShaderResourceViews = PipelineState.Common.UAVCache.Views[ShaderFrequency];
		for (int32 i = 0; i <= MAX_UAVS; ++i)
		{
			if (CurrentShaderResourceViews[i] && CurrentShaderResourceViews[i]->GetResourceLocation() == ResourceLocation)
			{
				SetUAV(ShaderFrequency, i, nullptr);
			}
		}
	}

}

void FD3D12StateCache::FlushComputeShaderCache(bool bForce)
{
	if (bForce)
	{
		CmdContext.AddGlobalBarrier(ED3D12Access::UAVCompute, ED3D12Access::UAVCompute);
		INC_DWORD_STAT(STAT_D3D12UAVBarriers);
	}
}

void FD3D12StateCache::DirtyStateForNewCommandList()
{
	// Dirty state that doesn't align with command list defaults.

	// Always need to set PSOs and root signatures
	PipelineState.Common.bNeedSetPSO = true;
	PipelineState.Common.bNeedSetRootConstants = true;
	PipelineState.Compute.bNeedSetRootSignature = true;
	PipelineState.Graphics.bNeedSetRootSignature = true;
	bNeedSetPrimitiveTopology = true;

	if (PipelineState.Graphics.VBCache.BoundVBMask) { bNeedSetVB = true; }

	// IndexBuffers are set in DrawIndexed*() calls, so there's no way to depend on previously set IndexBuffers without making a new DrawIndexed*() call.
	PipelineState.Graphics.IBCache.Clear();

	if (PipelineState.Graphics.CurrentNumberOfRenderTargets || PipelineState.Graphics.CurrentDepthStencilTarget) { bNeedSetRTs = true; }
	if (PipelineState.Graphics.CurrentNumberOfViewports) { bNeedSetViewports = true; }
	if (PipelineState.Graphics.CurrentNumberOfScissorRects) { bNeedSetScissorRects = true; }

	if (PipelineState.Graphics.CurrentBlendFactor[0] != D3D12_DEFAULT_BLEND_FACTOR_RED ||
		PipelineState.Graphics.CurrentBlendFactor[1] != D3D12_DEFAULT_BLEND_FACTOR_GREEN ||
		PipelineState.Graphics.CurrentBlendFactor[2] != D3D12_DEFAULT_BLEND_FACTOR_BLUE ||
		PipelineState.Graphics.CurrentBlendFactor[3] != D3D12_DEFAULT_BLEND_FACTOR_ALPHA)
	{
		bNeedSetBlendFactor = true;
	}

	if (PipelineState.Graphics.CurrentReferenceStencil != D3D12_DEFAULT_STENCIL_REFERENCE) { bNeedSetStencilRef = true; }

	if (PipelineState.Graphics.MinDepth != 0.0 || 
		PipelineState.Graphics.MaxDepth != 1.0)
	{
		bNeedSetDepthBounds = GSupportsDepthBoundsTest;
	}
	
	bNeedSetShadingRate = GRHISupportsPipelineVariableRateShading;
	bNeedSetShadingRateImage = GRHISupportsAttachmentVariableRateShading;

	// Always dirty View and Sampler bindings. We detect the slots that are actually used at Draw/Dispatch time.
	PipelineState.Common.SRVCache.DirtyAll();
	PipelineState.Common.UAVCache.DirtyAll();
	PipelineState.Common.CBVCache.DirtyAll();
	PipelineState.Common.SamplerCache.DirtyAll();
}

void FD3D12StateCache::DirtyState()
{
	// Mark bits dirty so the next call to ApplyState will set all this state again
	PipelineState.Common.bNeedSetPSO = true;
	PipelineState.Common.bNeedSetRootConstants = true;
	PipelineState.Compute.bNeedSetRootSignature = true;
	PipelineState.Graphics.bNeedSetRootSignature = true;
	bNeedSetVB = true;
	bNeedSetRTs = true;
	bNeedSetViewports = true;
	bNeedSetScissorRects = true;
	bNeedSetPrimitiveTopology = true;
	bNeedSetBlendFactor = true;
	bNeedSetStencilRef = true;
	bNeedSetDepthBounds = GSupportsDepthBoundsTest;
	bNeedSetShadingRate = GRHISupportsPipelineVariableRateShading;
	bNeedSetShadingRateImage = GRHISupportsAttachmentVariableRateShading;
	PipelineState.Common.SRVCache.DirtyAll();
	PipelineState.Common.UAVCache.DirtyAll();
	PipelineState.Common.CBVCache.DirtyAll();
	PipelineState.Common.SamplerCache.DirtyAll();
}

void FD3D12StateCache::DirtyViewDescriptorTables()
{
	// Mark the CBV/SRV/UAV descriptor tables dirty for the current root signature.
	// Note: Descriptor table state is undefined at the beginning of a command list and after descriptor heaps are changed on a command list.
	// This will cause the next call to ApplyState to copy and set these descriptors again.
	PipelineState.Common.SRVCache.DirtyAll();
	PipelineState.Common.UAVCache.DirtyAll();
	PipelineState.Common.CBVCache.DirtyAll(GDescriptorTableCBVSlotMask);	// Only mark descriptor table slots as dirty.
}

void FD3D12StateCache::DirtySamplerDescriptorTables()
{
	// Mark the sampler descriptor tables dirty for the current root signature.
	// Note: Descriptor table state is undefined at the beginning of a command list and after descriptor heaps are changed on a command list.
	// This will cause the next call to ApplyState to copy and set these descriptors again.
	PipelineState.Common.SamplerCache.DirtyAll();
}

void FD3D12StateCache::SetViewport(const D3D12_VIEWPORT& Viewport)
{
	if ((PipelineState.Graphics.CurrentNumberOfViewports != 1 || FMemory::Memcmp(&PipelineState.Graphics.CurrentViewport[0], &Viewport, sizeof(D3D12_VIEWPORT))) || GD3D12SkipStateCaching)
	{
		FMemory::Memcpy(&PipelineState.Graphics.CurrentViewport[0], &Viewport, sizeof(D3D12_VIEWPORT));
		PipelineState.Graphics.CurrentNumberOfViewports = 1;
		bNeedSetViewports = true;
	}
}

void FD3D12StateCache::SetViewports(uint32 Count, const D3D12_VIEWPORT* const Viewports)
{
	check(Count < UE_ARRAY_COUNT(PipelineState.Graphics.CurrentViewport));
	if ((PipelineState.Graphics.CurrentNumberOfViewports != Count || FMemory::Memcmp(&PipelineState.Graphics.CurrentViewport[0], Viewports, sizeof(D3D12_VIEWPORT) * Count)) || GD3D12SkipStateCaching)
	{
		FMemory::Memcpy(&PipelineState.Graphics.CurrentViewport[0], Viewports, sizeof(D3D12_VIEWPORT) * Count);
		PipelineState.Graphics.CurrentNumberOfViewports = Count;
		bNeedSetViewports = true;
	}
}

static void ValidateScissorRect(const D3D12_VIEWPORT& Viewport, const D3D12_RECT& ScissorRect)
{
	bool bScissorRectValid = true;
	bScissorRectValid = bScissorRectValid && ScissorRect.left >= (LONG)FMath::RoundToZero(Viewport.TopLeftX);
	bScissorRectValid = bScissorRectValid && ScissorRect.top >= (LONG)FMath::RoundToZero(Viewport.TopLeftY);
	bScissorRectValid = bScissorRectValid && ScissorRect.right <= (LONG)FMath::RoundFromZero(Viewport.TopLeftX + Viewport.Width);
	bScissorRectValid = bScissorRectValid && ScissorRect.bottom <= (LONG)FMath::RoundFromZero(Viewport.TopLeftY + Viewport.Height);
	bScissorRectValid = bScissorRectValid && ScissorRect.left <= ScissorRect.right && ScissorRect.top <= ScissorRect.bottom;

	ensureMsgf(bScissorRectValid,
		TEXT("Scissor invalid with current Viewport. Scissor: [left:%li, top:%li, right:%li, bottom:%li]. Viewport: [left:%.2f, top:%.2f, right:%.2f, bottom:%.2f]")
			, ScissorRect.left
			, ScissorRect.top
			, ScissorRect.right
			, ScissorRect.bottom
			, Viewport.TopLeftX
			, Viewport.TopLeftY
			, Viewport.TopLeftX + Viewport.Width
			, Viewport.TopLeftY + Viewport.Height);
}

void FD3D12StateCache::SetScissorRect(const D3D12_RECT& ScissorRect)
{
	ValidateScissorRect(PipelineState.Graphics.CurrentViewport[0], ScissorRect);

	if ((PipelineState.Graphics.CurrentNumberOfScissorRects != 1 || FMemory::Memcmp(&PipelineState.Graphics.CurrentScissorRects[0], &ScissorRect, sizeof(D3D12_RECT))) || GD3D12SkipStateCaching)
	{
		FMemory::Memcpy(&PipelineState.Graphics.CurrentScissorRects[0], &ScissorRect, sizeof(D3D12_RECT));
		PipelineState.Graphics.CurrentNumberOfScissorRects = 1;
		bNeedSetScissorRects = true;
	}
}

void FD3D12StateCache::SetScissorRects(uint32 Count, const D3D12_RECT* const ScissorRects)
{
	check(Count < UE_ARRAY_COUNT(PipelineState.Graphics.CurrentScissorRects));

	for (uint32 Rect = 0; Rect < Count; ++Rect)
	{
		ValidateScissorRect(PipelineState.Graphics.CurrentViewport[Rect], ScissorRects[Rect]);
	}

	if ((PipelineState.Graphics.CurrentNumberOfScissorRects != Count || FMemory::Memcmp(&PipelineState.Graphics.CurrentScissorRects[0], ScissorRects, sizeof(D3D12_RECT) * Count)) || GD3D12SkipStateCaching)
	{
		FMemory::Memcpy(&PipelineState.Graphics.CurrentScissorRects[0], ScissorRects, sizeof(D3D12_RECT) * Count);
		PipelineState.Graphics.CurrentNumberOfScissorRects = Count;
		bNeedSetScissorRects = true;
	}
}

bool FD3D12StateCache::InternalSetDescriptorHeaps(bool bBindless)
{
	ED3D12SetDescriptorHeapsFlags Flags = ED3D12SetDescriptorHeapsFlags::None;
	if (bBindless)
	{
		EnumAddFlags(Flags, ED3D12SetDescriptorHeapsFlags::Bindless);
	}

	return DescriptorCache.SetDescriptorHeaps(Flags);
}

inline bool ShouldSkipStage(uint32 Stage)
{
	return ((Stage == SF_Mesh || Stage == SF_Amplification) && !GRHISupportsMeshShadersTier0);
}

bool FD3D12StateCache::InternalSetRootSignature(ED3D12PipelineType InPipelineType, const FD3D12RootSignature* InRootSignature, bool bForce)
{
	bool bWasRootSignatureChanged = false;

	if (InPipelineType == ED3D12PipelineType::Compute)
	{
		if (PipelineState.Compute.bNeedSetRootSignature || bForce)
		{
			CmdContext.GraphicsCommandList()->SetComputeRootSignature(InRootSignature->GetRootSignature());
			PipelineState.Compute.bNeedSetRootSignature = false;

			// After setting a root signature, all root parameters are undefined and must be set again.
			PipelineState.Common.SRVCache.DirtyCompute();
			PipelineState.Common.UAVCache.DirtyCompute();
			PipelineState.Common.SamplerCache.DirtyCompute();
			PipelineState.Common.CBVCache.DirtyCompute();
			PipelineState.Common.bNeedSetRootConstants = true;

			bWasRootSignatureChanged = true;
		}
	}
	else if (InPipelineType == ED3D12PipelineType::Graphics)
	{
		// See if we need to set a graphics root signature
		if (PipelineState.Graphics.bNeedSetRootSignature || bForce)
		{
			CmdContext.GraphicsCommandList()->SetGraphicsRootSignature(InRootSignature->GetRootSignature());
			PipelineState.Graphics.bNeedSetRootSignature = false;

			// After setting a root signature, all root parameters are undefined and must be set again.
			PipelineState.Common.SRVCache.DirtyGraphics();
			PipelineState.Common.UAVCache.DirtyGraphics();
			PipelineState.Common.SamplerCache.DirtyGraphics();
			PipelineState.Common.CBVCache.DirtyGraphics();
			PipelineState.Common.bNeedSetRootConstants = true;

			bWasRootSignatureChanged = true;
		}
	}

	return bWasRootSignatureChanged;
}

void FD3D12StateCache::InternalSetPipelineState(FD3D12PipelineState* InPipelineState)
{
	// See if we need to set our PSO:
	// In D3D11, you could Set dispatch arguments, then set Draw arguments, then call Draw/Dispatch/Draw/Dispatch without setting arguments again.
	// In D3D12, we need to understand when the app switches between Draw/Dispatch and make sure the correct PSO is set.

	ID3D12PipelineState* const CurrentD3DPipelineState = PipelineState.Common.CurrentPipelineStateObject;
	ID3D12PipelineState* const PendingD3DPipelineState = InPipelineState->GetPipelineState();

	if (PipelineState.Common.bNeedSetPSO || CurrentD3DPipelineState == nullptr || CurrentD3DPipelineState != PendingD3DPipelineState)
	{
		PipelineState.Common.CurrentPipelineStateObject = PendingD3DPipelineState;

		CmdContext.GraphicsCommandList()->SetPipelineState(PendingD3DPipelineState);

		PipelineState.Common.bNeedSetPSO = false;
	}
}

void FD3D12StateCache::ApplyState(ERHIPipeline HardwarePipe, ED3D12PipelineType PipelineType)
{
	// Make sure the command list is opened before we start caching anything.
	// There are certain situations where no command list actions were taken before here and we need it to be open before we cache anything.
	CmdContext.OpenIfNotAlready();

	//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateTime);
	const bool bForceState = false;
	if (bForceState)
	{
		// Mark all state as dirty.
		DirtyState();
	}

#if PLATFORM_SUPPORTS_VIRTUAL_TEXTURES
	CmdContext.FlushTextureCacheIfNeeded();
#endif

	FD3D12PipelineStateCommonData* PSOCommonData = nullptr;

	// PSO
	if (PipelineType == ED3D12PipelineType::Compute)
	{
		PSOCommonData = GetComputePipelineState();
	}
	else if (PipelineType == ED3D12PipelineType::Graphics)
	{
		PSOCommonData = GetGraphicsPipelineState();
	}
	else
	{
		checkf(false, TEXT("Unexpected pipeline type: %d"), (uint32)PipelineType);
		return;
	}

	const bool bShaderUsesBindlessResources = PSOCommonData->RootSignature->UsesDynamicResources();
	const bool bShaderUsesBindlessSamplers  = PSOCommonData->RootSignature->UsesDynamicSamplers();

	const bool bShaderUsesBindfulResources = PSOCommonData->RootSignature->HasTableResources();
	const bool bShaderUsesBindfulSamplers  = PSOCommonData->RootSignature->HasSamplers();

	const bool bApplyResourceTables = !bShaderUsesBindlessResources && bShaderUsesBindfulResources;
	const bool bApplySamplerTables = !bShaderUsesBindlessSamplers && bShaderUsesBindfulSamplers;

	bool bBindlessDescriptorHeaps = false;
	bool bSetExplicitDescriptorCache = false;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	{
		FD3D12BindlessDescriptorManager& BindlessManager = GetParentDevice()->GetBindlessDescriptorManager();

		// Require explicit descriptor heaps if 
		if (bShaderUsesBindlessResources && BindlessManager.GetConfiguration() == ERHIBindlessConfiguration::RayTracing)
		{
			// Samplers should be bindless as well
			check(bShaderUsesBindlessSamplers);

			// Should have any SRVs or UAVs
			check(!bShaderUsesBindfulResources);

			// Everything bindless, don't require any CVS, SRV and UAV view data
			FD3D12ExplicitDescriptorCache TransientDescriptorCache(GetParentDevice(), 1);
			TransientDescriptorCache.Init(0, 0, 0, ERHIBindlessConfiguration::RayTracing);

			GetDescriptorCache()->SetExplicitDescriptorCache(TransientDescriptorCache);

			bSetExplicitDescriptorCache = true;
		}
		else if (IsBindlessFullyEnabled(BindlessManager.GetConfiguration()))
		{
			check(bShaderUsesBindlessResources || !bShaderUsesBindfulResources);
			check(bShaderUsesBindlessSamplers || !bShaderUsesBindfulSamplers);

			bBindlessDescriptorHeaps = true;
		}
		else if (IsBindlessEnabledForAnyGraphics(BindlessManager.GetConfiguration()))
		{
			const bool bAnyBindless = (bShaderUsesBindlessResources || bShaderUsesBindlessSamplers);
			const bool bAnyBindful = (bShaderUsesBindfulResources || bShaderUsesBindfulSamplers);

			check(bAnyBindless != bAnyBindful || !bAnyBindful);

			bBindlessDescriptorHeaps = bAnyBindless;
		}
		else
		{
			check(!bShaderUsesBindlessResources && !bShaderUsesBindlessSamplers);
		}
	}
#endif

	const bool bDescriptorHeapsChanged = bSetExplicitDescriptorCache || InternalSetDescriptorHeaps(bBindlessDescriptorHeaps);
	const bool bRootSignatureChanged = InternalSetRootSignature(PipelineType, PSOCommonData->RootSignature, bDescriptorHeapsChanged);

	// Ensure the correct graphics PSO is set.
	InternalSetPipelineState(PSOCommonData->PipelineState);

	if (bRootSignatureChanged)
	{
		CmdContext.BindDiagnosticBuffer(PSOCommonData->RootSignature, PipelineType);
	}

	// Need to cache compute budget, as we need to reset after PSO changes
	if (PipelineType == ED3D12PipelineType::Compute && CmdContext.IsAsyncComputeContext())
	{
		CmdContext.SetAsyncComputeBudgetInternal(PipelineState.Compute.ComputeBudget);
	}

	if (PipelineType == ED3D12PipelineType::Graphics)
	{
		// Setup non-heap bindings
		if (bNeedSetVB)
		{
			bNeedSetVB = false;
			//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateSetVertexBufferTime);
			DescriptorCache.SetVertexBuffers(PipelineState.Graphics.VBCache);
		}
		if (bNeedSetViewports)
		{
			bNeedSetViewports = false;
			CmdContext.GraphicsCommandList()->RSSetViewports(PipelineState.Graphics.CurrentNumberOfViewports, PipelineState.Graphics.CurrentViewport);
		}
		if (bNeedSetScissorRects)
		{
			bNeedSetScissorRects = false;
			CmdContext.GraphicsCommandList()->RSSetScissorRects(PipelineState.Graphics.CurrentNumberOfScissorRects, PipelineState.Graphics.CurrentScissorRects);
		}
		if (bNeedSetPrimitiveTopology)
		{
			bNeedSetPrimitiveTopology = false;
			CmdContext.GraphicsCommandList()->IASetPrimitiveTopology(PipelineState.Graphics.CurrentPrimitiveTopology);
		}
		if (bNeedSetBlendFactor)
		{
			bNeedSetBlendFactor = false;
			CmdContext.GraphicsCommandList()->OMSetBlendFactor(PipelineState.Graphics.CurrentBlendFactor);
		}
		if (bNeedSetStencilRef)
		{
			bNeedSetStencilRef = false;
			CmdContext.GraphicsCommandList()->OMSetStencilRef(PipelineState.Graphics.CurrentReferenceStencil);
		}
		if (bNeedSetRTs)
		{
			bNeedSetRTs = false;
			DescriptorCache.SetRenderTargets(PipelineState.Graphics.RenderTargetArray, PipelineState.Graphics.CurrentNumberOfRenderTargets, PipelineState.Graphics.CurrentDepthStencilTarget);
		}
		if (bNeedSetDepthBounds)
		{
			bNeedSetDepthBounds = false;
			CmdContext.SetDepthBounds(PipelineState.Graphics.MinDepth, PipelineState.Graphics.MaxDepth);
		}

		// The shading rate image can influence RSSetShadingRate, because we want to set the combiners to passthrough/passthrough if the rate = 1x1 and the SRI is nullptr
		if (bNeedSetShadingRate || bNeedSetShadingRateImage)
		{
			bNeedSetShadingRate = false;
			bNeedSetShadingRateImage = false;
			CmdContext.SetShadingRate(PipelineState.Graphics.DrawShadingRate, PipelineState.Graphics.ShadingRateImage, PipelineState.Graphics.Combiners);
		}
	}

	// Note that ray tracing pipeline shares state with compute
	const uint32 StartStage = PipelineType == ED3D12PipelineType::Graphics ? 0 : SF_Compute;
	const uint32 EndStage = PipelineType == ED3D12PipelineType::Graphics ? SF_Compute : SF_NumStandardFrequencies;

	//
	// Reserve space in descriptor heaps
	// Since this can cause heap rollover (which causes old bindings to become invalid), the reserve must be done atomically
	//

	// Samplers
	if (bApplySamplerTables)
	{
		ApplySamplerTables(PSOCommonData->RootSignature, StartStage, EndStage);
	}

	if (bApplyResourceTables)
	{
		ApplyResourceTables(PSOCommonData->RootSignature, StartStage, EndStage);
	}
	else if (bShaderUsesBindlessResources)
	{
		ApplyBindlessResources(PSOCommonData->RootSignature, StartStage, EndStage);
	}

	ApplyConstants(PSOCommonData->RootSignature, StartStage, EndStage);

	int8 RootConstantsSlot = PSOCommonData->RootSignature->GetRootConstantsSlot();
	if (PipelineState.Common.bNeedSetRootConstants && RootConstantsSlot >= 0)
	{
		PipelineState.Common.bNeedSetRootConstants = false;

		uint32 UERootConstants[4];
		UERootConstants[0] = PipelineState.Common.ShaderRootConstants.X;
		UERootConstants[1] = PipelineState.Common.ShaderRootConstants.Y;
		UERootConstants[2] = PipelineState.Common.ShaderRootConstants.Z;
		UERootConstants[3] = PipelineState.Common.ShaderRootConstants.W;

		if (PipelineType == ED3D12PipelineType::Compute)
		{
			CmdContext.GraphicsCommandList()->SetComputeRoot32BitConstants(RootConstantsSlot, 4, &UERootConstants[0], 0);
		}
		else if (PipelineType == ED3D12PipelineType::Graphics) //-V547
		{
			CmdContext.GraphicsCommandList()->SetGraphicsRoot32BitConstants(RootConstantsSlot, 4, &UERootConstants[0], 0);
		}
		else
		{
			checkNoEntry();
		}
	}

	// Flush any needed resource barriers
	CmdContext.FlushResourceBarriers();

#if ASSERT_RESOURCE_STATES
	bool bSucceeded = AssertResourceStates(PipelineType);
	check(bSucceeded);
#endif
}

void FD3D12StateCache::ApplyResourceTables(const FD3D12RootSignature* const pRootSignature, uint32 StartStage, uint32 EndStage)
{
	const bool bUAVs = pRootSignature->HasUAVs();
	const bool bSRVs = pRootSignature->HasSRVs();
	const bool bCBVs = pRootSignature->HasCBVs();

	// Determine what resource bind slots are dirty for the current shaders and how many descriptor table slots we need.
	// We only set dirty resources that can be used for the upcoming Draw/Dispatch.
	SRVSlotMask CurrentShaderDirtySRVSlots[SF_NumStandardFrequencies] = {};
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	CBVSlotMask CurrentShaderDirtyCBVSlots[SF_NumStandardFrequencies] = {};
#endif
	UAVSlotMask CurrentShaderDirtyUAVSlots[SF_NumStandardFrequencies] = {};
	uint32 NumUAVs[SF_NumStandardFrequencies] = {};
	uint32 NumSRVs[SF_NumStandardFrequencies] = {};
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	uint32 NumCBVs[SF_NumStandardFrequencies] = {};
#endif
	uint32 NumViews = 0;

	for (uint32 iTries = 0; iTries < 2; ++iTries)
	{
		for (uint32 Stage = StartStage; Stage < EndStage; ++Stage)
		{
			if (ShouldSkipStage(Stage))
			{
				continue;
			}

			if (bUAVs)
			{
				const UAVSlotMask CurrentShaderUAVRegisterMask = BitMask<UAVSlotMask>(PipelineState.Common.CurrentShaderUAVCounts[Stage]);
				CurrentShaderDirtyUAVSlots[Stage] = CurrentShaderUAVRegisterMask & PipelineState.Common.UAVCache.DirtySlotMask[Stage];
				if (CurrentShaderDirtyUAVSlots[Stage])
				{
					check(D3D12DoesShaderFrequencySupportUAV(static_cast<EShaderFrequency>(Stage)));

					if (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_2)
					{
						// Tier 1 and 2 HW requires the full number of UAV descriptors defined in the root signature's descriptor table.
						NumUAVs[Stage] = pRootSignature->MaxUAVCount(Stage);
					}
					else
					{
						NumUAVs[Stage] = PipelineState.Common.CurrentShaderUAVCounts[Stage];
					}

					check(NumUAVs[Stage] > 0 && NumUAVs[Stage] <= MAX_UAVS);
					NumViews += NumUAVs[Stage];
				}
			}

			if (bSRVs)
			{
				// Note this code assumes the starting register is index 0.
				const SRVSlotMask CurrentShaderSRVRegisterMask = BitMask<SRVSlotMask>(PipelineState.Common.CurrentShaderSRVCounts[Stage]);
				CurrentShaderDirtySRVSlots[Stage] = CurrentShaderSRVRegisterMask & PipelineState.Common.SRVCache.DirtySlotMask[Stage];
				if (CurrentShaderDirtySRVSlots[Stage])
				{
					if (ResourceBindingTier == D3D12_RESOURCE_BINDING_TIER_1)
					{
						// Tier 1 HW requires the full number of SRV descriptors defined in the root signature's descriptor table.
						NumSRVs[Stage] = pRootSignature->MaxSRVCount(Stage);
					}
					else
					{
						NumSRVs[Stage] = PipelineState.Common.CurrentShaderSRVCounts[Stage];
					}

					check(NumSRVs[Stage] > 0 && NumSRVs[Stage] <= MAX_SRVS);
					NumViews += NumSRVs[Stage];
				}
			}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			if (bCBVs)
			{
				const CBVSlotMask CurrentShaderCBVRegisterMask = BitMask<CBVSlotMask>(PipelineState.Common.CurrentShaderCBCounts[Stage]);
				CurrentShaderDirtyCBVSlots[Stage] = CurrentShaderCBVRegisterMask & PipelineState.Common.CBVCache.DirtySlotMask[Stage];
				if (CurrentShaderDirtyCBVSlots[Stage])
				{
					if (ResourceBindingTier == D3D12_RESOURCE_BINDING_TIER_1)
					{
						// Tier 1 HW requires the full number of SRV descriptors defined in the root signature's descriptor table.
						NumCBVs[Stage] = pRootSignature->MaxCBVCount(Stage);
					}
					else
					{
						NumCBVs[Stage] = PipelineState.Common.CurrentShaderCBCounts[Stage];
					}

					check(NumCBVs[Stage] > 0 && NumCBVs[Stage] <= MAX_CBS);
					NumViews += NumCBVs[Stage];
				}
			}
#endif
			// Note: CBVs don't currently use descriptor tables but we still need to know what resource point slots are dirty.
		}

		// See if the descriptor slots will fit
		if (!DescriptorCache.GetCurrentViewHeap()->CanReserveSlots(NumViews))
		{
			const bool bDescriptorHeapsChanged = DescriptorCache.GetCurrentViewHeap()->RollOver();
			if (bDescriptorHeapsChanged)
			{
				// If descriptor heaps changed, then all our tables are dirty again and we need to recalculate the number of slots we need.
				NumViews = 0;
				continue;
			}
		}

		// We can reserve slots in the descriptor heap, no need to loop again.
		break;
	}

	uint32 ViewHeapSlot = DescriptorCache.GetCurrentViewHeap()->ReserveSlots(NumViews);

	// Unordered access views
	if (bUAVs)
	{
		//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateSetUAVTime);
		FD3D12UnorderedAccessViewCache& UAVCache = PipelineState.Common.UAVCache;

		for (uint32 Index = StartStage; Index < EndStage; Index++)
		{
			if (CurrentShaderDirtyUAVSlots[Index])
			{
				const EShaderFrequency ShaderFrequency = static_cast<EShaderFrequency>(Index);

				const D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = DescriptorCache.BuildUAVTable(ShaderFrequency, pRootSignature, UAVCache, CurrentShaderDirtyUAVSlots[Index], NumUAVs[Index], ViewHeapSlot);
				DescriptorCache.SetUAVTable(ShaderFrequency, pRootSignature, UAVCache, NumUAVs[Index], BindDescriptor);
			}
		}
	}

	// Shader resource views
	if (bSRVs)
	{
		//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateSetSRVTime);
		FD3D12ShaderResourceViewCache& SRVCache = PipelineState.Common.SRVCache;

		for (uint32 Index = StartStage; Index < EndStage; Index++)
		{
			if (CurrentShaderDirtySRVSlots[Index])
			{
				const D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = DescriptorCache.BuildSRVTable(static_cast<EShaderFrequency>(Index), pRootSignature, SRVCache, CurrentShaderDirtySRVSlots[Index], NumSRVs[Index], ViewHeapSlot);
				DescriptorCache.SetSRVTable(static_cast<EShaderFrequency>(Index), pRootSignature, SRVCache, NumSRVs[Index], BindDescriptor);
			}
		}
	}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	// Constant buffers
	if (bCBVs)
	{
		//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateSetConstantBufferTime);
		FD3D12ConstantBufferCache& CBVCache = PipelineState.Common.CBVCache;

		for (uint32 Index = StartStage; Index < EndStage; Index++)
		{
			if (CurrentShaderDirtyCBVSlots[Index])
			{
				DescriptorCache.SetConstantBufferViews(static_cast<EShaderFrequency>(Index), pRootSignature, CBVCache, CurrentShaderDirtyCBVSlots[Index], NumCBVs[Index], ViewHeapSlot);
			}
		}
	}
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
}

void FD3D12StateCache::ApplyBindlessResources(const FD3D12RootSignature* const pRootSignature, uint32 StartStage, uint32 EndStage)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (uint32 Index = StartStage; Index < EndStage; Index++)
	{
		DescriptorCache.PrepareBindlessViews(
			static_cast<EShaderFrequency>(Index)
			, PipelineState.Common.QueuedBindlessSRVs[Index]
			, PipelineState.Common.QueuedBindlessUAVs[Index]);

		PipelineState.Common.QueuedBindlessSRVs[Index].Reset();
		PipelineState.Common.QueuedBindlessUAVs[Index].Reset();
	}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	if (pRootSignature && pRootSignature->HasCBVs())
	{
		FD3D12ConstantBufferCache& CBVCache = PipelineState.Common.CBVCache;

		CBVSlotMask CurrentShaderDirtyCBVSlots[SF_NumStandardFrequencies] = {};
		uint32 NumCBVs[SF_NumStandardFrequencies] = {};

		uint32 NumViews = 0;

		for (uint32 iTries = 0; iTries < 2; ++iTries)
		{
			for (uint32 Stage = StartStage; Stage < EndStage; ++Stage)
			{
				if (ShouldSkipStage(Stage))
				{
					continue;
				}

				const uint32 ConstantBufferCount = PipelineState.Common.CurrentShaderCBCounts[Stage];

				const CBVSlotMask CurrentShaderCBVRegisterMask = BitMask<CBVSlotMask>(ConstantBufferCount);
				CurrentShaderDirtyCBVSlots[Stage] = CurrentShaderCBVRegisterMask & CBVCache.DirtySlotMask[Stage];
				if (CurrentShaderDirtyCBVSlots[Stage])
				{
					check(ConstantBufferCount > 0 && ConstantBufferCount <= MAX_CBS);

					NumCBVs[Stage] = ConstantBufferCount;
					NumViews += ConstantBufferCount;
				}
				// Note: CBVs don't currently use descriptor tables but we still need to know what resource point slots are dirty.
			}

			// See if the descriptor slots will fit
			if (!DescriptorCache.GetCurrentViewHeap()->CanReserveSlots(NumViews))
			{
				if (DescriptorCache.GetCurrentViewHeap()->RollOver())
				{
					// If descriptor heaps changed, then all our tables are dirty again and we need to recalculate the number of slots we need.
					NumViews = 0;
					continue;
				}
			}
		}

		uint32 ViewHeapSlot = DescriptorCache.GetCurrentViewHeap()->ReserveSlots(NumViews);

		for (uint32 Index = StartStage; Index < EndStage; Index++)
		{
			if (CurrentShaderDirtyCBVSlots[Index])
			{
				DescriptorCache.SetConstantBufferViews(static_cast<EShaderFrequency>(Index), pRootSignature, CBVCache, CurrentShaderDirtyCBVSlots[Index], NumCBVs[Index], ViewHeapSlot);
			}
		}
	}
#endif // D3D12RHI_USE_CONSTANT_BUFFER_VIEWS

#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
}

void FD3D12StateCache::ApplyConstants(const FD3D12RootSignature* const pRootSignature, uint32 StartStage, uint32 EndStage)
{
#if !D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	// Determine what resource bind slots are dirty for the current shaders and how many descriptor table slots we need.
	// We only set dirty resources that can be used for the upcoming Draw/Dispatch.
	CBVSlotMask CurrentShaderDirtyCBVSlots[SF_NumStandardFrequencies] = {};

	for (uint32 Stage = StartStage; Stage < EndStage; ++Stage)
	{
		if (ShouldSkipStage(Stage))
		{
			continue;
		}

		const CBVSlotMask CurrentShaderCBVRegisterMask = BitMask<CBVSlotMask>(PipelineState.Common.CurrentShaderCBCounts[Stage]);
		CurrentShaderDirtyCBVSlots[Stage] = CurrentShaderCBVRegisterMask & PipelineState.Common.CBVCache.DirtySlotMask[Stage];
		// Note: CBVs don't currently use descriptor tables but we still need to know what resource point slots are dirty.
	}

	// Constant buffers
	{
		//SCOPE_CYCLE_COUNTER(STAT_D3D12ApplyStateSetConstantBufferTime);
		FD3D12ConstantBufferCache& CBVCache = PipelineState.Common.CBVCache;

		for (uint32 Index = StartStage; Index < EndStage; Index++)
		{
			if (CurrentShaderDirtyCBVSlots[Index])
			{
				DescriptorCache.SetRootConstantBuffers(static_cast<EShaderFrequency>(Index), pRootSignature, CBVCache, CurrentShaderDirtyCBVSlots[Index], this);
			}
		}
	}
#endif // !D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
}


void FD3D12StateCache::ApplySamplerTables(const FD3D12RootSignature* const pRootSignature, uint32 StartStage, uint32 EndStage)
{
	bool HighLevelCacheMiss = false;

	FD3D12SamplerStateCache& Cache = PipelineState.Common.SamplerCache;
	SamplerSlotMask CurrentShaderDirtySamplerSlots[SF_NumStandardFrequencies] = {};
	uint32 NumSamplers[SF_NumStandardFrequencies + 1] = {};

	const auto& pfnCalcSamplersNeeded = [&]()
	{
		NumSamplers[SF_NumStandardFrequencies] = 0;

		for (uint32 Stage = StartStage; Stage < EndStage; ++Stage)
		{
			if (ShouldSkipStage(Stage))
			{
				continue;
			}

			// Note this code assumes the starting register is index 0.
			const SamplerSlotMask CurrentShaderSamplerRegisterMask = BitMask<SamplerSlotMask>(PipelineState.Common.CurrentShaderSamplerCounts[Stage]);
			CurrentShaderDirtySamplerSlots[Stage] = CurrentShaderSamplerRegisterMask & Cache.DirtySlotMask[Stage];
			if (CurrentShaderDirtySamplerSlots[Stage])
			{
				if (ResourceBindingTier == D3D12_RESOURCE_BINDING_TIER_1)
				{
					// Tier 1 HW requires the full number of sampler descriptors defined in the root signature.
					NumSamplers[Stage] = pRootSignature->MaxSamplerCount(Stage);
				}
				else
				{
					NumSamplers[Stage] = PipelineState.Common.CurrentShaderSamplerCounts[Stage];
				}

				check(NumSamplers[Stage] > 0 && NumSamplers[Stage] <= MAX_SAMPLERS);
				NumSamplers[SF_NumStandardFrequencies] += NumSamplers[Stage];
			}
		}
	};

	pfnCalcSamplersNeeded();

	if (DescriptorCache.UsingGlobalSamplerHeap())
	{
		auto& GlobalSamplerSet = DescriptorCache.GetLocalSamplerSet();

		for (uint32 Stage = StartStage; Stage < EndStage; Stage++)
		{
			if (ShouldSkipStage(Stage))
			{
				continue;
			}

			if (CurrentShaderDirtySamplerSlots[Stage] && NumSamplers[Stage])
			{
				SamplerSlotMask& CurrentDirtySlotMask = Cache.DirtySlotMask[Stage];
				FD3D12SamplerState** Samplers = Cache.States[Stage];

				FD3D12UniqueSamplerTable Table;
				Table.Key.Count = NumSamplers[Stage];

				for (uint32 i = 0; i < NumSamplers[Stage]; i++)
				{
					Table.Key.SamplerID[i] = Samplers[i] ? Samplers[i]->ID : 0;
					FD3D12SamplerStateCache::CleanSlot(CurrentDirtySlotMask, i);
				}

				FD3D12UniqueSamplerTable* CachedTable = GlobalSamplerSet.Find(Table);
				if (CachedTable)
				{
					// Make sure the global sampler heap is really set on the command list before we try to find a cached descriptor table for it.
					check(DescriptorCache.IsHeapSet(GetParentDevice()->GetGlobalSamplerHeap().GetHeap()));
					check(CachedTable->GPUHandle.ptr);
					if (Stage == SF_Compute)
					{
						const uint32 RDTIndex = pRootSignature->SamplerRDTBindSlot(EShaderFrequency(Stage));
						CmdContext.GraphicsCommandList()->SetComputeRootDescriptorTable(RDTIndex, CachedTable->GPUHandle);
					}
					else
					{
						const uint32 RDTIndex = pRootSignature->SamplerRDTBindSlot(EShaderFrequency(Stage));
						CmdContext.GraphicsCommandList()->SetGraphicsRootDescriptorTable(RDTIndex, CachedTable->GPUHandle);
					}

					// We changed the descriptor table, so all resources bound to slots outside of the table's range are now dirty.
					// If a shader needs to use resources bound to these slots later, we need to set the descriptor table again to ensure those
					// descriptors are valid.
					const SamplerSlotMask OutsideCurrentTableRegisterMask = ~BitMask<SamplerSlotMask>(Table.Key.Count);
					Cache.Dirty(static_cast<EShaderFrequency>(Stage), OutsideCurrentTableRegisterMask);
				}
				else
				{
					HighLevelCacheMiss = true;
					break;
				}
			}
		}

		if (!HighLevelCacheMiss)
		{
			// Success, all the tables were found in the high level heap
			INC_DWORD_STAT_BY(STAT_NumReusedSamplerOnlineDescriptors, NumSamplers[SF_NumStandardFrequencies]);
			return;
		}
	}

	if (HighLevelCacheMiss)
	{
		// Move to per context heap strategy
		const bool bDescriptorHeapsChanged = DescriptorCache.SwitchToContextLocalSamplerHeap();
		if (bDescriptorHeapsChanged)
		{
			// If descriptor heaps changed, then all our tables are dirty again and we need to recalculate the number of slots we need.
			pfnCalcSamplersNeeded();
		}
	}

	FD3D12OnlineHeap* const SamplerHeap = DescriptorCache.GetCurrentSamplerHeap();
	check(DescriptorCache.UsingGlobalSamplerHeap() == false);
	check(SamplerHeap != &GetParentDevice()->GetGlobalSamplerHeap());
	check(DescriptorCache.IsHeapSet(SamplerHeap->GetHeap()));
	check(!DescriptorCache.IsHeapSet(GetParentDevice()->GetGlobalSamplerHeap().GetHeap()));

	if (!SamplerHeap->CanReserveSlots(NumSamplers[SF_NumStandardFrequencies]))
	{
		const bool bDescriptorHeapsChanged = SamplerHeap->RollOver();
		if (bDescriptorHeapsChanged)
		{
			// If descriptor heaps changed, then all our tables are dirty again and we need to recalculate the number of slots we need.
			pfnCalcSamplersNeeded();
		}
	}
	uint32 SamplerHeapSlot = SamplerHeap->ReserveSlots(NumSamplers[SF_NumStandardFrequencies]);

	for (uint32 Index = StartStage; Index < EndStage; Index++)
	{
		if (CurrentShaderDirtySamplerSlots[Index])
		{
			D3D12_GPU_DESCRIPTOR_HANDLE BindDescriptor = DescriptorCache.BuildSamplerTable(static_cast<EShaderFrequency>(Index), pRootSignature, Cache, CurrentShaderDirtySamplerSlots[Index], NumSamplers[Index], SamplerHeapSlot);
			DescriptorCache.SetSamplerTable(static_cast<EShaderFrequency>(Index), pRootSignature, Cache, NumSamplers[Index], BindDescriptor);
		}
	}

	SamplerHeap->SetNextSlot(SamplerHeapSlot);
}

//#if ASSERT_RESOURCE_STATES
#if 0
/** Determine if an two views intersect */
static inline bool ResourceViewsIntersect(FD3D12View* pLeftView, FD3D12View* pRightView)
{
	if (pLeftView == nullptr || pRightView == nullptr)
	{
		// Cannot intersect if at least one is null
		return false;
	}

	if ((void*)pLeftView == (void*)pRightView)
	{
		// Cannot intersect with itself
		return false;
	}

	FD3D12Resource* pRTVResource = pLeftView->GetResource();
	FD3D12Resource* pSRVResource = pRightView->GetResource();
	if (pRTVResource != pSRVResource)
	{
		// Not the same resource
		return false;
	}

	// Same resource, so see if their subresources overlap
	return !pLeftView->DoesNotOverlap(*pRightView);
}

bool FD3D12StateCache::AssertResourceStates(ED3D12PipelineType PipelineType)
{
// This requires the debug layer
#if !D3D12_SUPPORTS_DEBUG_COMMAND_LIST
	UE_LOGF(LogD3D12RHI, Log, "*** VerifyResourceStates requires the debug layer ***", this);
	return true;
#else
	// Can only verify resource states if the debug layer is used
	static const bool bWithD3DDebug = GRHIGlobals.IsDebugLayerEnabled;
	if (!bWithD3DDebug)
	{
		UE_LOGF(LogD3D12RHI, Fatal, "*** AssertResourceStates requires the debug layer ***");
		return false;
	}

	//
	// Verify common pipeline state
	//

	// Note that ray tracing pipeline shares state with compute
	const uint32 StartStage = PipelineType == ED3D12PipelineType::Graphics ? 0 : SF_Compute;
	const uint32 EndStage = PipelineType == ED3D12PipelineType::Graphics ? SF_Compute : SF_NumStandardFrequencies;

	bool bSRVIntersectsWithDepth = false;
	bool bSRVIntersectsWithStencil = false;
	for (uint32 Stage = StartStage; Stage < EndStage; Stage++)
	{
		if (ShouldSkipStage(Stage))
		{
			continue;
		}

		// UAVs
		{
			const uint32 numUAVs = PipelineState.Common.CurrentShaderUAVCounts[Stage];
			for (uint32 i = 0; i < numUAVs; i++)
			{
				FD3D12UnorderedAccessView *pCurrentView = PipelineState.Common.UAVCache.Views[Stage][i];
				if (!AssertResourceState(CmdContext.GraphicsCommandList().Get(), pCurrentView, D3D12_RESOURCE_STATE_UNORDERED_ACCESS))
				{
					return false;
				}
			}
		}

		// SRVs
		{
			const uint32 numSRVs = PipelineState.Common.CurrentShaderSRVCounts[Stage];
			for (uint32 i = 0; i < numSRVs; i++)
			{
				FD3D12ShaderResourceView* pCurrentView = PipelineState.Common.SRVCache.Views[Stage][i];
				D3D12_RESOURCE_STATES expectedState = D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE | D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE;

				FD3D12Resource* Resource = pCurrentView->GetResource();
				if (pCurrentView && Resource->IsDepthStencilResource())
				{
					expectedState = expectedState | D3D12_RESOURCE_STATE_DEPTH_READ;

					// Sanity check that we don't have a read/write hazard between the DSV and SRV.
					FD3D12DepthStencilView* DSV = PipelineState.Graphics.CurrentDepthStencilTarget;
					if (ResourceViewsIntersect(DSV, pCurrentView))
					{
						const D3D12_DEPTH_STENCIL_VIEW_DESC &DSVDesc = DSV->GetDesc();

						const bool bHasDepth = DSV->HasDepth();
						const bool bHasStencil = DSV->HasStencil();

						const bool bWritableDepth = bHasDepth && (DSVDesc.Flags & D3D12_DSV_FLAG_READ_ONLY_DEPTH) == 0;
						const bool bWritableStencil = bHasStencil && (DSVDesc.Flags & D3D12_DSV_FLAG_READ_ONLY_STENCIL) == 0;

						if (pCurrentView->IsStencilPlaneResource())
						{
							bSRVIntersectsWithStencil = true;
							if (bWritableStencil)
							{
								// DSV is being used for stencil write and this SRV is being used for read which is not supported.
								return false;
							}
						}

						if (pCurrentView->IsDepthPlaneResource())
						{
							bSRVIntersectsWithDepth = true;
							if (bWritableDepth)
							{
								// DSV is being used for depth write and this SRV is being used for read which is not supported.
								return false;
							}
						}
					}
				}

				if (!AssertResourceState(CmdContext.GraphicsCommandList().Get(), pCurrentView, expectedState))
				{
					return false;
				}
			}
		}
	}

	// Note: There is nothing special to check for compute and ray tracing pipelines
	if (PipelineType == ED3D12PipelineType::Graphics)
	{
		//
		// Verify graphics pipeline state
		//

		// DSV
		{
			FD3D12DepthStencilView* pCurrentView = PipelineState.Graphics.CurrentDepthStencilTarget;

			if (pCurrentView)
			{
				// Check if the depth/stencil resource has an SRV bound
				const D3D12_DEPTH_STENCIL_VIEW_DESC& desc = pCurrentView->GetDesc();
				const bool bDepthIsReadOnly = !!(desc.Flags & D3D12_DSV_FLAG_READ_ONLY_DEPTH);
				const bool bStencilIsReadOnly = !!(desc.Flags & D3D12_DSV_FLAG_READ_ONLY_STENCIL);

				// Decompose the view into the subresources (depth and stencil are on different planes)
				FD3D12Resource* pResource = pCurrentView->GetResource();
				for (uint32 SubresourceIndex : pCurrentView->GetViewSubset())
				{
					uint16 MipSlice;
					uint16 ArraySlice;
					uint8 PlaneSlice;
					D3D12DecomposeSubresource(SubresourceIndex,
						pResource->GetMipLevels(),
						pResource->GetArraySize(),
						MipSlice, ArraySlice, PlaneSlice);

					D3D12_RESOURCE_STATES expectedState;
					if (PlaneSlice == 0)
					{
						// Depth plane
						expectedState = bDepthIsReadOnly ? D3D12_RESOURCE_STATE_DEPTH_READ : D3D12_RESOURCE_STATE_DEPTH_WRITE;
						if (bSRVIntersectsWithDepth)
						{
							// Depth SRVs just contain the depth plane
							check(bDepthIsReadOnly);
							expectedState |=
								D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE |
								D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE;
						}
					}
					else
					{
						// Stencil plane
						expectedState = bStencilIsReadOnly ? D3D12_RESOURCE_STATE_DEPTH_READ : D3D12_RESOURCE_STATE_DEPTH_WRITE;
						if (bSRVIntersectsWithStencil)
						{
							// Stencil SRVs just contain the stencil plane
							check(bStencilIsReadOnly);
							expectedState |=
								D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE |
								D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE;
						}
					}

					bool bGoodState = !!CmdContext.DebugCommandList()->AssertResourceState(pResource->GetResource(), SubresourceIndex, expectedState);
					if (!bGoodState)
					{
						return false;
					}
				}
			}
		}

		// RTV
		{
			const uint32 numRTVs = UE_ARRAY_COUNT(PipelineState.Graphics.RenderTargetArray);
			for (uint32 i = 0; i < numRTVs; i++)
			{
				FD3D12RenderTargetView* pCurrentView = PipelineState.Graphics.RenderTargetArray[i];
				if (!AssertResourceState(CmdContext.GraphicsCommandList().Get(), pCurrentView, D3D12_RESOURCE_STATE_RENDER_TARGET))
				{
					return false;
				}
			}
		}

		// TODO: Verify vertex buffer, index buffer, and constant buffer state.
	}

	return true;
#endif
}
#endif

void FD3D12StateCache::SetRootConstants(const FUint32Vector4& Constants)
{
	if (Constants != PipelineState.Common.ShaderRootConstants)
	{
		PipelineState.Common.ShaderRootConstants = Constants;
		PipelineState.Common.bNeedSetRootConstants = true;
	}
}

void FD3D12StateCache::ClearUAVs(EShaderFrequency ShaderStage)
{
	FD3D12UnorderedAccessViewCache& Cache = PipelineState.Common.UAVCache;

	for (uint32 i = 0; i < MAX_UAVS; ++i)
	{
		if(Cache.Views[ShaderStage][i] != nullptr)
		{
			FD3D12UnorderedAccessViewCache::DirtySlot(Cache.DirtySlotMask[ShaderStage], i);
		}
		Cache.Views[ShaderStage][i] = nullptr;
	}
}

void FD3D12StateCache::SetUAV(EShaderFrequency ShaderStage, uint32 SlotIndex, FD3D12UnorderedAccessView* UAV)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12SetUnorderedAccessViewTime);

	FD3D12UnorderedAccessViewCache& Cache = PipelineState.Common.UAVCache;
	if (Cache.Views[ShaderStage][SlotIndex] == UAV)
	{
		return;
	}

	// When setting UAV's for Graphics, it wipes out all existing bound resources.
	const bool bIsCompute = ShaderStage == SF_Compute;
	Cache.StartSlot[ShaderStage] = bIsCompute ? FMath::Min(SlotIndex, Cache.StartSlot[ShaderStage]) : 0;

	Cache.Views[ShaderStage][SlotIndex] = UAV;
	FD3D12UnorderedAccessViewCache::DirtySlot(Cache.DirtySlotMask[ShaderStage], SlotIndex);

	if (UAV)
	{
		Cache.Resources[ShaderStage][SlotIndex] = UAV->GetResource();

		FD3D12Resource* CounterResource = UAV->GetCounterResource();
		if (CounterResource)
		{ 
			checkNoEntry(); // @todo fix this. UAV counters are not threadsafe. Initialization could happen out-of-order
			/*&& (!UAV->IsCounterResourceInitialized() || InitialCount != -1))
			{
				FD3D12Device* Device = CounterResource->GetParentDevice();
				FD3D12ResourceLocation UploadBufferLocation(Device);

				uint32* CounterUploadHeapData = static_cast<uint32*>(CmdContext.ConstantsAllocator.Allocate(sizeof(uint32), UploadBufferLocation, nullptr));

				// Initialize the counter to 0 if it's not been previously initialized and the UAVInitialCount is -1, if not use the value that was passed.
				*CounterUploadHeapData = (!UAV->IsCounterResourceInitialized() && InitialCount == -1) ? 0 : InitialCount;

				// Transition to copy dest
				CmdContext.TransitionResource(CounterResource, D3D12_RESOURCE_STATE_UNORDERED_ACCESS, D3D12_RESOURCE_STATE_COPY_DEST, 0);
				CmdContext.FlushResourceBarriers();

				CmdContext.GraphicsCommandList()->CopyBufferRegion(
					CounterResource->GetResource(),
					0,
					UploadBufferLocation.GetResource()->GetResource(),
					UploadBufferLocation.GetOffsetFromBaseOfResource(),
					4);

				// Restore UAV state
				CmdContext.TransitionResource(CounterResource, D3D12_RESOURCE_STATE_COPY_DEST, D3D12_RESOURCE_STATE_UNORDERED_ACCESS, 0);

				CmdContext.UpdateResidency(CounterResource);

				UAV->MarkCounterResourceInitialized();
			}*/
		}
	}
	else
	{
		Cache.Resources[ShaderStage][SlotIndex] = nullptr;
	}
}

void FD3D12StateCache::SetBlendFactor(const float BlendFactor[4])
{
	if (FMemory::Memcmp(PipelineState.Graphics.CurrentBlendFactor, BlendFactor, sizeof(PipelineState.Graphics.CurrentBlendFactor)))
	{
		FMemory::Memcpy(PipelineState.Graphics.CurrentBlendFactor, BlendFactor, sizeof(PipelineState.Graphics.CurrentBlendFactor));
		bNeedSetBlendFactor = true;
	}
}

void FD3D12StateCache::SetStencilRef(uint32 StencilRef)
{
	if (PipelineState.Graphics.CurrentReferenceStencil != StencilRef)
	{
		PipelineState.Graphics.CurrentReferenceStencil = StencilRef;
		bNeedSetStencilRef = true;
	}
}

void FD3D12StateCache::SetNewShaderData(EShaderFrequency InFrequency, const FD3D12ShaderData* InShaderData)
{
	PipelineState.Common.CurrentShaderSamplerCounts[InFrequency] = InShaderData ? InShaderData->ResourceCounts.NumSamplers : 0;
	PipelineState.Common.CurrentShaderSRVCounts[InFrequency] = InShaderData ? InShaderData->ResourceCounts.NumSRVs : 0;
	PipelineState.Common.CurrentShaderCBCounts[InFrequency] = InShaderData ? InShaderData->ResourceCounts.NumCBs : 0;
	PipelineState.Common.CurrentShaderUAVCounts[InFrequency] = InShaderData ? InShaderData->ResourceCounts.NumUAVs : 0;

	// Shader changed so its resource table is dirty
	SetDirtyUniformBuffers(CmdContext, InFrequency);
}

void FD3D12StateCache::SetComputePipelineState(FD3D12ComputePipelineState* ComputePipelineState)
{
	check(ComputePipelineState);

	FD3D12ComputePipelineState* CurrentComputePipelineState = PipelineState.Compute.CurrentPipelineStateObject;
	const bool bForceSet = CurrentComputePipelineState == nullptr;

	if (bForceSet || CurrentComputePipelineState != ComputePipelineState)
	{
		if (bForceSet || CurrentComputePipelineState->RootSignature != ComputePipelineState->RootSignature)
		{
			PipelineState.Compute.bNeedSetRootSignature = true;
		}

		if (bForceSet || CurrentComputePipelineState->GetComputeShader() != ComputePipelineState->GetComputeShader())
		{
			SetNewShaderData(SF_Compute, FD3D12DynamicRHI::ResourceCast(ComputePipelineState->GetComputeShader()));
		}

		// Save the PSO
		PipelineState.Common.bNeedSetPSO = true;
		PipelineState.Compute.CurrentPipelineStateObject = ComputePipelineState;

		// Set the PSO
		InternalSetPipelineState(ComputePipelineState->PipelineState);
	}
}

void FD3D12StateCache::SetGraphicsPipelineState(FD3D12GraphicsPipelineState* GraphicsPipelineState)
{
	check(GraphicsPipelineState);

	FD3D12GraphicsPipelineState* CurrentGraphicsPipelineState = PipelineState.Graphics.CurrentPipelineStateObject;
	const bool bForceSet = CurrentGraphicsPipelineState == nullptr;

	if (bForceSet || CurrentGraphicsPipelineState != GraphicsPipelineState)
	{
		if (bForceSet || CurrentGraphicsPipelineState->GetVertexShader() != GraphicsPipelineState->GetVertexShader())
		{
			SetNewShaderData(SF_Vertex, GraphicsPipelineState->GetVertexShader());
		}

#if PLATFORM_SUPPORTS_MESH_SHADERS
		if (bForceSet || CurrentGraphicsPipelineState->GetMeshShader() != GraphicsPipelineState->GetMeshShader())
		{
			SetNewShaderData(SF_Mesh, GraphicsPipelineState->GetMeshShader());
		}

		if (bForceSet || CurrentGraphicsPipelineState->GetAmplificationShader() != GraphicsPipelineState->GetAmplificationShader())
		{
			SetNewShaderData(SF_Amplification, GraphicsPipelineState->GetAmplificationShader());
		}
#endif

		if (bForceSet || CurrentGraphicsPipelineState->GetPixelShader() != GraphicsPipelineState->GetPixelShader())
		{
			SetNewShaderData(SF_Pixel, GraphicsPipelineState->GetPixelShader());
		}

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		if (bForceSet || CurrentGraphicsPipelineState->GetGeometryShader() != GraphicsPipelineState->GetGeometryShader())
		{
			SetNewShaderData(SF_Geometry, GraphicsPipelineState->GetGeometryShader());
		}
#endif

		// See if we need to change the root signature
		if (bForceSet || CurrentGraphicsPipelineState->RootSignature != GraphicsPipelineState->RootSignature)
		{
			PipelineState.Graphics.bNeedSetRootSignature = true;
		}

		PipelineState.Graphics.StreamStrides = GraphicsPipelineState->StreamStrides;

		// Save the PSO
		PipelineState.Common.bNeedSetPSO = true;
		PipelineState.Graphics.CurrentPipelineStateObject = GraphicsPipelineState;

		EPrimitiveType PrimitiveType = GraphicsPipelineState->PipelineStateInitializer.PrimitiveType;
		if (PipelineState.Graphics.CurrentPrimitiveType != PrimitiveType)
		{
			PipelineState.Graphics.CurrentPrimitiveType = PrimitiveType;
			PipelineState.Graphics.CurrentPrimitiveTopology = GetD3D12PrimitiveType(PrimitiveType);
			bNeedSetPrimitiveTopology = true;

			static_assert(PT_Num == 6, "This computation needs to be updated, matching that of GetVertexCountForPrimitiveCount()");
			PipelineState.Graphics.PrimitiveTypeFactor = (PrimitiveType == PT_TriangleList) ? 3 : (PrimitiveType == PT_LineList) ? 2 : (PrimitiveType == PT_RectList) ? 3 : 1;
			PipelineState.Graphics.PrimitiveTypeOffset = (PrimitiveType == PT_TriangleStrip) ? 2 : 0;
		}

		// Set the PSO
		InternalSetPipelineState(GraphicsPipelineState->PipelineState);
	}
}

void FD3D12StateCache::InternalSetIndexBuffer(FD3D12Resource* Resource)
{
	CmdContext.UpdateResidency(Resource);
	CmdContext.GraphicsCommandList()->IASetIndexBuffer(&PipelineState.Graphics.IBCache.CurrentIndexBufferView);

	if (Resource->RequiresResourceStateTracking())
	{
		check(Resource->GetSubresourceCount() == 1);
	}
}

void FD3D12StateCache::InternalSetStreamSource(FD3D12ResourceLocation* VertexBufferLocation, uint32 StreamIndex, uint32 Stride, uint32 Offset)
{
	// If we have a vertex buffer location, that location should also have an underlying resource.
	check(VertexBufferLocation == nullptr || VertexBufferLocation->GetResource());

	check(StreamIndex < ARRAYSIZE(PipelineState.Graphics.VBCache.CurrentVertexBufferResources));

	__declspec(align(16)) D3D12_VERTEX_BUFFER_VIEW NewView;
	NewView.BufferLocation = (VertexBufferLocation) ? VertexBufferLocation->GetGPUVirtualAddress() + Offset : 0;
	NewView.StrideInBytes = Stride;
	NewView.SizeInBytes = (VertexBufferLocation) ? VertexBufferLocation->GetSize() - Offset : 0; // Make sure we account for how much we offset into the VB

	D3D12_VERTEX_BUFFER_VIEW& CurrentView = PipelineState.Graphics.VBCache.CurrentVertexBufferViews[StreamIndex];

	if (NewView.BufferLocation != CurrentView.BufferLocation ||
		NewView.StrideInBytes != CurrentView.StrideInBytes ||
		NewView.SizeInBytes != CurrentView.SizeInBytes ||
		GD3D12SkipStateCaching)
	{
		bNeedSetVB = true;
		PipelineState.Graphics.VBCache.CurrentVertexBufferResources[StreamIndex] = VertexBufferLocation;

		if (VertexBufferLocation != nullptr)
		{
			PipelineState.Graphics.VBCache.Resources[StreamIndex] = VertexBufferLocation->GetResource();
			FMemory::Memcpy(CurrentView, NewView);
			PipelineState.Graphics.VBCache.BoundVBMask |= ((VBSlotMask)1 << StreamIndex);
		}
		else
		{
			FMemory::Memzero(&CurrentView, sizeof(CurrentView));
			PipelineState.Graphics.VBCache.CurrentVertexBufferResources[StreamIndex] = nullptr;
			PipelineState.Graphics.VBCache.Resources[StreamIndex] = nullptr;

			PipelineState.Graphics.VBCache.BoundVBMask &= ~((VBSlotMask)1 << StreamIndex);
		}

		if (PipelineState.Graphics.VBCache.BoundVBMask)
		{
			PipelineState.Graphics.VBCache.MaxBoundVertexBufferIndex = FMath::FloorLog2(PipelineState.Graphics.VBCache.BoundVBMask);
		}
		else
		{
			PipelineState.Graphics.VBCache.MaxBoundVertexBufferIndex = INDEX_NONE;
		}
	}
}

void FD3D12StateCache::SetShaderResourceView(EShaderFrequency ShaderFrequency, FD3D12ShaderResourceView* SRV, uint32 ResourceIndex)
{
	//SCOPE_CYCLE_COUNTER(STAT_D3D12SetShaderResourceViewTime);

	check(ResourceIndex < MAX_SRVS);
	FD3D12ShaderResourceViewCache& Cache = PipelineState.Common.SRVCache;
	auto& CurrentShaderResourceViews = Cache.Views[ShaderFrequency];

	// If we are binding an SRV that isn't actually backed by a resource, we shouldn't act as if we are non-null.
	// Fall back to the 'null' binding path.
	if (SRV && SRV->GetResource() == nullptr)
	{
		SRV = nullptr;
	}

	if ((CurrentShaderResourceViews[ResourceIndex] != SRV) || GD3D12SkipStateCaching)
	{
		if (SRV != nullptr)
		{
			// Mark the SRVs as not cleared
			bSRVSCleared = false;

			Cache.BoundMask[ShaderFrequency] |= ((SRVSlotMask)1 << ResourceIndex);
			Cache.Resources[ShaderFrequency][ResourceIndex] = SRV->GetResource();
		}
		else
		{
			Cache.BoundMask[ShaderFrequency] &= ~((SRVSlotMask)1 << ResourceIndex);
			Cache.Resources[ShaderFrequency][ResourceIndex] = nullptr;
		}

		// Find the highest set SRV
		Cache.MaxBoundIndex[ShaderFrequency] =
			(Cache.BoundMask[ShaderFrequency] == 0)? INDEX_NONE :
#if MAX_SRVS > 32
			FMath::FloorLog2_64(Cache.BoundMask[ShaderFrequency]);
#else
			FMath::FloorLog2(Cache.BoundMask[ShaderFrequency]);
#endif

		CurrentShaderResourceViews[ResourceIndex] = SRV;
		FD3D12ShaderResourceViewCache::DirtySlot(Cache.DirtySlotMask[ShaderFrequency], ResourceIndex);
	}
}

void FD3D12StateCache::SetRenderTargets(uint32 NumSimultaneousRenderTargets, FD3D12RenderTargetView** RTArray, FD3D12DepthStencilView* DSTarget)
{
	if (PipelineState.Graphics.CurrentDepthStencilTarget != DSTarget)
	{
		PipelineState.Graphics.CurrentDepthStencilTarget = DSTarget;
		bNeedSetRTs = true;
	}

	// Update the render targets
	PipelineState.Graphics.CurrentNumberOfRenderTargets = 0;
	for (uint32 Index = 0; Index < UE_ARRAY_COUNT(PipelineState.Graphics.RenderTargetArray); ++Index)
	{
		FD3D12RenderTargetView* RTV = Index < NumSimultaneousRenderTargets
			? RTArray[Index]
			: nullptr;

		if (RTV)
		{
			PipelineState.Graphics.CurrentNumberOfRenderTargets++;
		}

		if (PipelineState.Graphics.RenderTargetArray[Index] != RTV)
		{
			PipelineState.Graphics.RenderTargetArray[Index] = RTV;
			bNeedSetRTs = true;
		}
	}
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
void FD3D12StateCache::SetNewBindlessResourcesHeap(FD3D12DescriptorHeap* InHeap)
{
	DescriptorCache.SetBindlessResourcesHeap(InHeap);
	BindlessHeapsInvalidated();
}

void FD3D12StateCache::SetNewBindlessSamplerHeap(FD3D12DescriptorHeap* InHeap)
{
	DescriptorCache.SetBindlessSamplersHeap(InHeap);
	BindlessHeapsInvalidated();
}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
