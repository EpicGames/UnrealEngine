// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of Device Context State Caching to improve draw
//	thread performance by removing redundant device context calls.

#pragma once

#include "D3D12ConstantBuffer.h"
#include "D3D12DirectCommandListManager.h"
#include "D3D12DescriptorCache.h"
// TODO reorder includes so we just include D3D12PipelineState.h here
#include COMPILED_PLATFORM_HEADER(D3D12PipelineState.h)
#include "D3D12Resources.h"

#include "Math/IntVector.h"

enum class EShaderParameterTypeMask : uint16;

class FD3D12SamplerState;

struct FD3D12ComputePipelineState;
struct FD3D12GraphicsPipelineState;
struct FD3D12PipelineState;
struct FD3D12ShaderData;

//-----------------------------------------------------------------------------
//	Configuration
//-----------------------------------------------------------------------------

// If set, includes a runtime toggle console command for debugging D3D12  state caching.
// ("TOGGLESTATECACHE")
#define D3D12_STATE_CACHE_RUNTIME_TOGGLE 0

// Uncomment only for debugging of the descriptor heap management; this is very noisy
//#define VERBOSE_DESCRIPTOR_HEAP_DEBUG 1

// The number of view descriptors available per (online) descriptor heap, depending on hardware tier
#define NUM_SAMPLER_DESCRIPTORS D3D12_MAX_SHADER_VISIBLE_SAMPLER_HEAP_SIZE

// Keep set state functions inline to reduce call overhead
#define D3D12_STATE_CACHE_INLINE FORCEINLINE_DEBUGGABLE

#if D3D12_STATE_CACHE_RUNTIME_TOGGLE
extern bool GD3D12SkipStateCaching;
#else
static const bool GD3D12SkipStateCaching = false;
#endif

extern int32 GGlobalResourceDescriptorHeapSize;
extern int32 GGlobalSamplerDescriptorHeapSize;

extern int32 GGlobalSamplerHeapSize;
extern int32 GOnlineDescriptorHeapSize;
extern int32 GOnlineDescriptorHeapBlockSize;
extern int32 GBindlessOnlineDescriptorHeapSize;
extern int32 GBindlessOnlineDescriptorHeapBlockSize;

enum class ED3D12PipelineType : uint8
{
	Graphics,
	Compute,
	RayTracing,
};

namespace ED3D12VRSCombinerStages
{
	constexpr int32 PerPrimitive	= 0;
	constexpr int32 ScreenSpace		= PerPrimitive + 1;
	constexpr int32 Num				= ScreenSpace + 1;
};

#define MAX_VBS			D3D12_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT

typedef uint32 VBSlotMask;
static_assert((8 * sizeof(VBSlotMask)) >= MAX_VBS, "VBSlotMask isn't large enough to cover all VBs. Please increase the size.");

struct FD3D12VertexBufferCache
{
	FD3D12VertexBufferCache()
	{
		Clear();
	};

	inline void Clear()
	{
		FMemory::Memzero(CurrentVertexBufferViews, sizeof(CurrentVertexBufferViews));
		FMemory::Memzero(CurrentVertexBufferResources, sizeof(CurrentVertexBufferResources));
		FMemory::Memzero(Resources, sizeof(Resources));
		MaxBoundVertexBufferIndex = INDEX_NONE;
		BoundVBMask = 0;
	}

	D3D12_VERTEX_BUFFER_VIEW CurrentVertexBufferViews[MAX_VBS];
	FD3D12ResourceLocation* CurrentVertexBufferResources[MAX_VBS];
	FD3D12Resource* Resources[MAX_VBS];
	int32 MaxBoundVertexBufferIndex;
	VBSlotMask BoundVBMask;
};

struct FD3D12IndexBufferCache
{
	FD3D12IndexBufferCache()
	{
		Clear();
	}

	inline void Clear()
	{
		FMemory::Memzero(&CurrentIndexBufferView, sizeof(CurrentIndexBufferView));
	}

	D3D12_INDEX_BUFFER_VIEW CurrentIndexBufferView;
};

template<typename ResourceSlotMask>
struct FD3D12ResourceCache
{
	static inline void CleanSlot(ResourceSlotMask& SlotMask, uint32 SlotIndex)
	{
		SlotMask &= ~((ResourceSlotMask)1 << SlotIndex);
	}

	static inline void CleanSlots(ResourceSlotMask& SlotMask, uint32 NumSlots)
	{
		SlotMask &= (NumSlots >= std::numeric_limits<ResourceSlotMask>::digits) ? 0 : ~(((ResourceSlotMask)1 << NumSlots) - 1);
	}

	static inline void DirtySlot(ResourceSlotMask& SlotMask, uint32 SlotIndex)
	{
		SlotMask |= ((ResourceSlotMask)1 << SlotIndex);
	}

	static inline bool IsSlotDirty(const ResourceSlotMask& SlotMask, uint32 SlotIndex)
	{
		return (SlotMask & ((ResourceSlotMask)1 << SlotIndex)) != 0;
	}

	// Mark a specific shader stage as dirty.
	inline void Dirty(EShaderFrequency ShaderFrequency, const ResourceSlotMask& SlotMask = -1)
	{
		checkSlow(ShaderFrequency < UE_ARRAY_COUNT(DirtySlotMask));
		DirtySlotMask[ShaderFrequency] |= SlotMask;
	}

	// Mark specified bind slots, on all graphics stages, as dirty.
	inline void DirtyGraphics(const ResourceSlotMask& SlotMask = -1)
	{
		Dirty(SF_Vertex, SlotMask);
		Dirty(SF_Mesh, SlotMask);
		Dirty(SF_Amplification, SlotMask);
		Dirty(SF_Pixel, SlotMask);
		Dirty(SF_Geometry, SlotMask);
	}

	// Mark specified bind slots on compute as dirty.
	inline void DirtyCompute(const ResourceSlotMask& SlotMask = -1)
	{
		Dirty(SF_Compute, SlotMask);
	}

	// Mark specified bind slots on graphics and compute as dirty.
	inline void DirtyAll(const ResourceSlotMask& SlotMask = -1)
	{
		DirtyGraphics(SlotMask);
		DirtyCompute(SlotMask);
	}

	ResourceSlotMask DirtySlotMask[SF_NumStandardFrequencies];
};

struct FD3D12ConstantBufferCache : public FD3D12ResourceCache<CBVSlotMask>
{
	FD3D12ConstantBufferCache()
	{
		Clear();
	}

	inline void Clear()
	{
		DirtyAll();

		FMemory::Memzero(CurrentGPUVirtualAddress, sizeof(CurrentGPUVirtualAddress));
		FMemory::Memzero(Resources, sizeof(Resources));
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
		FMemory::Memzero(CBHandles, sizeof(CBHandles));
#endif
	}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
	D3D12_CPU_DESCRIPTOR_HANDLE CBHandles[SF_NumStandardFrequencies][MAX_CBS];
#endif
	D3D12_GPU_VIRTUAL_ADDRESS CurrentGPUVirtualAddress[SF_NumStandardFrequencies][MAX_CBS];
	FD3D12Resource* Resources[SF_NumStandardFrequencies][MAX_CBS];
};

struct FD3D12ShaderResourceViewCache : public FD3D12ResourceCache<SRVSlotMask>
{
	FD3D12ShaderResourceViewCache()
	{
		Clear();
	}

	inline void Clear()
	{
		DirtyAll();

		FMemory::Memzero(Resources);
		FMemory::Memzero(BoundMask);
		
		for (int32& Index : MaxBoundIndex)
		{
			Index = INDEX_NONE;
		}

		for (int32 FrequencyIdx = 0; FrequencyIdx < SF_NumStandardFrequencies; ++FrequencyIdx)
		{
			for (int32 SRVIdx = 0; SRVIdx < MAX_SRVS; ++SRVIdx)
			{
				Views[FrequencyIdx][SRVIdx] = nullptr;
			}
		}
	}

	FD3D12ShaderResourceView* Views[SF_NumStandardFrequencies][MAX_SRVS];
	FD3D12Resource* Resources[SF_NumStandardFrequencies][MAX_SRVS];

	SRVSlotMask BoundMask[SF_NumStandardFrequencies];
	int32 MaxBoundIndex[SF_NumStandardFrequencies];
};

struct FD3D12UnorderedAccessViewCache : public FD3D12ResourceCache<UAVSlotMask>
{
	FD3D12UnorderedAccessViewCache()
	{
		Clear();
	}

	inline void Clear()
	{
		DirtyAll();

		FMemory::Memzero(Views);
		FMemory::Memzero(Resources);

		for (uint32& Index : StartSlot)
		{
			Index = INDEX_NONE;
		}
	}

	FD3D12UnorderedAccessView* Views[SF_NumStandardFrequencies][MAX_UAVS];
	FD3D12Resource* Resources[SF_NumStandardFrequencies][MAX_UAVS];
	uint32 StartSlot[SF_NumStandardFrequencies];
};

struct FD3D12SamplerStateCache : public FD3D12ResourceCache<SamplerSlotMask>
{
	FD3D12SamplerStateCache()
	{
		Clear();
	}

	inline void Clear()
	{
		DirtyAll();

		FMemory::Memzero(States);
	}

	FD3D12SamplerState* States[SF_NumStandardFrequencies][MAX_SAMPLERS];
};


static inline D3D_PRIMITIVE_TOPOLOGY GetD3D12PrimitiveType(uint32 PrimitiveType)
{
	static const uint8 D3D12PrimitiveType[] =
	{
		D3D_PRIMITIVE_TOPOLOGY_TRIANGLELIST,               // PT_TriangleList
		D3D_PRIMITIVE_TOPOLOGY_TRIANGLESTRIP,              // PT_TriangleStrip
		D3D_PRIMITIVE_TOPOLOGY_LINELIST,                   // PT_LineList
		0,                                                 // PT_QuadList
		D3D_PRIMITIVE_TOPOLOGY_POINTLIST,                  // PT_PointList
#if defined(D3D12RHI_PRIMITIVE_TOPOLOGY_RECTLIST)          // PT_RectList
		D3D_PRIMITIVE_TOPOLOGY_RECTLIST,
#else
		0,
#endif
	};
	static_assert(UE_ARRAY_COUNT(D3D12PrimitiveType) == PT_Num, "Primitive lookup table is wrong size");

	D3D_PRIMITIVE_TOPOLOGY D3DType = (D3D_PRIMITIVE_TOPOLOGY) D3D12PrimitiveType[PrimitiveType];
	checkf(D3DType, TEXT("Unknown primitive type: %u"), PrimitiveType);
	return D3DType;
}

//-----------------------------------------------------------------------------
//	FD3D12StateCache Class Definition
//-----------------------------------------------------------------------------
class FD3D12StateCache final : public FD3D12DeviceChild, public FD3D12SingleNodeGPUObject
{
	friend class FD3D12DynamicRHI;

protected:
	FD3D12CommandContext& CmdContext;

	bool bNeedSetVB = true;
	bool bNeedSetRTs = true;
	bool bNeedSetViewports = true;
	bool bNeedSetScissorRects = true;
	bool bNeedSetPrimitiveTopology = true;
	bool bNeedSetBlendFactor = true;
	bool bNeedSetStencilRef = true;
	bool bNeedSetDepthBounds = true;
	bool bNeedSetShadingRate = true;
	bool bNeedSetShadingRateImage = true;

	bool bSRVSCleared = true;

	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier;

	struct
	{
		struct FGraphicsState
		{
			// Cache
			TRefCountPtr<FD3D12GraphicsPipelineState> CurrentPipelineStateObject = nullptr;

			// Note: Current root signature is part of the bound shader state, which is part of the PSO
			bool bNeedSetRootSignature;

			// Depth Stencil State Cache
			uint32 CurrentReferenceStencil = D3D12_DEFAULT_STENCIL_REFERENCE;

			// Blend State Cache
			float CurrentBlendFactor[4] = 
			{
				D3D12_DEFAULT_BLEND_FACTOR_RED,
				D3D12_DEFAULT_BLEND_FACTOR_GREEN,
				D3D12_DEFAULT_BLEND_FACTOR_BLUE,
				D3D12_DEFAULT_BLEND_FACTOR_ALPHA
			};

			// Viewport
			uint32	       CurrentNumberOfViewports = 0;
			D3D12_VIEWPORT CurrentViewport[D3D12_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE] = {};

			// Vertex Buffer State
			FD3D12VertexBufferCache VBCache = {};

			// Index Buffer State
			FD3D12IndexBufferCache IBCache = {};

			// Primitive Topology State
			EPrimitiveType CurrentPrimitiveType = PT_Num;
			D3D_PRIMITIVE_TOPOLOGY CurrentPrimitiveTopology = D3D_PRIMITIVE_TOPOLOGY_UNDEFINED;
			uint32 PrimitiveTypeFactor;
			uint32 PrimitiveTypeOffset;

			// Input Layout State
			D3D12_RECT CurrentScissorRects[D3D12_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE] = {};
			uint32 CurrentNumberOfScissorRects = 0;

			TStaticArray<uint16, MaxVertexElementCount> StreamStrides;

			FD3D12RenderTargetView* RenderTargetArray[D3D12_SIMULTANEOUS_RENDER_TARGET_COUNT] = {};
			uint32 CurrentNumberOfRenderTargets = 0;

			FD3D12DepthStencilView* CurrentDepthStencilTarget = nullptr;

			float MinDepth = 0.0f;
			float MaxDepth = 1.0f;

			EVRSShadingRate  DrawShadingRate = EVRSShadingRate::VRSSR_1x1;

			TStaticArray<EVRSRateCombiner, ED3D12VRSCombinerStages::Num> Combiners;

			FD3D12Resource*  ShadingRateImage = nullptr;

			FGraphicsState()
			{
				for (auto& Combiner : Combiners)
				{
					Combiner = EVRSRateCombiner::VRSRB_Passthrough;
				}
			}
		} Graphics = {};

		struct
		{
			// Cache
			TRefCountPtr<FD3D12ComputePipelineState> CurrentPipelineStateObject = nullptr;

			// Note: Current root signature is part of the bound compute shader, which is part of the PSO
			bool bNeedSetRootSignature;

			// Need to cache compute budget, as we need to reset if after PSO changes
			EAsyncComputeBudget ComputeBudget = EAsyncComputeBudget::EAll_4;
		} Compute = {};

		struct
		{
			FD3D12ShaderResourceViewCache  SRVCache     = {};
			FD3D12ConstantBufferCache      CBVCache     = {};
			FD3D12UnorderedAccessViewCache UAVCache     = {};
			FD3D12SamplerStateCache        SamplerCache = {};

			// PSO
			ID3D12PipelineState* CurrentPipelineStateObject = nullptr;
			bool bNeedSetPSO;

			// Root Constants
			bool bNeedSetRootConstants;
			FUint32Vector4 ShaderRootConstants = FUint32Vector4(0, 0, 0, 0);

			uint32 CurrentShaderSamplerCounts[SF_NumStandardFrequencies] = {};
			uint32 CurrentShaderSRVCounts    [SF_NumStandardFrequencies] = {};
			uint32 CurrentShaderCBCounts     [SF_NumStandardFrequencies] = {};
			uint32 CurrentShaderUAVCounts    [SF_NumStandardFrequencies] = {};

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			TArray<FD3D12ShaderResourceView*> QueuedBindlessSRVs[SF_NumStandardFrequencies];
			TArray<FD3D12UnorderedAccessView*> QueuedBindlessUAVs[SF_NumStandardFrequencies];
#endif
		} Common = {};
	} PipelineState = {};

	FD3D12DescriptorCache DescriptorCache;

	void InternalSetIndexBuffer(FD3D12Resource* Resource);
	void InternalSetStreamSource(FD3D12ResourceLocation* VertexBufferLocation, uint32 StreamIndex, uint32 Stride, uint32 Offset);

	bool InternalSetDescriptorHeaps(bool bBindless);
	bool InternalSetRootSignature(ED3D12PipelineType InPipelineType, const FD3D12RootSignature* InRootSignature, bool bForce);
	void InternalSetPipelineState(FD3D12PipelineState* InPipelineState);

private:

	// SetDirtyUniformBuffers and SetPipelineState helper functions are required
	// to allow using FD3D12CommandContext type which is not defined at this point.
	// Making ContextType a template parameter delays instantiation of these functions.

	template <typename ContextType>
	static void SetDirtyUniformBuffers(ContextType& Context, EShaderFrequency Frequency)
	{
		Context.DirtyUniformBuffers[Frequency] = 0xffff;
	}

public:

	FD3D12DescriptorCache* GetDescriptorCache()
	{
		return &DescriptorCache;
	}

	FD3D12GraphicsPipelineState* GetGraphicsPipelineState() const
	{
		return PipelineState.Graphics.CurrentPipelineStateObject;
	}

	FD3D12ComputePipelineState* GetComputePipelineState() const
	{
		return PipelineState.Compute.CurrentPipelineStateObject;
	}

	EPrimitiveType GetGraphicsPipelinePrimitiveType() const
	{
		return PipelineState.Graphics.CurrentPrimitiveType;
	}

	uint32 GetVertexCount(uint32 NumPrimitives)
	{
		return PipelineState.Graphics.PrimitiveTypeFactor * NumPrimitives + PipelineState.Graphics.PrimitiveTypeOffset;
	}

	void ClearSRVs();

	void ClearResourceViewCaches(EShaderFrequency ShaderFrequency, FD3D12ResourceLocation*& ResourceLocation, EShaderParameterTypeMask ShaderParameterTypeMask);

	void SetShaderResourceView(EShaderFrequency ShaderFrequency, FD3D12ShaderResourceView* SRV, uint32 ResourceIndex);
	
	void SetScissorRects(uint32 Count, const D3D12_RECT* const ScissorRects);
	void SetScissorRect(const D3D12_RECT& ScissorRect);

	D3D12_STATE_CACHE_INLINE const D3D12_RECT& GetScissorRect(int32 Index = 0) const
	{
		return PipelineState.Graphics.CurrentScissorRects[Index];
	}

	void SetViewport(const D3D12_VIEWPORT& Viewport);
	void SetViewports(uint32 Count, const D3D12_VIEWPORT* const Viewports);

	D3D12_STATE_CACHE_INLINE uint32 GetNumViewports() const
	{
		return PipelineState.Graphics.CurrentNumberOfViewports;
	}

	D3D12_STATE_CACHE_INLINE const D3D12_VIEWPORT& GetViewport(int32 Index = 0) const
	{
		return PipelineState.Graphics.CurrentViewport[Index];
	}

	D3D12_STATE_CACHE_INLINE void GetViewports(uint32* Count, D3D12_VIEWPORT* Viewports) const
	{
		check(*Count);
		if (Viewports) //NULL is legal if you just want count
		{
			//as per d3d spec
			const int32 StorageSizeCount = (int32)(*Count);
			const int32 CopyCount = FMath::Min(FMath::Min(StorageSizeCount, (int32)PipelineState.Graphics.CurrentNumberOfViewports), D3D12_VIEWPORT_AND_SCISSORRECT_OBJECT_COUNT_PER_PIPELINE);
			if (CopyCount > 0)
			{
				FMemory::Memcpy(Viewports, &PipelineState.Graphics.CurrentViewport[0], sizeof(D3D12_VIEWPORT) * CopyCount);
			}
			//remaining viewports in supplied array must be set to zero
			if (StorageSizeCount > CopyCount)
			{
				FMemory::Memset(&Viewports[CopyCount], 0, sizeof(D3D12_VIEWPORT) * (StorageSizeCount - CopyCount));
			}
		}
		*Count = PipelineState.Graphics.CurrentNumberOfViewports;
	}

	D3D12_STATE_CACHE_INLINE void SetSamplerState(EShaderFrequency ShaderFrequency, FD3D12SamplerState* SamplerState, uint32 SamplerIndex)
	{
		check(SamplerIndex < MAX_SAMPLERS);
		auto& Samplers = PipelineState.Common.SamplerCache.States[ShaderFrequency];
		if ((Samplers[SamplerIndex] != SamplerState) || GD3D12SkipStateCaching)
		{
			Samplers[SamplerIndex] = SamplerState;
			FD3D12SamplerStateCache::DirtySlot(PipelineState.Common.SamplerCache.DirtySlotMask[ShaderFrequency], SamplerIndex);
		}
	}

	D3D12_STATE_CACHE_INLINE void SetConstantsFromUniformBuffer(EShaderFrequency ShaderFrequency, uint32 SlotIndex, FD3D12UniformBuffer* UniformBuffer)
	{
		check(SlotIndex < MAX_CBS);
		FD3D12ConstantBufferCache& CBVCache = PipelineState.Common.CBVCache;
		D3D12_GPU_VIRTUAL_ADDRESS& CurrentGPUVirtualAddress = CBVCache.CurrentGPUVirtualAddress[ShaderFrequency][SlotIndex];

		if (UniformBuffer && UniformBuffer->ResourceLocation.GetGPUVirtualAddress())
		{
			const FD3D12ResourceLocation& ResourceLocation = UniformBuffer->ResourceLocation;
			// Only update the constant buffer if it has changed.
			if (ResourceLocation.GetGPUVirtualAddress() != CurrentGPUVirtualAddress)
			{
				CurrentGPUVirtualAddress = ResourceLocation.GetGPUVirtualAddress();
				CBVCache.Resources[ShaderFrequency][SlotIndex] = ResourceLocation.GetResource();
				FD3D12ConstantBufferCache::DirtySlot(CBVCache.DirtySlotMask[ShaderFrequency], SlotIndex);
			}

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			CBVCache.CBHandles[ShaderFrequency][SlotIndex] = UniformBuffer->View->GetOfflineCpuHandle();
#endif
		}
		else if (CurrentGPUVirtualAddress != 0)
		{
			CurrentGPUVirtualAddress = 0;
			CBVCache.Resources[ShaderFrequency][SlotIndex] = {};
			FD3D12ConstantBufferCache::DirtySlot(CBVCache.DirtySlotMask[ShaderFrequency], SlotIndex);
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			CBVCache.CBHandles[ShaderFrequency][SlotIndex].ptr = 0;
#endif
		}
		else
		{
#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			CBVCache.CBHandles[ShaderFrequency][SlotIndex].ptr = 0;
#endif
		}
	}

	D3D12_STATE_CACHE_INLINE void SetConstantBuffer(EShaderFrequency ShaderFrequency, FD3D12ConstantBuffer& Buffer, bool bDiscardSharedConstants)
	{
		FD3D12ResourceLocation Location(GetParentDevice());

		if (Buffer.Version(Location, bDiscardSharedConstants))
		{
			// Note: Code assumes the slot index is always 0.
			const uint32 SlotIndex = 0;

			FD3D12ConstantBufferCache& CBVCache = PipelineState.Common.CBVCache;
			D3D12_GPU_VIRTUAL_ADDRESS& CurrentGPUVirtualAddress = CBVCache.CurrentGPUVirtualAddress[ShaderFrequency][SlotIndex];
			check(Location.GetGPUVirtualAddress() != CurrentGPUVirtualAddress);
			CurrentGPUVirtualAddress = Location.GetGPUVirtualAddress();
			CBVCache.Resources[ShaderFrequency][SlotIndex] = Location.GetResource();
			FD3D12ConstantBufferCache::DirtySlot(CBVCache.DirtySlotMask[ShaderFrequency], SlotIndex);

#if D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			CBVCache.CBHandles[ShaderFrequency][SlotIndex] = Buffer.GetOfflineCpuHandle();
#endif
		}
	}

	void SetBlendFactor(const float BlendFactor[4]);
	void SetStencilRef(uint32 StencilRef);

	FRHIShader* GetShader(EShaderFrequency InFrequency)
	{
		switch (InFrequency)
		{
		case SF_Vertex:        return GetGraphicsPipelineState()->GetVertexShader();
		case SF_Mesh:          return GetGraphicsPipelineState()->GetMeshShader();
		case SF_Amplification: return GetGraphicsPipelineState()->GetAmplificationShader();
		case SF_Pixel:         return GetGraphicsPipelineState()->GetPixelShader();
		case SF_Geometry:      return GetGraphicsPipelineState()->GetGeometryShader();
		case SF_Compute:       return GetComputePipelineState()->GetComputeShader();
		default:               return nullptr;
		}
	}

	void SetNewShaderData(EShaderFrequency InFrequency, const FD3D12ShaderData* InShaderData);
	void SetGraphicsPipelineState(FD3D12GraphicsPipelineState* GraphicsPipelineState);
	void SetComputePipelineState(FD3D12ComputePipelineState* ComputePipelineState);

	D3D12_STATE_CACHE_INLINE void SetStreamSource(FD3D12ResourceLocation* VertexBufferLocation, uint32 StreamIndex, uint32 Stride, uint32 Offset)
	{
		ensure(Stride == PipelineState.Graphics.StreamStrides[StreamIndex]);
		InternalSetStreamSource(VertexBufferLocation, StreamIndex, Stride, Offset);
	}

	D3D12_STATE_CACHE_INLINE void SetStreamSource(FD3D12ResourceLocation* VertexBufferLocation, uint32 StreamIndex, uint32 Offset)
	{
		InternalSetStreamSource(VertexBufferLocation, StreamIndex, PipelineState.Graphics.StreamStrides[StreamIndex], Offset);
	}

	D3D12_STATE_CACHE_INLINE void ClearVertexBuffer(const FD3D12ResourceLocation* VertexBufferLocation)
	{
		for (int32 index = 0; index <= PipelineState.Graphics.VBCache.MaxBoundVertexBufferIndex; ++index)
		{
			if (PipelineState.Graphics.VBCache.CurrentVertexBufferResources[index] == VertexBufferLocation)
			{
				PipelineState.Graphics.VBCache.CurrentVertexBufferResources[index] = nullptr;
			}
		}
	}

	D3D12_STATE_CACHE_INLINE void ClearVertexBuffers()
	{
		PipelineState.Graphics.VBCache.Clear();
		bNeedSetVB = true;
	}

public:

	D3D12_STATE_CACHE_INLINE void SetIndexBuffer(const FD3D12ResourceLocation& IndexBufferLocation, DXGI_FORMAT Format, uint32 Offset)
	{
		D3D12_GPU_VIRTUAL_ADDRESS BufferLocation = IndexBufferLocation.GetGPUVirtualAddress() + Offset;
		UINT SizeInBytes = IndexBufferLocation.GetSize() - Offset;

		D3D12_INDEX_BUFFER_VIEW& CurrentView = PipelineState.Graphics.IBCache.CurrentIndexBufferView;

		if (BufferLocation != CurrentView.BufferLocation ||
			SizeInBytes != CurrentView.SizeInBytes ||
			Format != CurrentView.Format ||
			GD3D12SkipStateCaching)
		{
			CurrentView.BufferLocation = BufferLocation;
			CurrentView.SizeInBytes = SizeInBytes;
			CurrentView.Format = Format;

			InternalSetIndexBuffer(IndexBufferLocation.GetResource());
		}
	}

	FD3D12StateCache(FD3D12CommandContext& CmdContext, FRHIGPUMask Node);
	~FD3D12StateCache() = default;

#if D3D12_RHI_RAYTRACING
	// When transitioning between RayGen and Compute, it is necessary to clear the state cache
	void TransitionComputeState(ED3D12PipelineType PipelineType)
	{
		if (LastComputePipelineType != PipelineType)
		{
			PipelineState.Common.bNeedSetPSO = true;
			PipelineState.Common.bNeedSetRootConstants = true;
			PipelineState.Compute.bNeedSetRootSignature = true;

			LastComputePipelineType = PipelineType;
		}
	}

	ED3D12PipelineType LastComputePipelineType = ED3D12PipelineType::Compute;
#endif // D3D12_RHI_RAYTRACING

	void ApplyState(ERHIPipeline HardwarePipe, ED3D12PipelineType PipelineType);
	void ApplySamplerTables(const FD3D12RootSignature* pRootSignature, uint32 StartStage, uint32 EndStage);
	void ApplyResourceTables(const FD3D12RootSignature* pRootSignature, uint32 StartStage, uint32 EndStage);
	void ApplyBindlessResources(const FD3D12RootSignature* pRootSignature, uint32 StartStage, uint32 EndStage);
	void ApplyConstants(const FD3D12RootSignature* pRootSignature, uint32 StartStage, uint32 EndStage);
	void DirtyStateForNewCommandList();
	void DirtyState();
	void DirtyViewDescriptorTables();
	void DirtySamplerDescriptorTables();
	bool AssertResourceStates(ED3D12PipelineType PipelineType);

	void SetRenderTargets(uint32 NumSimultaneousRenderTargets, FD3D12RenderTargetView** RTArray, FD3D12DepthStencilView* DSTarget);
	D3D12_STATE_CACHE_INLINE void GetRenderTargets(FD3D12RenderTargetView **RTArray, uint32* NumSimultaneousRTs, FD3D12DepthStencilView** DepthStencilTarget)
	{
		if (RTArray) //NULL is legal
		{
			check(NumSimultaneousRTs);
			FMemory::Memcpy(RTArray, PipelineState.Graphics.RenderTargetArray, sizeof(FD3D12RenderTargetView*)* D3D12_SIMULTANEOUS_RENDER_TARGET_COUNT);
			*NumSimultaneousRTs = PipelineState.Graphics.CurrentNumberOfRenderTargets;
		}

		if (DepthStencilTarget)
		{
			*DepthStencilTarget = PipelineState.Graphics.CurrentDepthStencilTarget;
		}
	}

	void SetRootConstants(const FUint32Vector4& Constants);

	void SetUAV(EShaderFrequency ShaderStage, uint32 SlotIndex, FD3D12UnorderedAccessView* UAV);

	void ClearUAVs(EShaderFrequency ShaderStage);

	void SetDepthBounds(float MinDepth, float MaxDepth)
	{
		if (PipelineState.Graphics.MinDepth != MinDepth || PipelineState.Graphics.MaxDepth != MaxDepth)
		{
			PipelineState.Graphics.MinDepth = MinDepth;
			PipelineState.Graphics.MaxDepth = MaxDepth;

			bNeedSetDepthBounds = GSupportsDepthBoundsTest;
		}
	}

	void SetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner PerPrimitiveCombiner, EVRSRateCombiner ScreenSpaceCombiner)
	{
		if (PipelineState.Graphics.DrawShadingRate != ShadingRate
			|| PipelineState.Graphics.Combiners[ED3D12VRSCombinerStages::PerPrimitive] != PerPrimitiveCombiner
			|| PipelineState.Graphics.Combiners[ED3D12VRSCombinerStages::ScreenSpace] != ScreenSpaceCombiner)
		{
			PipelineState.Graphics.DrawShadingRate = ShadingRate;
			PipelineState.Graphics.Combiners[ED3D12VRSCombinerStages::PerPrimitive] = PerPrimitiveCombiner;
			PipelineState.Graphics.Combiners[ED3D12VRSCombinerStages::ScreenSpace] = ScreenSpaceCombiner;
			bNeedSetShadingRate = GRHISupportsPipelineVariableRateShading;
		}
	}	

	void SetShadingRateImage(FD3D12Resource* ShadingRateImage)
	{
		if (PipelineState.Graphics.ShadingRateImage != ShadingRateImage)
		{
			PipelineState.Graphics.ShadingRateImage = ShadingRateImage;
			bNeedSetShadingRateImage = GRHISupportsAttachmentVariableRateShading;
		}
	}

	void SetComputeBudget(EAsyncComputeBudget ComputeBudget)
	{
		PipelineState.Compute.ComputeBudget = ComputeBudget;
	}

	void FlushComputeShaderCache(bool bForce = false);

	/**
	 * Clears all D3D12 State, setting all input/output resource slots, shaders, input layouts,
	 * predications, scissor rectangles, depth-stencil state, rasterizer state, blend state,
	 * sampler state, and viewports to NULL
	 */
	void ClearState();

	void ForceSetComputeRootSignature() { PipelineState.Compute.bNeedSetRootSignature = true; }
	void ForceSetGraphicsRootSignature() { PipelineState.Graphics.bNeedSetRootSignature = true; }

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetNewBindlessResourcesHeap(FD3D12DescriptorHeap* InHeap);
	void SetNewBindlessSamplerHeap(FD3D12DescriptorHeap* InHeap);

	void BindlessHeapsInvalidated()
	{
		// If we just switched to a new bindless heap, we have to make sure to set the RootSignatures again.
		PipelineState.Compute.bNeedSetRootSignature = true;
		PipelineState.Graphics.bNeedSetRootSignature = true;
	}

	void QueueBindlessSRV(EShaderFrequency ShaderFrequency, FD3D12ShaderResourceView* SRV)
	{
		PipelineState.Common.QueuedBindlessSRVs[ShaderFrequency].Emplace(SRV);
	}
	void QueueBindlessSRVs(EShaderFrequency ShaderFrequency, TConstArrayView<FD3D12ShaderResourceView*> SRVs)
	{
		PipelineState.Common.QueuedBindlessSRVs[ShaderFrequency].Append(SRVs);
	}
	void QueueBindlessUAV(EShaderFrequency ShaderFrequency, FD3D12UnorderedAccessView* UAV)
	{
		PipelineState.Common.QueuedBindlessUAVs[ShaderFrequency].Emplace(UAV);
	}
	void QueueBindlessUAVs(EShaderFrequency ShaderFrequency, TConstArrayView<FD3D12UnorderedAccessView*> UAVs)
	{
		PipelineState.Common.QueuedBindlessUAVs[ShaderFrequency].Append(UAVs);
	}
#endif
};
