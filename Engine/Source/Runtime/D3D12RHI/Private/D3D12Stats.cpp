// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Stats.cpp:RHI Stats and timing implementation.
=============================================================================*/

#include "D3D12Stats.h"
#include "D3D12RHIPrivate.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "RHICoreStats.h"

static TStatId GetD3D12BufferStat(const FRHIBufferDesc& BufferDesc)
{
	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::UnorderedAccess))
	{
		return GET_STATID(STAT_D3D12UAVBuffers);
	}
	
	if (EnumHasAnyFlags(BufferDesc.Usage, EBufferUsageFlags::AccelerationStructure))
	{
		return GET_STATID(STAT_D3D12RTBuffers);
	}

	return GET_STATID(STAT_D3D12Buffer);
}

void D3D12BufferStats::UpdateBufferStats(FD3D12Buffer& Buffer, bool bAllocating)
{
	const FRHIBufferDesc& BufferDesc = Buffer.GetDesc();
	const FD3D12ResourceLocation& Location = Buffer.ResourceLocation;

	const int64 BufferSize = Location.GetSize();
	const int64 RequestedSize = bAllocating ? BufferSize : -BufferSize;

	UE::RHICore::UpdateGlobalBufferStats(BufferDesc, RequestedSize);

	INC_MEMORY_STAT_BY_FName(GetD3D12BufferStat(BufferDesc).GetName(), RequestedSize);
	INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, RequestedSize);

    // With D3D12RHI_PLATFORM_LOW_LEVEL_RES_TRACK=1, MemoryTrace_Alloc is called during resource allocation
#if UE_MEMORY_TRACE_ENABLED  && (D3D12RHI_PLATFORM_LOW_LEVEL_RES_TRACK == 0)
	const D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = Location.GetGPUVirtualAddress();

	if (bAllocating)
	{
		// Skip if it's created as a
		// 1) standalone resource, because MemoryTrace_Alloc has been called in FD3D12Adapter::CreateCommittedResource
		// 2) placed resource from a pool allocator, because MemoryTrace_Alloc has been called in FD3D12Adapter::CreatePlacedResource
		if (!Location.IsStandaloneOrPooledPlacedResource())
		{
			MemoryTrace_Alloc(GPUAddress, BufferSize, Buffer.BufferAlignment, EMemoryTraceRootHeap::VideoMemory);
		}
	}
	else
	{
		MemoryTrace_Free(GPUAddress, EMemoryTraceRootHeap::VideoMemory);
	}
#endif
}
