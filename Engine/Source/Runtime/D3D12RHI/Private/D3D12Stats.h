// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

/**
* The D3D RHI stats.
*/

DECLARE_STATS_GROUP(TEXT("D3D12RHI"), STATGROUP_D3D12RHI, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Memory"), STATGROUP_D3D12Memory, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Memory Details"), STATGROUP_D3D12MemoryDetails, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Resources"), STATGROUP_D3D12Resources, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Bindless"), STATGROUP_D3D12Bindless, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Buffer Details"), STATGROUP_D3D12BufferDetails, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Pipeline State (PSO)"), STATGROUP_D3D12PipelineState, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("D3D12RHI: Descriptor Heap (GPU Visible)"), STATGROUP_D3D12DescriptorHeap, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Present time"), STAT_D3D12PresentTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CustomPresent time"), STAT_D3D12CustomPresentTime, STATGROUP_D3D12RHI, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num command allocators (3D, Compute, Copy)"), STAT_D3D12NumCommandAllocators, STATGROUP_D3D12RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num command lists (3D, Compute, Copy)"), STAT_D3D12NumCommandLists, STATGROUP_D3D12RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Query Heaps"), STAT_D3D12NumQueryHeaps, STATGROUP_D3D12RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num pipeline state objects (PSOs)"), STAT_D3D12NumPSOs, STATGROUP_D3D12RHI, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Executed Command Lists"), STAT_D3D12ExecutedCommandLists, STATGROUP_D3D12RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Executed Command List Batches"), STAT_D3D12ExecutedCommandListBatches, STATGROUP_D3D12RHI, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Textures Allocated"), STAT_D3D12TexturesAllocated, STATGROUP_D3D12RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Textures Released"), STAT_D3D12TexturesReleased, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateTexture time"), STAT_D3D12CreateTextureTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("LockTexture time"), STAT_D3D12LockTextureTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("UnlockTexture time"), STAT_D3D12UnlockTextureTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateBuffer time"), STAT_D3D12CreateBufferTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CopyToStagingBuffer time"), STAT_D3D12CopyToStagingBufferTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("LockBuffer time"), STAT_D3D12LockBufferTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("UnlockBuffer time"), STAT_D3D12UnlockBufferTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Commit transient resource time"), STAT_D3D12CommitTransientResourceTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Decommit transient resource time"), STAT_D3D12DecommitTransientResourceTime, STATGROUP_D3D12RHI, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("UAV Barriers"), STAT_D3D12UAVBarriers, STATGROUP_D3D12RHI, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Resource Heaps Allocated"), STAT_D3D12BindlessResourceHeapsAllocated, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Resource Heaps Active"), STAT_D3D12BindlessResourceHeapsActive, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Resource Heaps in use by GPU"), STAT_D3D12BindlessResourceHeapsInUseByGPU, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Heaps Versioned"), STAT_D3D12BindlessResourceHeapsVersioned, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Descriptors Initialized"), STAT_D3D12BindlessResourceDescriptorsInitialized, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Descriptors Updated"), STAT_D3D12BindlessResourceDescriptorsUpdated, STATGROUP_D3D12Bindless, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource GPU Descriptors Copied"), STAT_D3D12BindlessResourceGPUDescriptorsCopied, STATGROUP_D3D12Bindless, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Resource Heaps GPU Memory Usage"), STAT_D3D12BindlessResourceHeapGPUMemoryUsage, STATGROUP_D3D12Bindless, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateBoundShaderState time"), STAT_D3D12CreateBoundShaderStateTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("New bound shader state time"), STAT_D3D12NewBoundShaderStateTime, STATGROUP_D3D12RHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num bound shader states"), STAT_D3D12NumBoundShaderState, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Set bound shader state"), STAT_D3D12SetBoundShaderState, STATGROUP_D3D12RHI, );

DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Update uniform buffer"), STAT_D3D12UpdateUniformBufferTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Commit resource tables"), STAT_D3D12CommitResourceTables, STATGROUP_D3D12RHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num textures in tables"), STAT_D3D12SetTextureInTableCalls, STATGROUP_D3D12RHI, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch shader bundle time"), STAT_D3D12DispatchShaderBundle, STATGROUP_D3D12RHI, );

DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Clear SRVs time"), STAT_D3D12ClearShaderResourceViewsTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Set SRV time"), STAT_D3D12SetShaderResourceViewTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Set UAV time"), STAT_D3D12SetUnorderedAccessViewTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Commit graphics constants (Set CBV time)"), STAT_D3D12CommitGraphicsConstants, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Commit compute constants (Set CBV time)"), STAT_D3D12CommitComputeConstants, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Set shader uniform buffer (Set CBV time)"), STAT_D3D12SetShaderUniformBuffer, STATGROUP_D3D12RHI, EStatFlags::Verbose, );

DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState time"), STAT_D3D12ApplyStateTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Rebuild PSO time"), STAT_D3D12ApplyStateRebuildPSOTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Find PSO time"), STAT_D3D12ApplyStateFindPSOTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Set SRV time"), STAT_D3D12ApplyStateSetSRVTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Set UAV time"), STAT_D3D12ApplyStateSetUAVTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Set Vertex Buffer time"), STAT_D3D12ApplyStateSetVertexBufferTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("ApplyState: Set CBV time"), STAT_D3D12ApplyStateSetConstantBufferTime, STATGROUP_D3D12RHI, EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PSO Create time"), STAT_D3D12PSOCreateTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Clear MRT time"), STAT_D3D12ClearMRT, STATGROUP_D3D12RHI, EStatFlags::Verbose, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("ExecuteCommandList time"), STAT_D3D12ExecuteCommandListTime, STATGROUP_D3D12RHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("WaitForFence time"), STAT_D3D12WaitForFenceTime, STATGROUP_D3D12RHI, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Global Constant buffer update time"), STAT_D3D12GlobalConstantBufferUpdateTime, STATGROUP_D3D12RHI, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("TOTAL"), STAT_D3D12MemoryCurrentTotal, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Render Targets"), STAT_D3D12RenderTargets, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UAV Textures"), STAT_D3D12UAVTextures, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Textures"), STAT_D3D12Textures, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UAV Buffers"), STAT_D3D12UAVBuffers, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("RayTracing Buffers"), STAT_D3D12RTBuffers, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Buffers"), STAT_D3D12Buffer, STATGROUP_D3D12Resources, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Transient Heaps"), STAT_D3D12TransientHeaps, STATGROUP_D3D12Resources, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("RenderTarget StandAlone Allocated"), STAT_D3D12RenderTargetStandAloneAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UAVTexture StandAlone Allocated"), STAT_D3D12UAVTextureStandAloneAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture StandAlone Allocated"), STAT_D3D12TextureStandAloneAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UAVBuffer StandAlone Allocated"), STAT_D3D12UAVBufferStandAloneAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Buffer StandAlone Allocated"), STAT_D3D12BufferStandAloneAllocated, STATGROUP_D3D12MemoryDetails, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RenderTarget StandAlone Allocated"), STAT_D3D12RenderTargetStandAloneCount, STATGROUP_D3D12MemoryDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("UAVTexture StandAlone Allocated"), STAT_D3D12UAVTextureStandAloneCount, STATGROUP_D3D12MemoryDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Texture StandAlone Allocated"), STAT_D3D12TextureStandAloneCount, STATGROUP_D3D12MemoryDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("UAVBuffer StandAlone Allocated"), STAT_D3D12UAVBufferStandAloneCount, STATGROUP_D3D12MemoryDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Buffer StandAlone Allocated"), STAT_D3D12BufferStandAloneCount, STATGROUP_D3D12MemoryDetails, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Allocator Allocated"), STAT_D3D12TextureAllocatorAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Allocator Unused"), STAT_D3D12TextureAllocatorUnused, STATGROUP_D3D12MemoryDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Texture Allocator Allocated"), STAT_D3D12TextureAllocatorCount, STATGROUP_D3D12MemoryDetails, );

DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("View DescriptorHeap Memory (SRV, CBV, UAV)"), STAT_ViewOnlineDescriptorHeapMemory, STATGROUP_D3D12MemoryDetails, FPlatformMemory::MCR_GPUSystem, );
DECLARE_MEMORY_STAT_POOL_EXTERN(TEXT("Sampler DescriptorHeap Memory"), STAT_SamplerOnlineDescriptorHeapMemory, STATGROUP_D3D12MemoryDetails, FPlatformMemory::MCR_GPUSystem, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferPool Memory Allocated"), STAT_D3D12BufferPoolMemoryAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferPool Memory Used"), STAT_D3D12BufferPoolMemoryUsed, STATGROUP_D3D12MemoryDetails, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("UploadPool Memory Allocated"), STAT_D3D12UploadPoolMemoryAllocated, STATGROUP_D3D12MemoryDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UploadPool Memory Used"), STAT_D3D12UploadPoolMemoryUsed, STATGROUP_D3D12MemoryDetails, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferPool Memory Free"), STAT_D3D12BufferPoolMemoryFree, STATGROUP_D3D12BufferDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("BufferPool Memory Alignment Waste"), STAT_D3D12BufferPoolAlignmentWaste, STATGROUP_D3D12BufferDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("BufferPool Page Count"), STAT_D3D12BufferPoolPageCount, STATGROUP_D3D12BufferDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("BufferPool Full Pages"), STAT_D3D12BufferPoolFullPages, STATGROUP_D3D12BufferDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Bufferool Fragmentation"), STAT_D3D12BufferPoolFragmentation, STATGROUP_D3D12BufferDetails, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Bufferool Fragmentation Percentage"), STAT_D3D12BufferPoolFragmentationPercentage, STATGROUP_D3D12BufferDetails, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("UploadPool Memory Free"), STAT_D3D12UploadPoolMemoryFree, STATGROUP_D3D12BufferDetails, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("UploadPool Memory Alignment Waste"), STAT_D3D12UploadPoolAlignmentWaste, STATGROUP_D3D12BufferDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("UploadPool Page Count"), STAT_D3D12UploadPoolPageCount, STATGROUP_D3D12BufferDetails, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("UploadPool Full Pages"), STAT_D3D12UploadPoolFullPages, STATGROUP_D3D12BufferDetails, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Reserved Resource Physical Memory"), STAT_D3D12ReservedResourcePhysical, STATGROUP_D3D12MemoryDetails, );

/**
* Detailed Descriptor heap stats
*/
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Unique Samplers"), STAT_UniqueSamplers, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("View: Heap changed"), STAT_ViewHeapChanged, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sampler: Heap changed"), STAT_SamplerHeapChanged, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("View: Num descriptor heaps"), STAT_NumViewOnlineDescriptorHeaps, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Sampler: Num descriptor heaps"), STAT_NumSamplerOnlineDescriptorHeaps, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Sampler: Num reusable unique descriptor table entries"), STAT_NumReuseableSamplerOnlineDescriptorTables, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Sampler: Num reusable unique descriptors"), STAT_NumReuseableSamplerOnlineDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("View: Num reserved descriptors"), STAT_NumReservedViewOnlineDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sampler: Num reserved descriptors"), STAT_NumReservedSamplerOnlineDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sampler: Num reused descriptors"), STAT_NumReusedSamplerOnlineDescriptors, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("View Global: Free Descriptors"), STAT_GlobalViewHeapFreeDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("View Global: Reserved Descriptors"), STAT_GlobalViewHeapReservedDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("View Global: Used Descriptors"), STAT_GlobalViewHeapUsedDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("View Global: Wasted Descriptors"), STAT_GlobalViewHeapWastedDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("View Global: Block Allocations"), STAT_GlobalViewHeapBlockAllocations, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Explicit sampler descriptor heaps"), STAT_ExplicitSamplerDescriptorHeaps, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Explicit sampler descriptors"), STAT_ExplicitSamplerDescriptors, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Explicit view descriptor heaps"), STAT_ExplicitViewDescriptorHeaps, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Explicit view descriptors"), STAT_ExplicitViewDescriptors, STATGROUP_D3D12DescriptorHeap, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Max used explicit sampler descriptors in a single heap"), STAT_ExplicitMaxUsedSamplerDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Used explicit sampler descriptors (per frame)"), STAT_ExplicitUsedSamplerDescriptors, STATGROUP_D3D12DescriptorHeap, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Used explicit view descriptors (per frame)"), STAT_ExplicitUsedViewDescriptors, STATGROUP_D3D12DescriptorHeap, );


struct FD3D12GlobalStats
{
	// in bytes, never change after RHI, needed to scale game features
	static int64 GDedicatedVideoMemory;

	// in bytes, never change after RHI, needed to scale game features
	static int64 GDedicatedSystemMemory;

	// in bytes, never change after RHI, needed to scale game features
	static int64 GSharedSystemMemory;

	// In bytes. Never changed after RHI init. Our estimate of the amount of memory that we can use for graphics resources in total.
	static int64 GTotalGraphicsMemory;
};
