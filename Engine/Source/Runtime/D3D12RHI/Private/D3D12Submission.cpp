// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12Submission.h"
#include "D3D12RHIPrivate.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IRenderCaptureProvider.h"
#include "Stats/ThreadIdleStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "D3D12Tracing.h"

#ifndef D3D12_PLATFORM_SUPPORTS_BLOCKING_FENCES
#define D3D12_PLATFORM_SUPPORTS_BLOCKING_FENCES 1
#endif

// These defines control which threads are enabled in the GPU submission pipeline.
#define D3D12_USE_SUBMISSION_THREAD (1)
#define D3D12_USE_INTERRUPT_THREAD  (1 && D3D12_PLATFORM_SUPPORTS_BLOCKING_FENCES)

static TAutoConsoleVariable<int32> CVarRHIUseSubmissionThread(
	TEXT("rhi.UseSubmissionThread"),
	2,
	TEXT("Whether to enable the RHI submission thread.\n")
	TEXT("  0: No\n")
	TEXT("  1: Yes, but not when running with multi-gpu.\n")
	TEXT("  2: Yes, always\n"),
	ECVF_ReadOnly);

DECLARE_CYCLE_STAT(TEXT("Submit"), STAT_D3D12Submit, STATGROUP_D3D12RHI);

DECLARE_STATS_GROUP(TEXT("D3D12RHIPipeline"), STATGROUP_D3D12RHIPipeline, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU IA Vertices"   ), STAT_D3D12RHI_IAVertices   , STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU IA Primitives" ), STAT_D3D12RHI_IAPrimitives , STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU VS Invocations"), STAT_D3D12RHI_VSInvocations, STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU GS Invocations"), STAT_D3D12RHI_GSInvocations, STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU GS Primitives" ), STAT_D3D12RHI_GSPrimitives , STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU C Invocations" ), STAT_D3D12RHI_CInvocations , STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU C Primitives"  ), STAT_D3D12RHI_CPrimitives  , STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU PS Invocations"), STAT_D3D12RHI_PSInvocations, STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU HS Invocations"), STAT_D3D12RHI_HSInvocations, STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU DS Invocations"), STAT_D3D12RHI_DSInvocations, STATGROUP_D3D12RHIPipeline);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("GPU CS Invocations"), STAT_D3D12RHI_CSInvocations, STATGROUP_D3D12RHIPipeline);

static float GD3D12SubmissionTimeout = 5.0;
static FAutoConsoleVariableRef CVarD3D12SubmissionTimeout(
	TEXT("r.D3D12.SubmissionTimeout"),
	GD3D12SubmissionTimeout,
	TEXT("The maximum time, in seconds, that a submitted GPU command list is allowed to take before the RHI reports a GPU hang"),
	ECVF_RenderThreadSafe);

static int32 GD3D12SubmissionMaxExecuteBatchSizeDirect = std::numeric_limits<int32>::max();
static FAutoConsoleVariableRef CVarD3D12SubmissionMaxExecuteBatchSizeDirect(
	TEXT("r.D3D12.Submission.MaxExecuteBatchSize.Direct"),
	GD3D12SubmissionMaxExecuteBatchSizeDirect,
	TEXT("The maximum number of command lists to pass to a single ExecuteCommandLists invocation for direct queues\n")
	TEXT("The valid range is 1 to INT_MAX inclusive. Values less than 1 will be clamped to 1"),
	ECVF_RenderThreadSafe);

static int32 GD3D12SubmissionMaxExecuteBatchSizeCopy = std::numeric_limits<int32>::max();
static FAutoConsoleVariableRef CVarD3D12SubmissionMaxExecuteBatchSizeCopy(
	TEXT("r.D3D12.Submission.MaxExecuteBatchSize.Copy"),
	GD3D12SubmissionMaxExecuteBatchSizeCopy,
	TEXT("The maximum number of command lists to pass to a single ExecuteCommandLists invocation for copy queues\n")
	TEXT("The valid range is 1 to INT_MAX inclusive. Values less than 1 will be clamped to 1"),
	ECVF_RenderThreadSafe);

static int32 GD3D12SubmissionMaxExecuteBatchSizeAsync = std::numeric_limits<int32>::max();
static FAutoConsoleVariableRef CVarD3D12SubmissionMaxExecuteBatchSizeAsync(
	TEXT("r.D3D12.Submission.MaxExecuteBatchSize.Async"),
	GD3D12SubmissionMaxExecuteBatchSizeAsync,
	TEXT("The maximum number of command lists to pass to a single ExecuteCommandLists invocation for async queues\n")
	TEXT("The valid range is 1 to INT_MAX inclusive. Values less than 1 will be clamped to 1"),
	ECVF_RenderThreadSafe);

static HRESULT GGPUD3DResult{};
static std::atomic<int> GGPUCrashDetected = false;

class FD3D12Thread final : private FRunnable
{
public:
	typedef FD3D12DynamicRHI::FProcessResult(FD3D12DynamicRHI::*FQueueFunc)();

	FD3D12Thread(TCHAR const* Name, EThreadPriority Priority, FD3D12DynamicRHI* RHI, FQueueFunc QueueFunc)
		: RHI(RHI)
		, QueueFunc(QueueFunc)
		, Event(CreateEvent(nullptr, false, false, nullptr))
		, Thread(FRunnableThread::Create(this, Name, 0, Priority))
	{}

	virtual ~FD3D12Thread()
	{
		bExit = true;
		SetEvent(Event);

		Thread->WaitForCompletion();
		delete Thread;

		CloseHandle(Event);
	}

	void Kick() const
	{
		SetEvent(Event);
	}

	void Join() const
	{
		Thread->WaitForCompletion();
	}

	uint32 GetThreadID() const
	{
		return Thread->GetThreadID();
	}

private:
	virtual uint32 Run() override
	{
		while (!bExit)
		{
			// Process the queue until no more progress is made
			FD3D12DynamicRHI::FProcessResult Result;
			do { Result = (RHI->*QueueFunc)(); }
			while (EnumHasAllFlags(Result.Status, FD3D12DynamicRHI::EQueueStatus::Processed));

			WaitForSingleObject(Event, Result.WaitTimeout);
		}

		// Drain any remaining work in the queue
		while (EnumHasAllFlags((RHI->*QueueFunc)().Status, FD3D12DynamicRHI::EQueueStatus::Pending)) {}

		return 0;
	}

	FD3D12DynamicRHI* RHI;
	FQueueFunc QueueFunc;
	bool bExit = false;

public:
	// Can't use FEvent here since we need to be able to get the underlying HANDLE
	// for the ID3D12Fences to signal via ID3D12Fence::SetEventOnCompletion().
	HANDLE const Event;

private:
	FRunnableThread* Thread = nullptr;
};

void FD3D12DynamicRHI::InitializeSubmissionPipe()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
#if D3D12_USE_INTERRUPT_THREAD
		InterruptThread = new FD3D12Thread(TEXT("RHIInterruptThread"), TPri_Highest, this, &FD3D12DynamicRHI::ProcessInterruptQueue);
#endif

#if D3D12_USE_SUBMISSION_THREAD
		bool bUseSubmissionThread = false;
		switch (CVarRHIUseSubmissionThread.GetValueOnAnyThread())
		{
		case 1: bUseSubmissionThread = FRHIGPUMask::All().HasSingleIndex(); break;
		case 2: bUseSubmissionThread = true; break;
		}

		// Currently RenderDoc can't make programmatic captures when we use a submission thread.
		bUseSubmissionThread &= !IRenderCaptureProvider::IsAvailable() || IRenderCaptureProvider::Get().CanSupportSubmissionThread();

		if (bUseSubmissionThread)
		{
			SubmissionThread = new FD3D12Thread(TEXT("RHISubmissionThread"), TPri_Highest, this, &FD3D12DynamicRHI::ProcessSubmissionQueue);
		}
#endif
	}

	// Initialize the timing structs in each queue, and the engine GPU profilers
	{
		TArray<FD3D12Payload*> Payloads;
		TArray<UE::RHI::GPUProfiler::FQueue> ProfilerQueues;

		ForEachQueue([&](FD3D12Queue& Queue)
		{
			FD3D12Payload* Payload = Payloads.Emplace_GetRef(new FD3D12Payload(Queue));
			Payload->Timing = CurrentTimingPerQueue.CreateNew(Queue);

			ProfilerQueues.Add(Queue.GetProfilerQueue());
		});

		UE::RHI::GPUProfiler::InitializeQueues(ProfilerQueues);
		SubmitPayloads(MoveTemp(Payloads));
	}
}

void FD3D12DynamicRHI::ShutdownSubmissionPipe()
{
	delete SubmissionThread;
	SubmissionThread = nullptr;

	delete InterruptThread;
	InterruptThread = nullptr;

	if (EopTask)
	{
		ProcessInterruptQueueUntil(EopTask);
		EopTask = nullptr;
	}
}

// A finalized set of command payloads. This type is used to implement the RHI command list submission API.
struct FD3D12FinalizedCommands : public IRHIPlatformCommandList, public TArray<FD3D12Payload*>
{};

FDynamicRHI::FRHIFinalizeContextsResult FD3D12DynamicRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	checkf(Args.UploadContext == nullptr, TEXT("D3D12 RHI does not implement upload contexts."));

	auto FinalizeContext = [](FD3D12CommandContext* CmdContext, FD3D12FinalizedCommands& Result)
	{
		CmdContext->Finalize(Result);

		if (!CmdContext->IsDefaultContext())
		{
			CmdContext->ClearState();
			CmdContext->GetParentDevice()->ReleaseContext(CmdContext);
		}
	};

	FDynamicRHI::FRHIFinalizeContextsResult Result;
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		if (FD3D12CommandContextBase* ContextBase = static_cast<FD3D12CommandContextBase*>(Args.Contexts[Pipeline]))
		{
			FD3D12FinalizedCommands FinalizedCommands;
			if (FD3D12CommandContextRedirector* Redirector = ContextBase->AsRedirector())
			{
				for (uint32 GPUIndex : Redirector->GetPhysicalGPUMask())
				{
					FinalizeContext(Redirector->GetSingleDeviceContext(GPUIndex), FinalizedCommands);
				}

				if (!Redirector->bIsDefaultContext)
				{
					delete Redirector;
				}
			}
			else
			{
				FinalizeContext(static_cast<FD3D12CommandContext*>(ContextBase), FinalizedCommands);
			}

			if (FinalizedCommands.Num())
			{
				Result.CommandLists[Pipeline] = new FD3D12FinalizedCommands(MoveTemp(FinalizedCommands));
			}
		}
	}

	return Result;
}

void FD3D12DynamicRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	SubmitCommands(MakeArrayView(reinterpret_cast<FD3D12FinalizedCommands**>(Args.CommandLists.GetData()), Args.CommandLists.Num()));
}

void FD3D12DynamicRHI::SubmitCommands(TConstArrayView<FD3D12FinalizedCommands*> Commands)
{
	SCOPED_NAMED_EVENT_TEXT("CommandList_Submit", FColor::Magenta);

	TArray<FD3D12Payload*> AllPayloads;
	for (FD3D12FinalizedCommands* Payloads : Commands)
	{
	#if WITH_RHI_BREADCRUMBS
		TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators {};
		if (Payloads->BreadcrumbAllocators.Num())
		{
			BreadcrumbAllocators = MakeShared<FRHIBreadcrumbAllocatorArray>(MoveTemp(Payloads->BreadcrumbAllocators));
		}

		for (FD3D12Payload* Payload : *Payloads)
		{
			Payload->BreadcrumbRange = Payloads->BreadcrumbRange;
			if (BreadcrumbAllocators.IsValid())
			{
				check(!Payload->BreadcrumbAllocators.IsValid());
				Payload->BreadcrumbAllocators = BreadcrumbAllocators;
			}
		}
	#endif

		AllPayloads.Append(MoveTemp(*Payloads));
		delete Payloads;
	}

	SubmitPayloads(MoveTemp(AllPayloads));
}

void FD3D12DynamicRHI::SubmitPayloads(TArray<FD3D12Payload*>&& Payloads)
{
	if (Payloads.Num())
	{
		PendingPayloadsForSubmission.Enqueue(new TArray<FD3D12Payload*>(MoveTemp(Payloads)));
	}

	if (SubmissionThread)
	{
		SubmissionThread->Kick();
	}
	else
	{
		// Since we're processing directly on the calling thread, we need to take a scope lock.
		// Multiple engine threads might be calling Submit().
		{
			FScopeLock Lock(&SubmissionCS);

			// Process the submission queue until no further progress is being made.
			while (EnumHasAnyFlags(ProcessSubmissionQueue().Status, EQueueStatus::Processed)) {}
		}
	}

	// Use this opportunity to pump the interrupt queue
	ProcessInterruptQueueUntil(nullptr);
}

static int32 GetMaxExecuteBatchSize(ED3D12QueueType QueueType)
{
	switch (QueueType)
	{
		case ED3D12QueueType::Direct:
			return std::max(1, GD3D12SubmissionMaxExecuteBatchSizeDirect);
		case ED3D12QueueType::Copy:
			return std::max(1, GD3D12SubmissionMaxExecuteBatchSizeCopy);
		case ED3D12QueueType::Async:
			return std::max(1, GD3D12SubmissionMaxExecuteBatchSizeAsync);
		default:
			// Need to add new queue type and CVar
			checkNoEntry();
			return std::numeric_limits<int32>::max();
	}
}

FD3D12DynamicRHI::FProcessResult FD3D12DynamicRHI::ProcessSubmissionQueue()
{
	SCOPED_NAMED_EVENT_TEXT("SubmissionQueue_Process", FColor::Turquoise);
	SCOPE_CYCLE_COUNTER(STAT_D3D12Submit);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ProcessSubmissionQueue"));
	
	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::ProcessSubmissionQueueEnter());

	FD3D12Queue::FPayloadArray PayloadsToHandDown;

	FProcessResult Result;

	auto FlushPayloads = [&PayloadsToHandDown, &Result, DynamicRHI = this](int32 MinPayloadsToFlush = 1)
	{
		if (PayloadsToHandDown.Num() >= MinPayloadsToFlush)
		{
			Result.Status |= EQueueStatus::Processed;
			DynamicRHI->FlushBatchedPayloads(PayloadsToHandDown);
		}
	};

	bool bProgress;
	bool bKickInterruptThread = false;

	do
	{
		bProgress = false;
		EnumRemoveFlags(Result.Status, EQueueStatus::Pending);

		// Push all pending payloads into the ordered per-device, per-pipe pending queues
		{
			TArray<FD3D12Payload*>* Array;
			while (PendingPayloadsForSubmission.Dequeue(Array))
			{
				for (FD3D12Payload* Payload : *Array)
				{
					Payload->Queue.PendingSubmission.Enqueue(Payload);
				}
				delete Array;
			}
		}

		//
		// Fence values for FD3D12SyncPoint are determined on the submission thread,
		// where each queue has a monotonically incrementing fence value.
		//
		// We might receive work that waits on a sync point which has not yet been submitted
		// to the queue that will signal it, so we need to delay processing of those
		// payloads until the fence value is known.
		//

		// Process all queues (across all devices and adapters) to flush work.
		// Any sync point waits where the fence value is unknown will be left in the 
		// appropriate queue, to be processed the next time commands are submitted.
		ForEachQueue([&](FD3D12Queue& CurrentQueue)
		{
			while (true)
			{
				{
					FD3D12Payload** PayloadPtr = CurrentQueue.PendingSubmission.Peek();
					if (!PayloadPtr)
						return;

					FD3D12Payload* Payload = *PayloadPtr;

					// Accumulate the list of fences to await, and their maximum values
					while (Payload->SyncPointsToWait.Index < Payload->SyncPointsToWait.Num())
					{
						FD3D12SyncPointRef& SyncPoint = Payload->SyncPointsToWait[Payload->SyncPointsToWait.Index];
						if (!SyncPoint->ResolvedFence.IsSet())
						{
							// Need to wait on a sync point, but the fence value has not been resolved yet
							// (no other payloads have signaled the sync point yet).
						
							// Skip processing this queue, and move on to the next. We will retry later when
							// further work is submitted, which may contain the sync point we need.
							Result.Status |= EQueueStatus::Pending;
							IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitYieldOnUnresolvedSyncPoint(CurrentQueue, Payload, SyncPoint));
							return;
						}

						Payload->AddQueueFenceWait(
							SyncPoint->ResolvedFence->Fence,
							SyncPoint->ResolvedFence->Value
						);

						Payload->SyncPointsToWait.Index++;
						bProgress = true;
					}

					while (Payload->ManualFencesToWait.Index < Payload->ManualFencesToWait.Num())
					{
						FD3D12PayloadBase::FManualFence& ManualFence = Payload->ManualFencesToWait[Payload->ManualFencesToWait.Index];
						if (ManualFence.bHasSignalled != nullptr && !(*ManualFence.bHasSignalled))
						{
							// The CPU-side submit action for a manual fence has not happened yet.
							// Don't process this payload until that has, otherwise we can deadlock due to inversion of submission order.
							// Skip processing this queue, and move on to the next. We will retry later when
							// further work is submitted, which may contain the fence we need.
							Result.Status |= EQueueStatus::Pending;
							IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitYieldOnUnresolvedManualFence(CurrentQueue, Payload, ManualFence.Fence, ManualFence.Value));
							return;
						}

						Payload->ManualFencesToWait.Index++;
						bProgress = true;
					}
				
					// All necessary sync points have been resolved.
					Payload->SyncPointsToWait = {};
					CurrentQueue.PendingSubmission.Dequeue();
					bProgress = true;

					check(!CurrentQueue.PayloadToSubmit);
					CurrentQueue.PayloadToSubmit = Payload;
					Result.Status |= EQueueStatus::Processed;
					bKickInterruptThread = true;

					//
					// Now we generate any required barrier command lists. These may require
					// executing on a different queue (e.g. graphics-only transitions required 
					// before async compute work), so we gather potential work across all
					// queues for this device.
					//
					auto AccumulateQueries = [&](FD3D12CommandList* CommandList)
					{
						FD3D12Queue& TargetQueue = CommandList->Device->GetQueue(CommandList->QueueType);
						const uint32 MaxBatchSize = GetMaxExecuteBatchSize(TargetQueue.QueueType);

						TargetQueue.BatchedObjects.OcclusionQueries    .Append(MoveTemp(CommandList->State.OcclusionQueries    ));
						TargetQueue.BatchedObjects.PipelineStatsQueries.Append(MoveTemp(CommandList->State.PipelineStatsQueries));
						TargetQueue.BatchedObjects.TimestampQueries    .Append(MoveTemp(CommandList->State.TimestampQueries    ));
					};

					for (int32 Index = 0; Index < Payload->CommandListsToExecute.Num(); Index++)
					{
						FD3D12CommandList* CurrentCommandList = Payload->CommandListsToExecute[Index];
						AccumulateQueries(CurrentCommandList);
					}
				}

				FlushPayloads(FD3D12Queue::MaxBatchedPayloads);

				// Now submit the original payload
				CurrentQueue.FinalizePayload(false, PayloadsToHandDown);
				FlushPayloads(FD3D12Queue::MaxBatchedPayloads);
			}
		});
	} while (bProgress);

	FlushPayloads();

	if (InterruptThread && bKickInterruptThread)
	{
		InterruptThread->Kick();
	}

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::ProcessSubmissionQueueExit(static_cast<uint8>(Result.Status)));
	return Result;
}

uint64 FD3D12Queue::FinalizePayload(bool bRequiresSignal, FPayloadArray& PayloadsToHandDown)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteCommandList);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ExecuteCommandLists"));

	check(PayloadToSubmit && this == &PayloadToSubmit->Queue);
	check(PayloadToSubmit->SyncPointsToWait.Num() == 0);

	BarrierTimestamps.CloseAndReset(PayloadToSubmit->BatchedObjects.QueryRanges);

	// Gather query ranges from this payload, grouping by heap pointer
	if (BatchedObjects.QueryRanges.Num())
	{
		for (auto& [Heap, Ranges] : PayloadToSubmit->BatchedObjects.QueryRanges)
		{
			BatchedObjects.QueryRanges.FindOrAdd(Heap).Append(MoveTemp(Ranges));
		}
		PayloadToSubmit->BatchedObjects.QueryRanges.Reset();
	}
	else
	{
		BatchedObjects.QueryRanges = MoveTemp(PayloadToSubmit->BatchedObjects.QueryRanges);
	}

	check(PayloadToSubmit->BatchedObjects.IsEmpty());

	if (!BatchedObjects.IsEmpty())
	{
		// Always resolve queries if we're switching the Timing struct,
		// since we need to gather the timestamps for that frame.
		bool bResolveQueries = PayloadToSubmit->Timing.IsSet();

		if (!bResolveQueries)
		{
			// If this payload will signal a CPU-visible sync point, we need to resolve queries.
			// This makes sure that the query data has reached the CPU before the sync point the CPU is waiting on is signaled.
			for (FD3D12SyncPoint* SyncPoint : PayloadToSubmit->SyncPointsToSignal)
			{
				if (SyncPoint->GetType() == ED3D12SyncPointType::GPUAndCPU)
				{
					bResolveQueries = true;
					break;
				}
			}
		}

		if (bResolveQueries)
		{
			{
				FD3D12CommandList* ResolveCommandList = nullptr;

				// We've got queries to resolve. Allocate a command list.
				auto GetResolveCommandList = [&]() -> FD3D12CommandList*
				{
					if (ResolveCommandList)
						return ResolveCommandList;

					if (!BarrierAllocator)
						BarrierAllocator = Device->ObtainCommandAllocator(QueueType);

					return ResolveCommandList = Device->ObtainCommandList(BarrierAllocator, nullptr, nullptr);
				};

				// Ranges are grouped by heap pointer.
				for (auto& [Heap, Ranges] : BatchedObjects.QueryRanges)
				{
					{
#if ENABLE_RESIDENCY_MANAGEMENT
						TArray<FD3D12ResidencyHandle*, TInlineAllocator<2>> ResidencyHandles;
						ResidencyHandles.Add(&Heap->GetHeapResidencyHandle());
						ResidencyHandles.Append(Heap->GetResultBuffer()->GetResidencyHandles());
						GetResolveCommandList()->AddToResidencySet(ResidencyHandles);
#endif // ENABLE_RESIDENCY_MANAGEMENT
					}

					if (Heap->GetD3DQueryHeap())
					{
						// Sort the ranges into ascending order so we can merge adjacent ones,
						// to reduce the number of ResolveQueryData calls we need to make.
						Ranges.Sort();

						for (int32 Index = 0; Index < Ranges.Num(); )
						{
							FD3D12QueryRange Range = Ranges[Index++];

							while (Index < Ranges.Num() && Ranges[Index].Start == Range.End)
							{
								// Ranges are contiguous. Extend.
								Range.End = Ranges[Index++].End;
							}

							GetResolveCommandList()->GraphicsCommandList()->ResolveQueryData(
								Heap->GetD3DQueryHeap(),
								Heap->QueryType,
								Range.Start,
								Range.End - Range.Start,
								Heap->GetResultBuffer()->GetResource(),
								Range.Start * Heap->GetResultSize()
							);
						}
					}
				}

				if (ResolveCommandList)
				{
					ResolveCommandList->Close();
					PayloadToSubmit->CommandListsToExecute.Add(ResolveCommandList);
				}
			}

			// Move all the batched objects in this queue into the payload, so they get passed down the pipe.
			PayloadToSubmit->BatchedObjects = MoveTemp(BatchedObjects);
		}
	}

	if (BarrierAllocator)
	{
		PayloadToSubmit->AllocatorsToRelease.Add(BarrierAllocator);
		BarrierAllocator = nullptr;
	}

	// Keep the latest fence value in the submitted payload.
	// The interrupt thread uses this to determine when work has completed.
	uint64 NextCompletionValue = Fence.NextCompletionValue;

	// Set the fence/value pair into any sync points we need to signal.
	for (FD3D12SyncPointRef& SyncPoint : PayloadToSubmit->SyncPointsToSignal)
	{
		check(!SyncPoint->ResolvedFence.IsSet());
		IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitResolveSyncPoint(*this, PayloadToSubmit, SyncPoint, Fence.NextCompletionValue));
		SyncPoint->ResolvedFence.Emplace(Fence, NextCompletionValue);
	}

	PayloadToSubmit->CompletionFenceValue = NextCompletionValue;
	PayloadToSubmit->bAlwaysSignal |= bRequiresSignal;

	if (PayloadToSubmit->RequiresQueueFenceSignal())
	{
		++Fence.NextCompletionValue;
	}

	PayloadsToHandDown.Add(PayloadToSubmit);
	PayloadToSubmit = nullptr;

	return NextCompletionValue;
}

void FD3D12DynamicRHI::UpdateReservedResources(FD3D12Payload* Payload)
{
	FD3D12Queue& Queue = Payload->Queue;

	// On some devices, some queues cannot perform tile remapping operations.
	// We can work around this limitation by running the remapping in lockstep on another queue:
	// - tile mapping queue waits for commands on this queue to finish
	// - tile mapping queue performs the commit/decommit operations
	// - this queue waits for tile mapping queue to finish
	// The extra sync is not required when the current queue is capable of the remapping operations.

	ID3D12CommandQueue* TileMappingQueue = (Queue.bSupportsTileMapping ? Queue.D3DCommandQueue : Queue.Device->TileMappingQueue).GetReference();
	FD3D12Fence& TileMappingFence = Queue.Device->TileMappingFence;

	const bool bCrossQueueSyncRequired = TileMappingQueue != Queue.D3DCommandQueue.GetReference();

	if (bCrossQueueSyncRequired)
	{
		// tile mapping queue waits for commands on this queue to finish
		Queue.D3DCommandQueue->Signal(TileMappingFence.D3DFence, ++TileMappingFence.LastSignaledValue);
		TileMappingQueue->Wait(TileMappingFence.D3DFence, TileMappingFence.LastSignaledValue);
	}

	for (const FD3D12CommitReservedResourceDesc& CommitDesc : Payload->ReservedResourcesToCommit)
	{
		checkf(CommitDesc.Resource, TEXT("FD3D12CommitReservedResourceDesc::Resource must be set"));
		CommitDesc.Resource->CommitReservedResource(TileMappingQueue, CommitDesc.CommitSizeInBytes);
	}

	if (bCrossQueueSyncRequired)
	{
		// this queue waits for tile mapping operations to finish
		TileMappingQueue->Signal(TileMappingFence.D3DFence, ++TileMappingFence.LastSignaledValue);
		Queue.D3DCommandQueue->Wait(TileMappingFence.D3DFence, TileMappingFence.LastSignaledValue);
	}
}

void FD3D12DynamicRHI::FlushBatchedPayloads(FD3D12Queue::FPayloadArray& PayloadsToSubmit)
{
	uint32 FirstPayload = 0, LastPayload = 0;

	auto Wait = [this](FD3D12Payload* Payload)
	{
		FD3D12Queue& Queue = Payload->Queue;

		// Wait for queue fences
		for (auto& [LocalFence, Value] : Payload->QueueFencesToWait)
		{
			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FWaitFence>(
				  FPlatformTime::Cycles64()
				, Value
				, LocalFence.OwnerQueue->GetProfilerQueue()
			);

			IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitWaitQueueFence(Payload, Queue, *LocalFence.OwnerQueue, Value));
			VERIFYD3D12RESULT(Queue.D3DCommandQueue->Wait(LocalFence.D3DFence, Value));
		}

		// Wait for manual fences
		for (auto& [LocalFence, Value, bFenceSignalled] : Payload->ManualFencesToWait)
		{
			checkf(!bFenceSignalled.IsValid() || *bFenceSignalled, TEXT("The signal operation for a provided manual D3D12 fence has not yet happened."));
			IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitWaitManualFence(Payload, Queue, LocalFence, Value));
			VERIFYD3D12RESULT(Queue.D3DCommandQueue->Wait(LocalFence, Value));
		}
	};

	auto Flush = [&]()
	{
		if (FirstPayload == LastPayload)
			return;

		FD3D12Queue& Queue = PayloadsToSubmit[FirstPayload]->Queue;

		// Build SOA layout needed to call ExecuteCommandLists().
		TArray<FD3D12CommandList*, TInlineAllocator<128>> CommandLists;
		TArray<ID3D12CommandList*, TInlineAllocator<128>> D3DCommandLists;
#if ENABLE_RESIDENCY_MANAGEMENT
		TArray<FD3D12ResidencySet*, TInlineAllocator<128>> ResidencySets;
#endif

		uint64 Time = FPlatformTime::Cycles64();

		// Accumulate the command lists from the payload
		for (uint32 Index = FirstPayload; Index < LastPayload; ++Index)
		{
			FD3D12Payload* Payload = PayloadsToSubmit[Index];
			check(&Payload->Queue == &Queue);

			for (FD3D12CommandList* CommandList : Payload->CommandListsToExecute)
			{
				check(CommandList->IsClosed());
				CommandList->FlushProfilerEvents(Payload->EventStream, Time);

				D3DCommandLists.Add(CommandList->Interfaces.CommandList);

#if ENABLE_RESIDENCY_MANAGEMENT
				ResidencySets.Add(CommandList->CloseResidencySet());
#endif
			}
			CommandLists.Append(MoveTemp(Payload->CommandListsToExecute));
		}

		IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitPlatformCommandLists(
			Queue,
			MakeConstArrayView<FD3D12Payload*>(&PayloadsToSubmit[FirstPayload], LastPayload - FirstPayload),
			D3DCommandLists
		));

		const int32 MaxBatchSize = GetMaxExecuteBatchSize(Queue.QueueType);
		const int32 NumCommandLists = D3DCommandLists.Num();

		for (int32 DispatchNum, Offset = 0; Offset < NumCommandLists; Offset += DispatchNum)
		{
			DispatchNum = FMath::Min(NumCommandLists - Offset, MaxBatchSize);

			extern int32 GD3D12MaxCommandsPerCommandList;
			if (GD3D12MaxCommandsPerCommandList > 0)
			{
				// Limit the dispatch group based on the total number of commands each command list contains, so that we
				// don't submit more than approx "GD3D12MaxCommandsPerCommandList" commands per call to ExecuteCommandLists().
				int32 Index = 0;
				for (int32 NumCommands = 0; Index < DispatchNum && NumCommands < GD3D12MaxCommandsPerCommandList; ++Index)
				{
					NumCommands += CommandLists[Offset + Index]->State.NumCommands;
				}

				DispatchNum = Index;
			}

			INC_DWORD_STAT(STAT_D3D12ExecutedCommandListBatches);
			INC_DWORD_STAT_BY(STAT_D3D12ExecutedCommandLists, DispatchNum);

			Queue.ExecuteCommandLists(
				MakeArrayView<ID3D12CommandList*>(&D3DCommandLists[Offset], DispatchNum)
#if ENABLE_RESIDENCY_MANAGEMENT
				, MakeArrayView<FD3D12ResidencySet*>(&ResidencySets[Offset], DispatchNum)
#endif
			);

#if LOG_EXECUTE_COMMAND_LISTS
			LogExecuteCommandLists(DispatchNum, &D3DCommandLists[Offset]);
#endif
		}

		// Release the FD3D12CommandList instances back to the parent device object pool.
		for (FD3D12CommandList* CommandList : CommandLists)
		{
			CommandList->Device->ReleaseCommandList(CommandList);
		}

		FirstPayload = LastPayload;
	};

	auto Signal = [this](FD3D12Payload* Payload)
	{
		FD3D12Queue& Queue = Payload->Queue;

		// Signal any manual fences
		for (auto& [ManualFence, Value, bFenceSignalled] : Payload->ManualFencesToSignal)
		{
			IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitSignalManualFence(Payload, Queue, ManualFence, Value));

			VERIFYD3D12RESULT(Queue.D3DCommandQueue->Signal(ManualFence, Value));
			if (bFenceSignalled)
			{
				*bFenceSignalled = true;
			}
		}

		// Signal the queue fence
		if (Payload->RequiresQueueFenceSignal())
		{
			check(Queue.Fence.LastSignaledValue < Payload->CompletionFenceValue);

			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FSignalFence>(
				  FPlatformTime::Cycles64()
				, Payload->CompletionFenceValue
			);

			IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::SubmitSignalQueueFence(Payload, Queue, Payload->CompletionFenceValue));

			VERIFYD3D12RESULT(Queue.D3DCommandQueue->Signal(Queue.Fence.D3DFence, Payload->CompletionFenceValue));
			Queue.Fence.LastSignaledValue.store(Payload->CompletionFenceValue, std::memory_order_release);
		}

		if (Payload->EndFrameEvent.IsSet())
		{
			Payload->EndFrameEvent->CPUTimestamp = FPlatformTime::Cycles64();
			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(*Payload->EndFrameEvent);
		}

		if (Payload->PresentArgs.IsValid())
		{
			Payload->PresentArgs.DrawingViewport->PresentOnSubmissionThread(Payload->PresentArgs);
		}

		// Submission of this payload is completed. Signal the submission event if one was provided.
		if (Payload->SubmissionEvent)
		{
			Payload->SubmissionEvent->DispatchSubsequents();
		}
	};

	FD3D12Queue* PrevQueue = nullptr;
	for (FD3D12Payload* Payload : PayloadsToSubmit)
	{
		if (PrevQueue != &Payload->Queue)
		{
			Flush();
			PrevQueue = &Payload->Queue;
		}

		Payload->Queue.PendingInterrupt.Enqueue(Payload);

		if (Payload->Timing.IsSet())
		{
			Flush();

			if (FD3D12Timing* LocalTiming = *Payload->Timing)
			{
				// Calibrate the GPU timestamp / clock, if the queue type supports calibration.
				if (Payload->Queue.QueueType != ED3D12QueueType::Copy || Payload->Queue.Device->GetParentAdapter()->AreCopyQueueTimestampQueriesSupported())
				{
					SCOPED_NAMED_EVENT(CalibrateClocks, FColor::Red);
					VERIFYD3D12RESULT(Payload->Queue.D3DCommandQueue->GetClockCalibration(&LocalTiming->GPUTimestamp, &LocalTiming->CPUTimestamp));
					VERIFYD3D12RESULT(Payload->Queue.D3DCommandQueue->GetTimestampFrequency(&LocalTiming->GPUFrequency));
					QueryPerformanceFrequency(reinterpret_cast<LARGE_INTEGER*>(&LocalTiming->CPUFrequency));
				}
			}
		}

		if (Payload->HasWaitWork())
		{
			Flush();
			Wait(Payload);
		}

		if (Payload->HasUpdateReservedResourcesWork())
		{
			Flush();
			UpdateReservedResources(Payload);
		}

		if (Payload->HasPreExecuteWork())
		{
			Flush();
			Payload->PreExecute();
		}

		LastPayload++;

		if (Payload->HasSignalWork())
		{
			Flush();
			Signal(Payload);
		}
	}

	Flush();

	for (FD3D12Payload* Payload : PayloadsToSubmit)
	{
		// Only set this bool to true once we'll never touch the payload again on this thread.
		// This is because the bool hands ownership to the interrupt thread, which might delete the payload.
		Payload->bSubmitted = true;
	}

	PayloadsToSubmit.Reset();	
}

void FD3D12PayloadBase::AddQueueFenceWait(FD3D12Fence& InFence, uint64 InValue)
{
	for (auto& [Fence, Value] : QueueFencesToWait)
	{
		if (&Fence == &InFence)
		{
			Value = FMath::Max(Value, InValue);
			return;
		}
	}

	QueueFencesToWait.Add({ InFence, InValue });
}

UE_TRACE_EVENT_BEGIN(Cpu, SyncPoint_Wait, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, WaitingForName )
UE_TRACE_EVENT_END()

void FD3D12SyncPoint::Wait() const
{
	checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot wait for completion on the CPU."));

	if (!GraphEvent->IsComplete())
	{
		// Block the calling thread until the graph event is signaled by the interrupt thread.
#if CPUPROFILERTRACE_ENABLED
		UE_TRACE_LOG_SCOPED_T(Cpu, SyncPoint_Wait, CpuChannel)
			<< SyncPoint_Wait.WaitingForName(*GetDebugName().ToString());
#endif // CPUPROFILERTRACE_ENABLED

		FD3D12DynamicRHI::GetD3DRHI()->ProcessInterruptQueueUntil(GraphEvent);
	}

	check(GraphEvent->IsComplete());
}

void FD3D12DynamicRHI::ProcessInterruptQueueUntil(FGraphEvent* GraphEvent)
{
	if (InterruptThread)
	{
		if (GraphEvent && !GraphEvent->IsComplete())
		{
			GraphEvent->Wait();
		}
	}
	else
	{
		// Use the current thread to process the interrupt queue until the sync point we're waiting for is signaled.
		// If GraphEvent is nullptr, process the queue until no further progress is made (assuming we can acquire the lock), then return.
		if (!GraphEvent || !GraphEvent->IsComplete())
		{
			// If we're waiting for a sync point, accumulate the idle time
			UE::Stats::FThreadIdleStats::FScopeIdle IdleScope(/* bIgnore = */GraphEvent == nullptr);

		Retry:
			if (InterruptCS.TryLock())
			{
				TGuardValue<uint32> Guard(InterruptThreadID, FPlatformTLS::GetCurrentThreadId());
				
				FProcessResult Result;
				do { Result = ProcessInterruptQueue(); }
				// If we have a sync point, keep processing until the sync point is signaled.
				// Otherwise, process until no more progress is being made.
				while (GraphEvent
					? !GraphEvent->IsComplete()
					: EnumHasAllFlags(Result.Status, EQueueStatus::Processed)
				);

				InterruptCS.Unlock();
			}
			else if (GraphEvent && !GraphEvent->IsComplete())
			{
				// Failed to get the lock. Another thread is processing the interrupt queue. Try again...
				FPlatformProcess::SleepNoStats(0);
				goto Retry;
			}
		}
	}
}

void FD3D12DynamicRHI::ProcessInterruptQueueOnGPUCrash(HRESULT D3DResult)
{
	// This function will not return.

	// We know this function was called due to a GPU crash, so let the thread know.
	GGPUD3DResult = D3DResult;
	GGPUCrashDetected.store(true);

	if (InterruptThread)
	{
		// Since we have an interrupt thread, allow it to process the GPU crash.
		// This is necessary so it can retrieve all the active payloads for resolving breadcrumbs.
		InterruptThread->Kick();

		// Wait for the interrupt thread to exit (which will never happen).
		InterruptThread->Join();
	}
	else
	{
		// If we have no interrupt thread, assume ownership on the current thread
		// (or block forever on the scope lock if multiple threads enter this function).
		FScopeLock Lock(&InterruptCS);
		TGuardValue<uint32> Guard(InterruptThreadID, FPlatformTLS::GetCurrentThreadId());

		while (true)
		{
			ProcessInterruptQueue();
		}
	}
}

bool FD3D12DynamicRHI::IsInInterruptThread() const
{
	uint32 ThisThreadID = FPlatformTLS::GetCurrentThreadId();

	// If we don't have a dedicated interrupt thread, the thread currently acting
	// as the interrupt thread is tracked via the InterruptThreadID field.

	if (InterruptThread)
	{
		return ThisThreadID == InterruptThread->GetThreadID();
	}
	else
	{
		return ThisThreadID == InterruptThreadID;
	}
}

D3D12_QUERY_DATA_PIPELINE_STATISTICS& operator += (D3D12_QUERY_DATA_PIPELINE_STATISTICS& LHS, D3D12_QUERY_DATA_PIPELINE_STATISTICS const& RHS)
{
	LHS.IAVertices	  += RHS.IAVertices;
	LHS.IAPrimitives  += RHS.IAPrimitives;
	LHS.VSInvocations += RHS.VSInvocations;
	LHS.GSInvocations += RHS.GSInvocations;
	LHS.GSPrimitives  += RHS.GSPrimitives;
	LHS.CInvocations  += RHS.CInvocations;
	LHS.CPrimitives	  += RHS.CPrimitives;
	LHS.PSInvocations += RHS.PSInvocations;
	LHS.HSInvocations += RHS.HSInvocations;
	LHS.DSInvocations += RHS.DSInvocations;
	LHS.CSInvocations += RHS.CSInvocations;
	return LHS;
}

FD3D12DynamicRHI::FProcessResult FD3D12DynamicRHI::ProcessInterruptQueue()
{
	SCOPED_NAMED_EVENT_TEXT("InterruptQueue_Process", FColor::Yellow);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ProcessInterruptQueue"));

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::ProcessInterruptQueueEnter());

	// Timer that clamps each tick to prevent false positive GPU timeouts
	// when a debugger is attached and the process is broken.
	struct FTimer
	{
		uint64 Elapsed;
		uint64 Last;

		FTimer()
			: Elapsed(0)
			, Last(FPlatformTime::Cycles64())
		{}

		void Tick()
		{
			static const uint64 MaxDeltaCycles = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64()); // 1 second
			uint64 Current = FPlatformTime::Cycles64();
			Elapsed += FMath::Min(MaxDeltaCycles, Current - Last);
			Last = Current;
		}
	} static Timer;

	Timer.Tick();

	auto CheckForDeviceRemoved = [this](FD3D12Queue& Queue)
	{
		// If we get an error code here, we can't pass it directly to VERIFYD3D12RESULT, because that expects DXGI_ERROR_DEVICE_REMOVED,
		// DXGI_ERROR_DEVICE_RESET etc. and wants to obtain the reason code itself by calling GetDeviceRemovedReason (again).
		HRESULT DeviceRemovedReason = Queue.Device->GetDevice()->GetDeviceRemovedReason();
		if (DeviceRemovedReason != S_OK)
		{
			TerminateOnGPUCrash(NullOpt);
		}
	};

	FProcessResult Result;
	ForEachQueue([&](FD3D12Queue& CurrentQueue)
	{
		while (FD3D12Payload** PayloadPtr = CurrentQueue.PendingInterrupt.Peek())
		{
			FD3D12Payload* Payload = *PayloadPtr;

			if (!Payload->bSubmitted)
				break;

			// Check for GPU completion
			uint64 CompletedFenceValue = CurrentQueue.Fence.D3DFence->GetCompletedValue();
			uint64 LastSignaledFenceValue = CurrentQueue.Fence.LastSignaledValue.load(std::memory_order_acquire);

			// If the GPU crashes or hangs, the driver will signal all fences to UINT64_MAX.
			if (CompletedFenceValue == UINT64_MAX)
			{
				CheckForDeviceRemoved(CurrentQueue);
			}

			if (CompletedFenceValue < Payload->CompletionFenceValue)
			{
				// Command list batch has not yet completed on this queue.
				// Ask the driver to wake this thread again when the required value is reached.
				if (InterruptThread && !CurrentQueue.Fence.bInterruptAwaited)
				{
					SCOPED_NAMED_EVENT_TEXT("SetEventOnCompletion", FColor::Red);
					VERIFYD3D12RESULT(CurrentQueue.Fence.D3DFence->SetEventOnCompletion(Payload->CompletionFenceValue, InterruptThread->Event));
					CurrentQueue.Fence.bInterruptAwaited = true;
				}

				// Skip processing this queue and move on to the next.
				Result.Status |= EQueueStatus::Pending;

				// Detect a hung GPU
				if (!Payload->SubmissionTime.IsSet() && LastSignaledFenceValue >= Payload->CompletionFenceValue)
				{
					//
					// Keep track of the first time we've checked for completion on the interrupt thread.
					// We set this here to avoid false positives when a debugger is attached. If we'd set this on the submission thread, it
					// is possible for the title to be paused by the debugger after the time is set but before the payload has reached the GPU.
					//
					Payload->SubmissionTime = Timer.Elapsed;
				}

				if (Payload->SubmissionTime.IsSet() && Payload->SubmissionTime != TNumericLimits<uint64>::Max())
				{
					static const double CyclesPerSecond = 1.0 / FPlatformTime::GetSecondsPerCycle64();
					const uint64 TimeoutCycles = FMath::TruncToInt64(GD3D12SubmissionTimeout * CyclesPerSecond);

					uint64 ElapsedCycles = Timer.Elapsed - Payload->SubmissionTime.GetValue();

					if (ElapsedCycles > TimeoutCycles)
					{
						// The last submission on this pipe did not complete within the timeout period. Assume the GPU has hung.
						HandleGpuTimeout(Payload, ElapsedCycles * FPlatformTime::GetSecondsPerCycle64());

						// Set to int max to indicate we've already reported the timeout for this payload.
						Payload->SubmissionTime = TNumericLimits<uint64>::Max();
					}
					else
					{
						// Adjust the event wait timeout to cause the interrupt thread to wake automatically when
						// the timeout for this payload is reached, assuming it hasn't been woken by the GPU already.
						uint64 RemainingCycles = TimeoutCycles - ElapsedCycles;
						uint32 RemainingMilliseconds = FMath::TruncToInt(RemainingCycles * FPlatformTime::GetSecondsPerCycle64() * 1000.0);
						Result.WaitTimeout = FMath::Min(Result.WaitTimeout, RemainingMilliseconds);
					}
				}
				break;
			}

			// At this point, the current command list has completed on the GPU.
			CurrentQueue.Fence.bInterruptAwaited = false;
			CurrentQueue.PendingInterrupt.Dequeue();
			Result.Status |= EQueueStatus::Processed;

			IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::InterruptQueueFenceSignaled(CurrentQueue, Payload, CompletedFenceValue, LastSignaledFenceValue));

			// Resolve query results
			{
				for (FD3D12QueryLocation& Query : Payload->BatchedObjects.OcclusionQueries)
				{
					check(Query.Target);
					Query.CopyResultTo(Query.Target);
				}

				for (FD3D12QueryLocation& Query : Payload->BatchedObjects.PipelineStatsQueries)
				{
					if (Query.Target)
					{
						Query.CopyResultTo(Query.Target);
					}
					else
					{
						// Pipeline stats queries without targets are the ones that surround whole command lists.
						CurrentQueue.Timing->PipelineStats += Query.GetResult<D3D12_QUERY_DATA_PIPELINE_STATISTICS>();
					}
				}

				if (Payload->BatchedObjects.TimestampQueries.Num())
				{
					// Some timestamp queries report in microseconds
					const double MicrosecondsScale = 1000000.0 / CurrentQueue.Device->GetTimestampFrequency(CurrentQueue.QueueType);

					for (FD3D12QueryLocation& Query : Payload->BatchedObjects.TimestampQueries)
					{
						if (Query.Target)
						{
							Query.CopyResultTo(Query.Target);
						}

						switch (Query.Type)
						{
						case ED3D12QueryType::TimestampMicroseconds:
						case ED3D12QueryType::TimestampRaw:
							check(Query.Target);
							if (Query.Type == ED3D12QueryType::TimestampMicroseconds)
							{
								// Convert to microseconds
								*static_cast<uint64*>(Query.Target) = FPlatformMath::TruncToInt(double(*static_cast<uint64*>(Query.Target)) * MicrosecondsScale);
							}
							break;

						case ED3D12QueryType::ProfilerTimestampTOP:
						case ED3D12QueryType::ProfilerTimestampBOP:
							{
								// Convert from GPU timestamp to CPU timestamp (relative to FPlatformTime::Cycles64())
								uint64& Target = *static_cast<uint64*>(Query.Target);

								uint64 GPUDelta = Target - CurrentQueue.Timing->GPUTimestamp;
								uint64 CPUDelta = (GPUDelta * CurrentQueue.Timing->CPUFrequency) / CurrentQueue.Timing->GPUFrequency;

								Target = CPUDelta + CurrentQueue.Timing->CPUTimestamp;
							}
							break;
						}
					}
				}
			}

			if (!Payload->EventStream.IsEmpty())
			{
				check(CurrentQueue.Timing);
				CurrentQueue.Timing->EventStream.Append(MoveTemp(Payload->EventStream));
			}

			if (Payload->Timing.IsSet())
			{
				// Switch the new timing struct into the queue. This redirects timestamp results to separate each frame's work.
				CurrentQueue.Timing = Payload->Timing.GetValue();
			}

			// Signal the CPU events of all sync points associated with this batch.
			for (FD3D12SyncPointRef& SyncPoint : Payload->SyncPointsToSignal)
			{
				if (SyncPoint->GraphEvent)
				{
					SyncPoint->GraphEvent->DispatchSubsequents();
				}
			}

			// We're done with this payload now.

			// GPU resources the payload is holding a reference to will be cleaned up here.
			// E.g. command list allocators, which get recycled on the parent device.
			delete Payload;
		}

		CheckForDeviceRemoved(CurrentQueue);
	});

	if (GGPUCrashDetected.load())
	{
		// If this was set by ProcessInterruptQueueOnGPUCrash, we know a crash was detected, so process it immediately. We can't always rely on
		// queue processing to catch it, as GetDeviceRemovedReason sometimes returns S_OK despite an earlier API call having reported a lost device.
		TerminateOnGPUCrash(GGPUD3DResult);
	}

	IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::ProcessInterruptQueueExit(static_cast<uint8>(Result.Status)));
	return Result;
}

FD3D12PayloadBase::FD3D12PayloadBase(FD3D12Queue& Queue)
	: Queue(Queue)
	, EventStream(Queue.GetProfilerQueue())
{
}

FD3D12PayloadBase::~FD3D12PayloadBase()
{
	for (FD3D12CommandAllocator* Allocator : AllocatorsToRelease)
	{
		Queue.Device->ReleaseCommandAllocator(Allocator);
	}
}

void FD3D12PayloadBase::PreExecute()
{
	if (PreExecuteCallback)
	{
		PreExecuteCallback(Queue.D3DCommandQueue);
	}
}

void FD3D12DynamicRHI::ProcessTimestamps(FD3D12TimingArray const& TimingPerQueue)
{
	{
		TArray<UE::RHI::GPUProfiler::FEventStream, TInlineAllocator<GD3D12MaxNumQueues>> Streams;
		for (auto const& Timing : TimingPerQueue)
		{
			Streams.Add(MoveTemp(Timing->EventStream));
		}
		UE::RHI::GPUProfiler::ProcessEvents(Streams);
	}

	D3D12_QUERY_DATA_PIPELINE_STATISTICS PipelineStats{};
	for (auto const& Current : TimingPerQueue)
	{
		PipelineStats += Current->PipelineStats;
	}

	SET_DWORD_STAT(STAT_D3D12RHI_IAVertices   , PipelineStats.IAVertices   );
	SET_DWORD_STAT(STAT_D3D12RHI_IAPrimitives , PipelineStats.IAPrimitives );
	SET_DWORD_STAT(STAT_D3D12RHI_VSInvocations, PipelineStats.VSInvocations);
	SET_DWORD_STAT(STAT_D3D12RHI_GSInvocations, PipelineStats.GSInvocations);
	SET_DWORD_STAT(STAT_D3D12RHI_GSPrimitives , PipelineStats.GSPrimitives );
	SET_DWORD_STAT(STAT_D3D12RHI_CInvocations , PipelineStats.CInvocations );
	SET_DWORD_STAT(STAT_D3D12RHI_CPrimitives  , PipelineStats.CPrimitives  );
	SET_DWORD_STAT(STAT_D3D12RHI_PSInvocations, PipelineStats.PSInvocations);
	SET_DWORD_STAT(STAT_D3D12RHI_HSInvocations, PipelineStats.HSInvocations);
	SET_DWORD_STAT(STAT_D3D12RHI_DSInvocations, PipelineStats.DSInvocations);
	SET_DWORD_STAT(STAT_D3D12RHI_CSInvocations, PipelineStats.CSInvocations);
}
