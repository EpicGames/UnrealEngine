// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "D3D12BindlessDescriptors.h"
#include "D3D12Query.h"
#include "D3D12Queue.h"
#include "D3D12RHICommon.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "RHIBreadcrumbs.h"
#include "GPUProfiler.h"

enum class ED3D12QueueType;

class FD3D12CommandAllocator;
class FD3D12CommandList;
class FD3D12DynamicRHI;
class FD3D12QueryHeap;
class FD3D12Queue;
class FD3D12Timing;
class FD3D12Buffer;
class FD3D12Resource;
class FD3D12Viewport;

class FD3D12SyncPoint;
using FD3D12SyncPointRef = TRefCountPtr<FD3D12SyncPoint>;

// Helper macro to create a static FName from a string literal, avoiding repeated FName construction.
#define UE_D3D12_STATIC_FNAME(String) (([]() -> const FName& { static const FName Name(TEXT(String)); return Name; })())

enum class ED3D12SyncPointType
{
	// Sync points of this type do not include an FGraphEvent, so cannot
	// report completion to the CPU (via either IsComplete() or Wait())
	GPUOnly,

	// Sync points of this type include an FGraphEvent. The IsComplete() and Wait() functions
	// can be used to poll for completion from the CPU, or block the CPU, respectively.
	GPUAndCPU,
};

// Fence type used by the device queues to manage GPU completion
struct FD3D12Fence
{
	FD3D12Queue* const OwnerQueue;
	TRefCountPtr<ID3D12Fence> D3DFence;
	uint64 NextCompletionValue = 1;
	std::atomic<uint64> LastSignaledValue = 0;
	bool bInterruptAwaited = false;

	FD3D12Fence(FD3D12Queue* OwnerQueue)
		: OwnerQueue(OwnerQueue)
	{}
};

// Used by FD3D12SyncPoint and the submission thread to fix up signaled fence values at the end-of-pipe
struct FD3D12ResolvedFence
{
	FD3D12Fence& Fence;
	uint64 Value = 0;

	FD3D12ResolvedFence(FD3D12Fence& Fence, uint64 Value)
		: Fence(Fence)
		, Value(Value)
	{}
};

//
// A sync point is a logical point on a GPU queue's timeline that can be awaited by other queues, or the CPU.
// These are used throughout the RHI as a way to abstract the underlying D3D12 fences. The submission thread 
// manages the underlying fences and signaled values, and reports completion to the relevant sync points via 
// an FGraphEvent.
//
// Sync points are one-shot, meaning they represent a single timeline point, and are released after use, via ref-counting.
// Use FD3D12SyncPoint::Create() to make a new sync point and hold a reference to it via a FD3D12SyncPointRef object.
//
class FD3D12SyncPoint final : public FRefCountedObject
{
	friend FD3D12DynamicRHI;
	friend FD3D12Queue;

	static TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MemoryPool;

	// No copying or moving
	FD3D12SyncPoint(FD3D12SyncPoint const&) = delete;
	FD3D12SyncPoint(FD3D12SyncPoint&&) = delete;

	TOptional<FD3D12ResolvedFence> ResolvedFence;
	FGraphEventRef GraphEvent;

	FD3D12SyncPoint(ED3D12SyncPointType Type, FName InDebugName)
#if RHI_USE_SYNC_POINT_DEBUG_NAME
		:DebugName(InDebugName)
#endif
	{
		if (Type == ED3D12SyncPointType::GPUAndCPU)
		{
			GraphEvent = FGraphEvent::CreateGraphEvent();
		}
	}

public:
	static FD3D12SyncPointRef Create(ED3D12SyncPointType Type, FName InDebugName)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateSyncPoint"));
		return new FD3D12SyncPoint(Type, InDebugName);
	}

	bool IsComplete() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot check completion on the CPU."));
		return GraphEvent->IsComplete();
	}

	void Wait() const;

	FGraphEvent* GetGraphEvent() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event."));
		return GraphEvent;
	}

	ED3D12SyncPointType GetType() const
	{
		return GraphEvent != nullptr
			? ED3D12SyncPointType::GPUAndCPU
			: ED3D12SyncPointType::GPUOnly;
	}

	void* operator new(size_t Size)
	{
		check(Size == sizeof(FD3D12SyncPoint));

		void* Memory = MemoryPool.Pop();
		if (!Memory)
		{
			Memory = FMemory::Malloc(sizeof(FD3D12SyncPoint), alignof(FD3D12SyncPoint));
		}
		return Memory;
	}

	void operator delete(void* Pointer)
	{
		MemoryPool.Push(Pointer);
	}
	 
	inline FName GetDebugName() const
	{
#if RHI_USE_SYNC_POINT_DEBUG_NAME
		return DebugName;
#else
		return UE_D3D12_STATIC_FNAME("SyncPointBusyWait");
#endif
	}

private:

#if RHI_USE_SYNC_POINT_DEBUG_NAME
	FName DebugName;
#endif

};

struct FD3D12CommitReservedResourceDesc
{
	FD3D12Resource* Resource = nullptr;
	uint64 CommitSizeInBytes = 0;
};

struct FD3D12BatchedPayloadObjects
{
	TArray<FD3D12QueryLocation> TimestampQueries;
	TArray<FD3D12QueryLocation> OcclusionQueries;
	TArray<FD3D12QueryLocation> PipelineStatsQueries;
	TMap<TRefCountPtr<FD3D12QueryHeap>, TArray<FD3D12QueryRange>> QueryRanges;

	bool IsEmpty() const
	{
		return
			   TimestampQueries    .Num() == 0
			&& OcclusionQueries    .Num() == 0
			&& PipelineStatsQueries.Num() == 0
			&& QueryRanges         .Num() == 0
		;
	}
};

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FD3D12PayloadBaseFixLayout
{
	virtual ~FD3D12PayloadBaseFixLayout() = default;
};

struct FD3D12PresentArgs
{
	//Sync event -> Present called
	FGraphEventRef	PresentEvent;

	// In params
	FD3D12Viewport*	DrawingViewport = nullptr;
	uint64 FrameCounter = 0;
	bool bLockToVsync = false;

	inline bool IsValid() const
	{
		return DrawingViewport != nullptr;
	}
};

// A single unit of work (specific to a single GPU node and queue type) to be processed by the submission thread.
struct FD3D12PayloadBase : public FD3D12PayloadBaseFixLayout
{
	// Used to signal FD3D12ManualFence instances on the submission thread.
	struct FManualFence
	{
		// The D3D fence to signal
		TRefCountPtr<ID3D12Fence> Fence;

		// The value to signal the fence with.
		uint64 Value;

		// Optional bool pointer to check whether this fence has been signalled
		TSharedPtr<std::atomic<bool>> bHasSignalled;

		FManualFence() = default;
		FManualFence(TRefCountPtr<ID3D12Fence>&& Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled)
			: Fence(MoveTemp(Fence))
			, Value(Value)
			, bHasSignalled(bHasSignalled)
		{}
	};

	// Constants
	FD3D12Queue& Queue;

	// Wait
	struct : public TArray<FD3D12SyncPointRef>
	{
		// Used to pause / resume iteration of the sync point array on the
		// submission thread when we find a sync point that is unresolved.
		int32 Index = 0;

	} SyncPointsToWait;

	struct FQueueFence
	{
		FD3D12Fence& Fence;
		uint64 Value;
	};
	TArray<FQueueFence, TInlineAllocator<GD3D12MaxNumQueues>> QueueFencesToWait;
	struct : TArray<FManualFence>
	{
		int32 Index = 0;

	} ManualFencesToWait;

	void AddQueueFenceWait(FD3D12Fence& Fence, uint64 Value);

	// UpdateReservedResources
	TArray<FD3D12CommitReservedResourceDesc> ReservedResourcesToCommit;

	// Flags.
	bool bAlwaysSignal = false;
	std::atomic<bool> bSubmitted { false };

	// Used by RHIRunOnQueue
	TFunction<void(ID3D12CommandQueue*)> PreExecuteCallback;

	// Execute
	TArray<FD3D12CommandList*> CommandListsToExecute;

	// Signal
	TArray<FManualFence> ManualFencesToSignal;
	TArray<FD3D12SyncPointRef> SyncPointsToSignal;
	uint64 CompletionFenceValue = 0;

	FGraphEventRef SubmissionEvent;
	TOptional<uint64> SubmissionTime;

	TOptional<FD3D12Timing*> Timing;

	// Cleanup
	TArray<FD3D12CommandAllocator*> AllocatorsToRelease;

	FD3D12BatchedPayloadObjects BatchedObjects;

	FD3D12PresentArgs PresentArgs;

#if WITH_RHI_BREADCRUMBS
	FRHIBreadcrumbRange BreadcrumbRange {};
	TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators {};
#endif

	UE::RHI::GPUProfiler::FEventStream EventStream;
	TOptional<UE::RHI::GPUProfiler::FEvent::FFrameBoundary> EndFrameEvent;

	virtual ~FD3D12PayloadBase();

	virtual void PreExecute();

	virtual bool HasPreExecuteWork() const
	{
		return PreExecuteCallback != nullptr;
	}

	virtual bool RequiresQueueFenceSignal() const
	{
		return bAlwaysSignal || SyncPointsToSignal.Num() > 0 || HasPreExecuteWork();
	}

	virtual bool HasWaitWork() const
	{
		return ManualFencesToWait.Num() > 0 || QueueFencesToWait.Num() > 0;
	}

	virtual bool HasUpdateReservedResourcesWork() const
	{
		return ReservedResourcesToCommit.Num() > 0;
	}

	virtual bool HasSignalWork() const
	{
		return RequiresQueueFenceSignal()
			|| ManualFencesToSignal.Num() > 0
			|| SubmissionEvent != nullptr
			|| PresentArgs.IsValid()
			|| EndFrameEvent.IsSet();
	}

protected:
	FD3D12PayloadBase(FD3D12Queue& Queue);
};

#include COMPILED_PLATFORM_HEADER(D3D12Submission.h)
