// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Texture.cpp: D3D texture RHI implementation.
	=============================================================================*/

#include "D3D12Texture.h"
#include "D3D12RHIPrivate.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "RHIUtilities.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreStats.h"
#include "RHICoreTexture.h"
#include "RHICoreTextureInitializer.h"
#include "RHITextureUtils.h"

int64 FD3D12GlobalStats::GDedicatedVideoMemory = 0;
int64 FD3D12GlobalStats::GDedicatedSystemMemory = 0;
int64 FD3D12GlobalStats::GSharedSystemMemory = 0;
int64 FD3D12GlobalStats::GTotalGraphicsMemory = 0;

int32 GAdjustTexturePoolSizeBasedOnBudget = 0;
static FAutoConsoleVariableRef CVarAdjustTexturePoolSizeBasedOnBudget(
	TEXT("D3D12.AdjustTexturePoolSizeBasedOnBudget"),
	GAdjustTexturePoolSizeBasedOnBudget,
	TEXT("Indicates if the RHI should lower the texture pool size when the application is over the memory budget provided by the OS. This can result in lower quality textures (but hopefully improve performance).")
	);

static TAutoConsoleVariable<int32> CVarUseUpdateTexture3DComputeShader(
	TEXT("D3D12.UseUpdateTexture3DComputeShader"),
	0,
	TEXT("If enabled, use a compute shader for UpdateTexture3D. Avoids alignment restrictions")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: on"),
	ECVF_RenderThreadSafe );

static bool GTexturePoolOnlyAccountStreamableTexture = false;
static FAutoConsoleVariableRef CVarTexturePoolOnlyAccountStreamableTexture(
	TEXT("D3D12.TexturePoolOnlyAccountStreamableTexture"),
	GTexturePoolOnlyAccountStreamableTexture,
	TEXT("Texture streaming pool size only account streamable texture .\n")
	TEXT(" - 0: All texture types are counted in the pool (legacy, default).\n")
	TEXT(" - 1: Only streamable textures are counted in the pool.\n")
	TEXT("When enabling the new behaviour, r.Streaming.PoolSize will need to be re-adjusted.\n"),
	ECVF_ReadOnly
);

extern int32 GD3D12BindResourceLabels;

///////////////////////////////////////////////////////////////////////////////////////////
// Texture Stats
///////////////////////////////////////////////////////////////////////////////////////////

#if STATS
static TStatId GetD3D12StatEnum(const FD3D12ResourceDesc& ResourceDesc)
{
	if (EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL))
	{
		return GET_STATID(STAT_D3D12RenderTargets);
	}

	if (EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS))
	{
		return GET_STATID(STAT_D3D12UAVTextures);
	}

	return GET_STATID(STAT_D3D12Textures);
}
#endif // STATS


void FD3D12TextureStats::UpdateD3D12TextureStats(FD3D12Texture& Texture, const FD3D12ResourceDesc& ResourceDesc, const FRHITextureDesc& TextureDesc, uint64 TextureSize, bool bNewTexture, bool bAllocating)
{
	if (TextureSize == 0)
	{
		return;
	}

	UE::RHICore::UpdateGlobalTextureStats(TextureDesc, TextureSize, GTexturePoolOnlyAccountStreamableTexture, bAllocating);

	const int64 TextureSizeDeltaInBytes = bAllocating ? static_cast<int64>(TextureSize) : -static_cast<int64>(TextureSize);

	INC_MEMORY_STAT_BY_FName(GetD3D12StatEnum(ResourceDesc).GetName(), TextureSizeDeltaInBytes);
	INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, TextureSizeDeltaInBytes);

	D3D12_GPU_VIRTUAL_ADDRESS GPUAddress = Texture.ResourceLocation.GetGPUVirtualAddress();
	if (GPUAddress == 0)
	{
		GPUAddress = reinterpret_cast<D3D12_GPU_VIRTUAL_ADDRESS>(Texture.ResourceLocation.GetAddressForLLMTracking());
	}

#if UE_MEMORY_TRACE_ENABLED
	// Textures don't have valid GPUVirtualAddress when IsTrackingAllAllocations() is false, so don't do memory trace in this case.
	const bool bTrackingAllAllocations = Texture.GetParentDevice()->GetParentAdapter()->IsTrackingAllAllocations();
	const bool bMemoryTrace = bTrackingAllAllocations || GPUAddress != 0;
#endif

	if (bAllocating)
	{
#if PLATFORM_WINDOWS
		// On Windows there is no way to hook into the low level d3d allocations and frees.
		// This means that we must manually add the tracking here.
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Texture.ResourceLocation.GetAddressForLLMTracking(), TextureSize, ELLMTag::GraphicsPlatform));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, Texture.ResourceLocation.GetAddressForLLMTracking(), TextureSize, ELLMTag::Textures));
		{
			LLM(UE_MEMSCOPE_DEFAULT(ELLMTag::Textures));

#if UE_MEMORY_TRACE_ENABLED
			// Skip if it's created as a
			// 1) standalone resource, because MemoryTrace_Alloc has been called in FD3D12Adapter::CreateCommittedResource
			// 2) placed resource from a pool allocator, because MemoryTrace_Alloc has been called in FD3D12Adapter::CreatePlacedResource
			if (bMemoryTrace && !Texture.ResourceLocation.IsStandaloneOrPooledPlacedResource())
			{
				MemoryTrace_Alloc(GPUAddress, TextureSize, ResourceDesc.Alignment, EMemoryTraceRootHeap::VideoMemory);
			}
#endif
		}
#endif
		INC_DWORD_STAT(STAT_D3D12TexturesAllocated);
	}
	else
	{
#if PLATFORM_WINDOWS
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Texture.ResourceLocation.GetAddressForLLMTracking()));
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, Texture.ResourceLocation.GetAddressForLLMTracking()));

#if UE_MEMORY_TRACE_ENABLED
		// Skip back buffers that aren't traced on alloc and don't have valid GPUVirtualAddress
		if (GPUAddress != 0)
		{
			MemoryTrace_Free(GPUAddress, EMemoryTraceRootHeap::VideoMemory);
		}
#endif
#endif
		INC_DWORD_STAT(STAT_D3D12TexturesReleased);
	}
}


void FD3D12TextureStats::D3D12TextureAllocated(FD3D12Texture& Texture)
{
	if (FD3D12Resource* D3D12Resource = Texture.GetResource())
	{
		const FD3D12ResourceDesc& ResourceDesc = D3D12Resource->GetDesc();
		const FRHITextureDesc& TextureDesc = Texture.GetDesc();

		// Don't update state for readback, virtual, or transient textures	
		if (!EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::Virtual | ETextureCreateFlags::CPUReadback) && !Texture.ResourceLocation.IsTransient())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::UpdateTextureStats);

			const uint64 TextureSize = Texture.ResourceLocation.GetSize();
			const bool bNewTexture = true;
			UpdateD3D12TextureStats(Texture, ResourceDesc, TextureDesc, TextureSize, bNewTexture, true);
		}
	}
}


void FD3D12TextureStats::D3D12TextureDeleted(FD3D12Texture& Texture)
{
	if (FD3D12Resource* D3D12Resource = Texture.GetResource())
	{
		const FD3D12ResourceDesc& ResourceDesc = D3D12Resource->GetDesc();
		const FRHITextureDesc& TextureDesc = Texture.GetDesc();

		// Don't update state for readback or transient textures, but virtual textures need to have their size deducted from calls to RHIVirtualTextureSetFirstMipInMemory.
		if (!EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::CPUReadback) && !Texture.ResourceLocation.IsTransient())
		{
			const uint64 TextureSize = Texture.ResourceLocation.GetSize();
			ensure(TextureSize > 0 || EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::Virtual) || Texture.ResourceLocation.IsAliased());

			const bool bNewTexture = false;
			UpdateD3D12TextureStats(Texture, ResourceDesc, TextureDesc, TextureSize, bNewTexture, false);
		}
	}
}


bool FD3D12Texture::CanBe4KAligned(const FD3D12ResourceDesc& Desc, EPixelFormat UEFormat)
{
	if (Desc.bReservedResource)
	{
		return false;
	}

	// Exclude video related formats
	if (UEFormat == PF_NV12 ||
		UEFormat == PF_P010)
	{
		return false;
	}

	// 4KB alignment is only available for non-RT/DS textures
	if (!EnumHasAnyFlags(Desc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL) &&
		Desc.SampleDesc.Count == 1)
	{
		D3D12_TILE_SHAPE Tile = {};
		Get4KTileShape(&Tile, Desc.Format, UEFormat, Desc.Dimension, Desc.SampleDesc.Count);

		uint32 TilesNeeded = GetTilesNeeded(Desc.Width, Desc.Height, Desc.DepthOrArraySize, Tile);

		constexpr uint32 NUM_4K_BLOCKS_PER_64K_PAGE = 16;
		return TilesNeeded <= NUM_4K_BLOCKS_PER_64K_PAGE;
	}
	else
	{
		return false;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////
// FD3D12DynamicRHI Texture functions
///////////////////////////////////////////////////////////////////////////////////////////

using namespace D3D12RHI;

void FD3D12DynamicRHI::SetFormatAliasedTexturesMustBeCreatedUsingCommonLayout(bool bValue)
{
	bFormatAliasedTexturesMustBeCreatedUsingCommonLayout = bValue;
}

FD3D12ResourceDesc FD3D12DynamicRHI::GetResourceDesc(const FRHITextureDesc& TextureDesc) const
{
	FD3D12ResourceDesc ResourceDesc;

	check(TextureDesc.Extent.X > 0 && TextureDesc.Extent.Y > 0 && TextureDesc.NumMips > 0);

	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat((DXGI_FORMAT)GPixelFormats[TextureDesc.Format].PlatformFormat, TextureDesc.Flags);

	bool bDenyShaderResource = false;
	if (TextureDesc.Dimension != ETextureDimension::Texture3D)
	{
		if (TextureDesc.IsTextureCube())
		{
			check(TextureDesc.Extent.X <= (int32)GetMaxCubeTextureDimension());
			check(TextureDesc.Extent.X == TextureDesc.Extent.Y);
		}
		else
		{
			check(TextureDesc.Extent.X <= (int32)GetMax2DTextureDimension());
			check(TextureDesc.Extent.Y <= (int32)GetMax2DTextureDimension());
		}

		if (TextureDesc.IsTextureArray())
		{
			check(TextureDesc.ArraySize <= (int32)GetMaxTextureArrayLayers());
		}

		uint32 ActualMSAACount = TextureDesc.NumSamples;
		uint32 ActualMSAAQuality = GetMaxMSAAQuality(ActualMSAACount);

		// 0xffffffff means not supported
		if (ActualMSAAQuality == 0xffffffff || EnumHasAnyFlags(TextureDesc.Flags, TexCreate_Shared))
		{
			// no MSAA
			ActualMSAACount = 1;
			ActualMSAAQuality = 0;
		}

		ResourceDesc = CD3DX12_RESOURCE_DESC::Tex2D(
			PlatformResourceFormat,
			TextureDesc.Extent.X,
			TextureDesc.Extent.Y,
			TextureDesc.ArraySize * (TextureDesc.IsTextureCube() ? 6 : 1),  // Array size
			TextureDesc.NumMips,
			ActualMSAACount,
			ActualMSAAQuality,
			D3D12_RESOURCE_FLAG_NONE);  // Add misc flags later

		if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_RenderTargetable))
		{
			check(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable));
			ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET;
		}
		else if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_DepthStencilTargetable))
		{
			check(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_RenderTargetable | TexCreate_ResolveTargetable));
			ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL;
		}
		else if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_ResolveTargetable))
		{
			check(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable));
			if (TextureDesc.Format == PF_DepthStencil || TextureDesc.Format == PF_ShadowDepth || TextureDesc.Format == PF_D24)
			{
				ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL;
			}
			else
			{
				ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET;
			}
		}

		// Only deny shader resources if it's a depth resource that will never be used as SRV
		if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_DepthStencilTargetable) && !EnumHasAnyFlags(TextureDesc.Flags, TexCreate_ShaderResource))
		{
			bDenyShaderResource = true;
		}
	}
	else // ETextureDimension::Texture3D
	{
		check(TextureDesc.Dimension == ETextureDimension::Texture3D);
		check(!EnumHasAnyFlags(TextureDesc.Flags, TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable));

		ResourceDesc = CD3DX12_RESOURCE_DESC::Tex3D(
			PlatformResourceFormat,
			TextureDesc.Extent.X,
			TextureDesc.Extent.Y,
			TextureDesc.Depth,
			TextureDesc.NumMips);

		if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_RenderTargetable))
		{
			ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET;
		}
	}

	if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_Shared))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;
	}

	if (bDenyShaderResource)
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE;
	}

	if (EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::UAV) && IsBlockCompressedFormat(TextureDesc.Format))
	{
		ResourceDesc.UAVPixelFormat = GetBlockCompressedFormatUAVAliasFormat(TextureDesc.Format);
	}

	if (EnumHasAnyFlags(TextureDesc.Flags, TexCreate_UAV))
	{
		ResourceDesc.Flags |= D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS;
	}

	if (EnumHasAllFlags(TextureDesc.Flags, TexCreate_ReservedResource))
	{
		checkf(GRHIGlobals.ReservedResources.Supported, TEXT("Reserved resources resources are not supported on this machine"));
		checkf(TextureDesc.IsTexture2D() || TextureDesc.IsTexture3D(), TEXT("Only 2D and 3D textures can be created as reserved resources"));
		checkf(!TextureDesc.IsTexture3D() || GRHIGlobals.ReservedResources.SupportsVolumeTextures, TEXT("Current RHI does not support reserved volume textures"));

		ResourceDesc.bReservedResource = true;
		ResourceDesc.Layout = D3D12_TEXTURE_LAYOUT_64KB_UNDEFINED_SWIZZLE;
	}

	ResourceDesc.PixelFormat = TextureDesc.Format;

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	ResourceDesc.bRequires64BitAtomicSupport = EnumHasAnyFlags(TextureDesc.Flags, ETextureCreateFlags::Atomic64Compatible);

	checkf(!(ResourceDesc.bRequires64BitAtomicSupport&& ResourceDesc.SupportsUncompressedUAV()), TEXT("Intel resource creation extensions don't support the new resource casting parameters."));
#endif

	// Check if the 4K aligment is possible
	ResourceDesc.Alignment = FD3D12Texture::CanBe4KAligned(ResourceDesc, (EPixelFormat)TextureDesc.Format) ? D3D12_SMALL_RESOURCE_PLACEMENT_ALIGNMENT : D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;

	return ResourceDesc;
}


FDynamicRHI::FRHICalcTextureSizeResult FD3D12DynamicRHI::RHICalcTexturePlatformSize(const FRHITextureDesc& InTextureDesc, uint32 FirstMipIndex)
{
	const D3D12_RESOURCE_DESC Desc = GetResourceDesc(InTextureDesc);
	const D3D12_RESOURCE_ALLOCATION_INFO AllocationInfo = GetAdapter().GetDevice(0)->GetResourceAllocationInfo(Desc);

	FDynamicRHI::FRHICalcTextureSizeResult Result;
	Result.Size = AllocationInfo.SizeInBytes;
	Result.Align = AllocationInfo.Alignment;
	return Result;
}


/**
 * Retrieves texture memory stats.
 */
void FD3D12DynamicRHI::RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	UE::RHICore::FillBaselineTextureMemoryStats(OutStats);

	OutStats.DedicatedVideoMemory = FD3D12GlobalStats::GDedicatedVideoMemory;
	OutStats.DedicatedSystemMemory = FD3D12GlobalStats::GDedicatedSystemMemory;
	OutStats.SharedSystemMemory = FD3D12GlobalStats::GSharedSystemMemory;
	OutStats.TotalGraphicsMemory = FD3D12GlobalStats::GTotalGraphicsMemory ? FD3D12GlobalStats::GTotalGraphicsMemory : -1;

	OutStats.LargestContiguousAllocation = OutStats.StreamingMemorySize;

#if PLATFORM_WINDOWS
	if (GAdjustTexturePoolSizeBasedOnBudget)
	{
		const FRHIMemoryStats& CurrentMemoryStats = GetAdapter().CollectMemoryStats();

		// Applications must explicitly manage their usage of physical memory and keep usage within the budget 
		// assigned to the application process. Processes that cannot keep their usage within their assigned budgets 
		// will likely experience stuttering, as they are intermittently frozen and paged out to allow other processes to run.
		const int64 TargetBudget = CurrentMemoryStats.BudgetLocal * 0.90f;	// Target using 90% of our budget to account for some fragmentation.
		OutStats.TotalGraphicsMemory = TargetBudget;

		const int64 BudgetPadding = TargetBudget * 0.05f;
		const int64 AvailableSpace = TargetBudget - int64(CurrentMemoryStats.UsedLocal);	// Note: AvailableSpace can be negative
		const int64 PreviousTexturePoolSize = RequestedTexturePoolSize;
		const bool bOverbudget = AvailableSpace < 0;

		// Only change the pool size if overbudget, or a reasonable amount of memory is available
		const int64 MinTexturePoolSize = int64(100 * 1024 * 1024);
		if (bOverbudget)
		{
			// Attempt to lower the texture pool size to meet the budget.
			const bool bOverActualBudget = CurrentMemoryStats.UsedLocal > CurrentMemoryStats.BudgetLocal;
			UE_CLOG(bOverActualBudget, LogD3D12RHI, Warning, //-V567
				TEXT("Video memory usage is overbudget by %llu MB (using %lld MB/%lld MB budget). Usage breakdown: %lld MB (Streaming Textures), %lld MB (Non Streaming Textures). Last requested texture pool size is %lld MB. This can cause stuttering due to paging."),
				(CurrentMemoryStats.UsedLocal - CurrentMemoryStats.BudgetLocal) / 1024ll / 1024ll,
				CurrentMemoryStats.UsedLocal / 1024ll / 1024ll,
				CurrentMemoryStats.BudgetLocal / 1024ll / 1024ll,
				GRHIGlobals.StreamingTextureMemorySizeInKB / 1024ll,
				GRHIGlobals.NonStreamingTextureMemorySizeInKB / 1024ll,
				PreviousTexturePoolSize / 1024ll / 1024ll);

			const int64 DesiredTexturePoolSize = PreviousTexturePoolSize + AvailableSpace - BudgetPadding;
			OutStats.TexturePoolSize = FMath::Max(DesiredTexturePoolSize, MinTexturePoolSize);

			UE_CLOGF(bOverActualBudget && (OutStats.TexturePoolSize >= PreviousTexturePoolSize) && (OutStats.TexturePoolSize > MinTexturePoolSize), LogD3D12RHI, Fatal,
				"Video memory usage is overbudget by %llu MB and the texture pool size didn't shrink.",
				(CurrentMemoryStats.UsedLocal - CurrentMemoryStats.BudgetLocal) / 1024ll / 1024ll);
		}
		else if (AvailableSpace > BudgetPadding)
		{
			// Increase the texture pool size to improve quality if we have a reasonable amount of memory available.
			int64 DesiredTexturePoolSize = PreviousTexturePoolSize + AvailableSpace - BudgetPadding;
			if (GPoolSizeVRAMPercentage > 0)
			{
				// The texture pool size is a percentage of GTotalGraphicsMemory.
				const float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(OutStats.TotalGraphicsMemory);

				// Truncate texture pool size to MB (but still counted in bytes).
				DesiredTexturePoolSize = int64(FGenericPlatformMath::TruncToFloat(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;
			}

			// Make sure the desired texture pool size doesn't make us go overbudget.
			const bool bIsLimitedTexturePoolSize = GTexturePoolSize > 0;
			const int64 LimitedMaxTexturePoolSize = bIsLimitedTexturePoolSize ? GTexturePoolSize : INT64_MAX;
			const int64 MaxTexturePoolSize = FMath::Min(PreviousTexturePoolSize + AvailableSpace - BudgetPadding, LimitedMaxTexturePoolSize);	// Max texture pool size without going overbudget or the pre-defined max.
			OutStats.TexturePoolSize = FMath::Min(DesiredTexturePoolSize, MaxTexturePoolSize);
		}
		else
		{
			// Keep the previous requested texture pool size.
			OutStats.TexturePoolSize = PreviousTexturePoolSize;
		}

		check(OutStats.TexturePoolSize >= MinTexturePoolSize);
	}

	// Cache the last requested texture pool size.
	RequestedTexturePoolSize = OutStats.TexturePoolSize;
#endif // PLATFORM_WINDOWS
}

/**
 * Fills a texture with to visualize the texture pool memory.
 *
 * @param	TextureData		Start address
 * @param	SizeX			Number of pixels along X
 * @param	SizeY			Number of pixels along Y
 * @param	Pitch			Number of bytes between each row
 * @param	PixelSize		Number of bytes each pixel represents
 *
 * @return true if successful, false otherwise
 */
bool FD3D12DynamicRHI::RHIGetTextureMemoryVisualizeData(FColor* /*TextureData*/, int32 /*SizeX*/, int32 /*SizeY*/, int32 /*Pitch*/, int32 /*PixelSize*/)
{
	// currently only implemented for console (Note: Keep this function for further extension. Talk to NiklasS for more info.)
	return false;
}

/**
 * Creates a 2D texture optionally guarded by a structured exception handler.
 */
void SafeCreateTexture2D(
	FD3D12Device* pDevice, 
	FD3D12Adapter* Adapter,
	const FD3D12ResourceDesc& TextureDesc,
	const D3D12_CLEAR_VALUE* ClearValue, 
	FD3D12ResourceLocation* OutTexture2D, 
	FD3D12BaseShaderResource* Owner,
	EPixelFormat Format,
	ETextureCreateFlags Flags,
	ED3D12Access InInitialD3D12Access,
	ED3D12Access InDefaultD3D12Access,
	const TCHAR* Name)
{
#if GUARDED_TEXTURE_CREATES
	bool bDriverCrash = true;
	__try
	{
#endif // #if GUARDED_TEXTURE_CREATES

		const D3D12_HEAP_TYPE HeapType = EnumHasAnyFlags(Flags, TexCreate_CPUReadback) ? D3D12_HEAP_TYPE_READBACK : D3D12_HEAP_TYPE_DEFAULT;

		switch (HeapType)
		{
		case D3D12_HEAP_TYPE_READBACK:
			{
				uint64 Size = 0;
				pDevice->GetDevice()->GetCopyableFootprints(&TextureDesc, 0, TextureDesc.MipLevels * TextureDesc.DepthOrArraySize, 0, nullptr, nullptr, nullptr, &Size);

				FD3D12Resource* Resource = nullptr;
				VERIFYD3D12CREATETEXTURERESULT(Adapter->CreateBuffer(HeapType, pDevice->GetGPUMask(), pDevice->GetVisibilityMask(), Size, &Resource, Name), TextureDesc, pDevice->GetDevice());
				OutTexture2D->AsStandAlone(Resource);
			}
			break;

		case D3D12_HEAP_TYPE_DEFAULT:
		{
			if (TextureDesc.bReservedResource)
			{
				FD3D12Resource* Resource = nullptr;
				VERIFYD3D12CREATETEXTURERESULT(
					Adapter->CreateReservedResource(
						TextureDesc,
						pDevice->GetGPUMask(),
						InInitialD3D12Access,
						ED3D12ResourceStateMode::MultiState,
						InDefaultD3D12Access,
						ClearValue,
						&Resource,
						Name,
						false),
					TextureDesc,
					pDevice->GetDevice());

				D3D12_RESOURCE_ALLOCATION_INFO AllocInfo = pDevice->GetResourceAllocationInfo(TextureDesc);

				OutTexture2D->AsStandAlone(Resource, AllocInfo.SizeInBytes);

				if (EnumHasAllFlags(Flags, TexCreate_ImmediateCommit))
				{
					// NOTE: Accessing the queue from this thread is OK, as D3D12 runtime acquires a lock around all command queue APIs.
					// https://microsoft.github.io/DirectX-Specs/d3d/CPUEfficiency.html#threading
					Resource->CommitReservedResource(pDevice->GetQueue(ED3D12QueueType::Direct).D3DCommandQueue, UINT64_MAX /*commit entire resource*/);
				}
			}
			else
			{
				VERIFYD3D12CREATETEXTURERESULT(
					pDevice->GetTextureAllocator().AllocateTexture(
						TextureDesc,
						ClearValue,
						Format,
						*OutTexture2D,
						InInitialD3D12Access,
						InDefaultD3D12Access,
						Name),
					TextureDesc,
					pDevice->GetDevice());
			}

			OutTexture2D->SetOwner(Owner);
			break;
		}

		default:
			check(false);	// Need to create a resource here
		}

#if GUARDED_TEXTURE_CREATES
		bDriverCrash = false;
	}
	__finally
	{
		if (bDriverCrash)
		{
			UE_LOGF(LogD3D12RHI, Error,
				"Driver crashed while creating texture: %llux%ux%u %ls(0x%08x) with %u mips",
				TextureDesc.Width,
				TextureDesc.Height,
				TextureDesc.DepthOrArraySize,
				UE::DXGIUtilities::GetFormatString(TextureDesc.Format),
				(uint32)TextureDesc.Format,
				TextureDesc.MipLevels
				);
		}
	}
#endif // #if GUARDED_TEXTURE_CREATES
}

FD3D12Texture* FD3D12DynamicRHI::CreateNewD3D12Texture(const FRHITextureCreateDesc& CreateDesc, FD3D12Device* Device)
{
	return new FD3D12Texture(CreateDesc, Device);
}

D3D12_CLEAR_VALUE* UE::D3D12RHI::TextureUtils::FillClearValue(D3D12_CLEAR_VALUE& ClearValue, const FD3D12ResourceDesc& ResourceDesc, const FRHITextureDesc& TextureDesc)
{
	bool bCreateRTV = EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET);
	bool bCreateDSV = EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);

	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat((DXGI_FORMAT)GPixelFormats[TextureDesc.Format].PlatformFormat, TextureDesc.Flags);

	D3D12_CLEAR_VALUE* ClearValuePtr = nullptr;
	if (bCreateDSV && TextureDesc.ClearValue.ColorBinding == EClearBinding::EDepthStencilBound)
	{
		const DXGI_FORMAT PlatformDepthStencilFormat = UE::DXGIUtilities::FindDepthStencilFormat(PlatformResourceFormat);

		ClearValue = CD3DX12_CLEAR_VALUE(PlatformDepthStencilFormat, TextureDesc.ClearValue.Value.DSValue.Depth, (uint8)TextureDesc.ClearValue.Value.DSValue.Stencil);
		ClearValuePtr = &ClearValue;
	}
	else if (bCreateRTV && TextureDesc.ClearValue.ColorBinding == EClearBinding::EColorBound)
	{
		const bool bSRGB = EnumHasAnyFlags(TextureDesc.Flags, TexCreate_SRGB);
		const DXGI_FORMAT PlatformRenderTargetFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);

		ClearValue = CD3DX12_CLEAR_VALUE(PlatformRenderTargetFormat, TextureDesc.ClearValue.Value.Color);
		ClearValuePtr = &ClearValue;
	}

	return ClearValuePtr;
}

FD3D12DynamicRHI::FCreateTextureInternalResult FD3D12DynamicRHI::CreateTextureInternal(const FRHITextureCreateDesc& InCreateDesc, ID3D12ResourceAllocator* ResourceAllocator)
{
#if PLATFORM_WINDOWS
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::CreateTextureInternal);

	// Make local copy of create desc because certain flags might be removed before actual texture creation
	FRHITextureCreateDesc CreateDesc = InCreateDesc;
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	// Virtual textures currently not supported in default D3D12
	check(!EnumHasAnyFlags(CreateDesc.Flags, TexCreate_Virtual));

	const bool bCreateAsCopyDest = (ResourceAllocator == nullptr && CreateDesc.InitAction != ERHITextureInitAction::Default);

	// Get the resource desc
	FD3D12ResourceDesc ResourceDesc = GetResourceDesc(CreateDesc);

	D3D12_CLEAR_VALUE ClearValue;
	D3D12_CLEAR_VALUE* ClearValuePtr = UE::D3D12RHI::TextureUtils::FillClearValue(ClearValue, ResourceDesc, CreateDesc);

	const FD3D12Resource::FD3D12ResourceTypeHelper Type(ResourceDesc, D3D12_HEAP_TYPE_DEFAULT);
	const ED3D12Access DesiredD3D12Access = Type.GetOptimalInitialD3D12Access(ConvertToD3D12Access(CreateDesc.InitialState), true);

	const ED3D12Access InitialD3D12Access = [DesiredD3D12Access, bCreateAsCopyDest, &ResourceDesc]() {
		if (bFormatAliasedTexturesMustBeCreatedUsingCommonLayout
			&& ResourceDesc.SupportsUncompressedUAV())
		{
			// When creating a resource with castable formats, d3d12 uses enhanced barriers behind the scenes
			// which means we have to start it in the "common" state to then transition it to "legacy" barriers
			return ED3D12Access::Common;
		}
		else
		{
			return bCreateAsCopyDest ?
				// If we have initial data, we want the resource created in COPY_DEST so we can copy the data immediately
				ED3D12Access::CopyDest :
				// Otherwise, create the resource in the calculated optimal state
				DesiredD3D12Access;
		}
	}();

	FD3D12Adapter* Adapter = &GetAdapter();
	FD3D12Texture* D3D12TextureOut = Adapter->CreateLinkedObject<FD3D12Texture>(CreateDesc.GPUMask, [&](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
	{
		FD3D12Texture* NewTexture = CreateNewD3D12Texture(CreateDesc, Device);

#if NAME_OBJECTS
		if (CreateDesc.DebugName)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::SetDebugName);
			NewTexture->SetName(CreateDesc.DebugName);
		}
#endif // NAME_OBJECTS

		FD3D12ResourceLocation& Location = NewTexture->ResourceLocation;

		if (ResourceAllocator)
		{
			const D3D12_HEAP_TYPE HeapType = EnumHasAnyFlags(CreateDesc.Flags, TexCreate_CPUReadback) ? D3D12_HEAP_TYPE_READBACK : D3D12_HEAP_TYPE_DEFAULT;
			ResourceAllocator->AllocateTexture(
				Device->GetGPUIndex(),
				HeapType,
				ResourceDesc,
				(EPixelFormat)CreateDesc.Format,
				InitialD3D12Access,
				ED3D12ResourceStateMode::Default,
				DesiredD3D12Access,
				ClearValuePtr,
				CreateDesc.DebugName,
				Location);
			Location.SetOwner(NewTexture);
		}
		else if(EnumHasAnyFlags(CreateDesc.Flags, TexCreate_CPUReadback))
		{
			uint64 Size = 0;
			uint32 NumSubResources = ResourceDesc.MipLevels;
			if (CreateDesc.IsTextureArray())
			{
				NumSubResources *= ResourceDesc.DepthOrArraySize;
			}
			Device->GetDevice()->GetCopyableFootprints(&ResourceDesc, 0, NumSubResources, 0, nullptr, nullptr, nullptr, &Size);

			FD3D12Resource* Resource = nullptr;
			VERIFYD3D12CREATETEXTURERESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, Device->GetGPUMask(), Device->GetVisibilityMask(), Size, &Resource, CreateDesc.DebugName), ResourceDesc, Device->GetDevice());
			Location.AsStandAlone(Resource);
		}
		else if (CreateDesc.IsTexture3D())
		{
			if (ResourceDesc.bReservedResource)
			{
				FD3D12Resource* Resource = nullptr;
				VERIFYD3D12CREATETEXTURERESULT(
					Adapter->CreateReservedResource(
						ResourceDesc,
						Device->GetGPUMask(),
						InitialD3D12Access,
						ED3D12ResourceStateMode::MultiState,
						DesiredD3D12Access,
						ClearValuePtr,
						&Resource,
						CreateDesc.DebugName,
						false),
					ResourceDesc,
					Device->GetDevice());

				D3D12_RESOURCE_ALLOCATION_INFO AllocInfo = Device->GetResourceAllocationInfo(ResourceDesc);

				Location.AsStandAlone(Resource, AllocInfo.SizeInBytes);

				if (EnumHasAllFlags(CreateDesc.Flags, TexCreate_ImmediateCommit))
				{
					Resource->CommitReservedResource(Device->GetQueue(ED3D12QueueType::Direct).D3DCommandQueue, UINT64_MAX /*commit entire resource*/);
				}
			}
			else
			{
				VERIFYD3D12CREATETEXTURERESULT(
					Device->GetTextureAllocator().AllocateTexture(
						ResourceDesc,
						ClearValuePtr,
						CreateDesc.Format,
						Location,
						InitialD3D12Access,
						DesiredD3D12Access,
						CreateDesc.DebugName),
					ResourceDesc,
					Device->GetDevice());
			}

			Location.SetOwner(NewTexture);
		}
		else
		{
			SafeCreateTexture2D(
				Device,
				Adapter,
				ResourceDesc,
				ClearValuePtr,
				&Location,
				NewTexture,
				CreateDesc.Format,
				CreateDesc.Flags,
				InitialD3D12Access,
				DesiredD3D12Access,
				CreateDesc.DebugName);
		}

		// Unlock immediately if no initial data
		if (!bCreateAsCopyDest)
		{
			Location.UnlockPoolData();
		}

		check(Location.IsValid());

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
		if (ResourceDesc.NeedsUAVAliasWorkarounds())
		{
			Adapter->CreateUAVAliasResourceDesc(Location);
		}
#endif

		NewTexture->CreateViews(FirstLinkedObject);

#if WITH_GPUDEBUGCRASH
		if (EnumHasAnyFlags(CreateDesc.Flags, TexCreate_Invalid))
		{
			ID3D12Pageable* EvictableTexture = NewTexture->GetResource()->GetPageable();
			Device->GetDevice()->Evict(1, &EvictableTexture);
		}
#endif
		return NewTexture;
	});

	FD3D12TextureStats::D3D12TextureAllocated(*D3D12TextureOut);

	// Discard for all non-simultaneous-access RTV/DSV to be simple and safe.
	// THe debug layer reports these require Discard/Clear/Copy as the first operation:
	// - Placed resources
	// - Reserved resources
	// - Committed resources with D3D12_HEAP_FLAG_CREATE_NOT_ZEROED flag
	const bool bNeedsDiscard = (DesiredD3D12Access != ED3D12Access::Discard) &&
		!EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS) &&
		EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET | D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);

	FCreateTextureInternalResult Result{};
	Result.Texture = D3D12TextureOut;
	Result.CreateD3D12Access = InitialD3D12Access;
	Result.DesiredD3D12Access = DesiredD3D12Access;
	Result.bNeedsDiscard = bNeedsDiscard;

	return Result;
#else
	checkf(false, TEXT("XBOX_CODE_MERGE : Removed. The Xbox platform version should be used."));
	FCreateTextureInternalResult Result{};
	return Result;
#endif // PLATFORM_WINDOWS
}

void UE::D3D12RHI::TextureUtils::ReconcileInitialState(
	FRHICommandListBase& RHICmdList,
	FD3D12Texture* Texture,
	ED3D12Access CurrentD3D12Access,
	ED3D12Access NeededD3D12Access,
	bool bNeedsDiscard)
{
	if (bNeedsDiscard)
	{
		RHICmdList.EnqueueLambda([Texture, CurrentD3D12Access, NeededD3D12Access](FRHICommandListBase& ExecutingCmdList)
		{
			for (FD3D12Texture::FLinkedObjectIterator TextureIt(Texture); TextureIt; ++TextureIt)
			{
				const ETextureCreateFlags Flags = TextureIt->GetDesc().Flags;
				const uint32 GPUIndex = TextureIt->GetParentDevice()->GetGPUIndex();
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
				FD3D12Resource* const Resource = TextureIt->GetResource();

				ED3D12Access NextD3D12Access = CurrentD3D12Access;
				if (Context.GetPipeline() == ERHIPipeline::AsyncCompute)
				{
					if (!EnumHasAnyFlags(Flags, TexCreate_UAV))
					{
						checkf(false, TEXT("Can't discard non-UAVs for ERHIPipeline::AsyncCompute."));
						goto SkipDiscard;
					}

					NextD3D12Access = ED3D12Access::UAVMask;
				}
				else if (EnumHasAnyFlags(Flags, TexCreate_RenderTargetable))
				{
					NextD3D12Access = ED3D12Access::RTV;
				}
				else if (EnumHasAnyFlags(Flags, TexCreate_DepthStencilTargetable))
				{
					NextD3D12Access = ED3D12Access::DSVWrite;
				}

				if (CurrentD3D12Access != NextD3D12Access)
				{
					Context.AddBarrier(
						Resource,
						CurrentD3D12Access,
						NextD3D12Access,
						D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
					Context.FlushResourceBarriers();
				}

				Context.GetCommandList().GraphicsCommandList()->DiscardResource(Resource->GetResource(), nullptr);

			SkipDiscard:
				if (NextD3D12Access != NeededD3D12Access)
				{
					Context.AddBarrier(
						Resource,
						NextD3D12Access,
						NeededD3D12Access,
						D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
				}
			}
		});
	}
	else if (CurrentD3D12Access != NeededD3D12Access)
	{
		RHICmdList.EnqueueLambda([Texture, CurrentD3D12Access, NeededD3D12Access](FRHICommandListBase& ExecutingCmdList)
		{
			for (FD3D12Texture::FLinkedObjectIterator TextureIt(Texture); TextureIt; ++TextureIt)
			{
				const uint32 GPUIndex = TextureIt->GetParentDevice()->GetGPUIndex();
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, GPUIndex);
				Context.AddBarrier(
					TextureIt->GetResource(),
					CurrentD3D12Access,
					NeededD3D12Access,
					D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
			}
		});
	}
}

static uint32 CalculateSubresourceCount(const FRHITextureDesc& TextureDesc)
{
	// each mip of each array slice counts as a subresource
	uint16 ArraySize = TextureDesc.IsTextureArray() ? TextureDesc.ArraySize : 1;
	if (TextureDesc.IsTextureCube())
	{
		ArraySize *= 6;
	}
	return TextureDesc.NumMips * ArraySize;
}

uint64 UE::D3D12RHI::TextureUtils::CalculateResourceSize(FD3D12Texture* Texture)
{
	const uint32 NumSubresources = CalculateSubresourceCount(Texture->GetDesc());

	const D3D12_RESOURCE_DESC& ResourceDesc = Texture->GetResource()->GetDesc();
	uint64 AllocationSize = 0;
	Texture->GetParentDevice()->GetDevice()->GetCopyableFootprints(
		&ResourceDesc,
		0,
		NumSubresources,
		0,
		nullptr,
		nullptr,
		nullptr,
		&AllocationSize
	);

	return AllocationSize;
}

void UE::D3D12RHI::TextureUtils::CopyBulkData(void* UploadMemory, uint64 UploadMemorySize, FD3D12Texture* Texture, FResourceBulkDataInterface* BulkData)
{
	const uint32 NumSubresources = CalculateSubresourceCount(Texture->GetDesc());
	CA_ASSUME(NumSubresources > 0);

	const size_t FootprintsMemorySize = NumSubresources * (sizeof(D3D12_PLACED_SUBRESOURCE_FOOTPRINT) + sizeof(UINT) + sizeof(UINT64));
	const bool bAllocateOnStack = (FootprintsMemorySize < 4096);
	void* const FootprintsMemory = bAllocateOnStack ? FMemory_Alloca(FootprintsMemorySize) : FMemory::Malloc(FootprintsMemorySize);
	ON_SCOPE_EXIT
	{
		if (!bAllocateOnStack)
		{
			FMemory::Free(FootprintsMemory);
		}
	};

	CA_ASSUME(FootprintsMemory != nullptr);

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT* const Footprints = reinterpret_cast<D3D12_PLACED_SUBRESOURCE_FOOTPRINT*>(FootprintsMemory);
	UINT* const Rows = reinterpret_cast<UINT*>(Footprints + NumSubresources);
	UINT64* const RowSizeInBytes = reinterpret_cast<UINT64*>(Rows + NumSubresources);

	FD3D12Device* Device = Texture->GetParentDevice();

	const D3D12_RESOURCE_DESC& ResourceDesc = Texture->GetResource()->GetDesc();
	Device->GetDevice()->GetCopyableFootprints(&ResourceDesc, 0, NumSubresources, 0, Footprints, Rows, RowSizeInBytes, nullptr);

	check(UploadMemorySize >= BulkData->GetResourceBulkDataSize());

	uint8* const DestinationDataStart = reinterpret_cast<uint8*>(UploadMemory);
	const uint8* const SourceDataStart = reinterpret_cast<const uint8*>(BulkData->GetResourceBulkData());

	const uint8* SrcData = SourceDataStart;

	for (uint32 Subresource = 0; Subresource < NumSubresources; Subresource++)
	{
		uint8* DstData = DestinationDataStart + Footprints[Subresource].Offset;

		const uint32 NumRows = Rows[Subresource] * Footprints[Subresource].Footprint.Depth;
		const uint32 SrcRowPitch = RowSizeInBytes[Subresource];
		const uint32 DstRowPitch = Footprints[Subresource].Footprint.RowPitch;

		// If src and dst pitch are aligned, which is typically the case for the bulk of the data (most large mips, POT textures), we can use a single large memcpy()
		if (SrcRowPitch == DstRowPitch)
		{
			FMemory::Memcpy(DstData, SrcData, SrcRowPitch * NumRows);
			SrcData += SrcRowPitch * NumRows;
		}
		else
		{
			for (uint32 Row = 0; Row < NumRows; ++Row)
			{
				FMemory::Memcpy(DstData, SrcData, SrcRowPitch);

				SrcData += SrcRowPitch;
				DstData += DstRowPitch;
			}
		}
	}

	check(SrcData == SourceDataStart + BulkData->GetResourceBulkDataSize());
}

// GetCopyableFootprints gives us offsets in reference to the start of the subresource we ask for, so we have to ask for ALL subresources before that one so that we can get the actual offset to the subresource
D3D12_PLACED_SUBRESOURCE_FOOTPRINT UE::D3D12RHI::TextureUtils::GetInitializerSubresourceFootprint(FD3D12Texture* Texture, uint32 Subresource, uint32& NumRows)
{
	FD3D12Device* Device = Texture->GetParentDevice();
	const FD3D12ResourceDesc& ResourceDesc = Texture->GetResource()->GetDesc();

	const uint32 MaxSubresources = CalculateSubresourceCount(Texture->GetDesc());
	const uint32 SubresourceIndex = Subresource < MaxSubresources ? Subresource : (MaxSubresources - 1);

	const uint32 NumSubresources = SubresourceIndex + 1;

	const size_t FootprintsMemorySize = NumSubresources * (sizeof(D3D12_PLACED_SUBRESOURCE_FOOTPRINT) + sizeof(UINT));
	const bool bAllocateOnStack = (FootprintsMemorySize < 4096);
	void* const FootprintsMemory = bAllocateOnStack ? FMemory_Alloca(FootprintsMemorySize) : FMemory::Malloc(FootprintsMemorySize);
	ON_SCOPE_EXIT
	{
		if (!bAllocateOnStack)
		{
			FMemory::Free(FootprintsMemory);
		}
	};

	CA_ASSUME(FootprintsMemory != nullptr);

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT* const Footprints = reinterpret_cast<D3D12_PLACED_SUBRESOURCE_FOOTPRINT*>(FootprintsMemory);
	UINT* const Rows = reinterpret_cast<UINT*>(Footprints + NumSubresources);

	Device->GetDevice()->GetCopyableFootprints(&ResourceDesc, 0, NumSubresources, 0, Footprints, Rows, nullptr, nullptr);

	NumRows = Rows[SubresourceIndex];
	return Footprints[SubresourceIndex];
}

static UE::RHICore::FBaseTextureInitializerImplementation CreateTextureInitializerForWriting(FRHICommandListBase& RHICmdList, const FD3D12DynamicRHI::FCreateTextureInternalResult& CreateResult)
{
	FD3D12Texture* Texture = CreateResult.Texture;

	const uint64 TextureSize = UE::D3D12RHI::TextureUtils::CalculateResourceSize(Texture);

	FD3D12ResourceLocation UploadLocation(Texture->GetParentDevice());
	Texture->GetParentDevice()->GetDefaultFastAllocator().Allocate(TextureSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, &UploadLocation);

	void* WritableData = UploadLocation.GetMappedBaseAddress();

	return UE::RHICore::FBaseTextureInitializerImplementation(RHICmdList, Texture, WritableData, TextureSize,
		[Texture = TRefCountPtr<FD3D12Texture>(Texture), CreateD3D12Access = CreateResult.CreateD3D12Access, DesiredD3D12Access = CreateResult.DesiredD3D12Access, UploadLocation = MoveTemp(UploadLocation)](FRHICommandListBase& RHICmdList) mutable
		{
			UE::D3D12RHI::TextureUtils::ReconcileInitialState(RHICmdList, Texture, CreateD3D12Access, ED3D12Access::CopyDest, false);

			Texture->UploadInitialData(RHICmdList, MoveTemp(UploadLocation), DesiredD3D12Access);
			return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
		},
		[Texture = TRefCountPtr<FD3D12Texture>(Texture), WritableData](FRHITextureInitializer::FSubresourceIndex SubresourceIndex)
		{
			const FD3D12ResourceDesc& ResourceDesc = Texture->GetResource()->GetDesc();

			const uint32 ArrayOffset = UE::RHICore::GetCombinedArrayIndex(Texture->GetDesc(), SubresourceIndex.FaceIndex, SubresourceIndex.ArrayIndex);
			const uint32 Subresource = CalcSubresource(SubresourceIndex.MipIndex, ArrayOffset, ResourceDesc.MipLevels);

			uint32 NumRows = 0;
			const D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedFootprint = UE::D3D12RHI::TextureUtils::GetInitializerSubresourceFootprint(Texture, Subresource, NumRows);

			FRHITextureSubresourceInitializer Result{};

			Result.Data = reinterpret_cast<uint8*>(WritableData) + PlacedFootprint.Offset;
			Result.Stride = PlacedFootprint.Footprint.RowPitch;
			Result.Size = PlacedFootprint.Footprint.RowPitch * NumRows * PlacedFootprint.Footprint.Depth;

			return Result;
		}
	);
}

FRHITextureInitializer FD3D12DynamicRHI::RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::RHICreateTextureInitializer);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	FCreateTextureInternalResult CreateResult = CreateTextureInternal(CreateDesc, nullptr);

	if (CreateDesc.InitAction == ERHITextureInitAction::Default)
	{
		// Just return the texture with its default contents
		return UE::RHICore::FBaseTextureInitializerImplementation(RHICmdList, CreateResult.Texture,
			[Texture = TRefCountPtr<FD3D12Texture>(CreateResult.Texture), CreateD3D12Access = CreateResult.CreateD3D12Access, DesiredD3D12Access = CreateResult.DesiredD3D12Access, bNeedsDiscard = CreateResult.bNeedsDiscard](FRHICommandListBase& RHICmdList) mutable
			{
				UE::D3D12RHI::TextureUtils::ReconcileInitialState(RHICmdList, Texture, CreateD3D12Access, DesiredD3D12Access, bNeedsDiscard);
				return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
			}
		);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
	{
		check(CreateDesc.BulkData);

		UE::RHICore::FBaseTextureInitializerImplementation Initializer(CreateTextureInitializerForWriting(RHICmdList, CreateResult));

		UE::D3D12RHI::TextureUtils::CopyBulkData(Initializer.GetWritableData(), Initializer.GetWritableSize(), CreateResult.Texture, CreateDesc.BulkData);

		// Discard the bulk data's contents.
		CreateDesc.BulkData->Discard();

		return MoveTemp(Initializer);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
	{
		return CreateTextureInitializerForWriting(RHICmdList, CreateResult);
	}

	return UE::RHICore::HandleUnknownTextureInitializerInitAction(RHICmdList, CreateDesc);
}

class FWaitInitialMipDataUploadTask
{
public:
	TRefCountPtr<FD3D12Texture> Texture;
	FD3D12ResourceLocation TempResourceLocation;
	FD3D12ResourceLocation TempResourceLocationLowMips;

	FWaitInitialMipDataUploadTask(
		FD3D12Texture* InTexture,
		FD3D12ResourceLocation&& InTempResourceLocation,
		FD3D12ResourceLocation&& InTempResourceLocationLowMips)
		: Texture(InTexture)
		, TempResourceLocation(MoveTemp(InTempResourceLocation))
		, TempResourceLocationLowMips(MoveTemp(InTempResourceLocationLowMips))
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		for (FD3D12Texture& CurrentTexture : *Texture)
		{
			// Initial data upload is done
			CurrentTexture.ResourceLocation.UnlockPoolData();
		}
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyNormalThreadNormalTask;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGatherRequestsTask, STATGROUP_D3D12RHI);
	}
};

FTextureRHIRef FD3D12DynamicRHI::RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{	
	check(GRHISupportsAsyncTextureCreation);
	const ETextureCreateFlags InvalidFlags = TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilTargetable | TexCreate_UAV | TexCreate_Presentable | TexCreate_CPUReadback;
	check(!EnumHasAnyFlags(Flags, InvalidFlags));

	FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create2D(DebugName)
		.SetExtent(FIntPoint(SizeX, SizeY))
		.SetFormat((EPixelFormat)Format)
		.SetFlags(Flags) 
		.SetNumMips(NumMips)
		.SetInitialState(ERHIAccess::SRVMask);

	const DXGI_FORMAT PlatformResourceFormat = (DXGI_FORMAT)GPixelFormats[Format].PlatformFormat;
	const DXGI_FORMAT PlatformShaderResourceFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, EnumHasAnyFlags(Flags, TexCreate_SRGB));
	const D3D12_RESOURCE_DESC TextureDesc = CD3DX12_RESOURCE_DESC::Tex2D(
		PlatformResourceFormat,
		SizeX,
		SizeY,
		1,
		NumMips,
		1,  // Sample count
		0);  // Sample quality

	D3D12_SUBRESOURCE_DATA SubResourceData[MAX_TEXTURE_MIP_COUNT] = { };
	for (uint32 MipIndex = 0; MipIndex < NumInitialMips; ++MipIndex)
	{
		uint64 RowPitch = 0;
		uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(CreateDesc, MipIndex, RowPitch);

		SubResourceData[MipIndex].pData = InitialMipData[MipIndex];
		SubResourceData[MipIndex].RowPitch = RowPitch;
		SubResourceData[MipIndex].SlicePitch = MipSize;
	}

	void* TempBuffer = ZeroBuffer;
	uint64 TempBufferSize = ZeroBufferSize;
	for (uint32 MipIndex = NumInitialMips; MipIndex < NumMips; ++MipIndex)
	{
		uint64 RowPitch = 0;
		uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(CreateDesc, MipIndex, RowPitch);

		if (MipSize > TempBufferSize)
		{
			UE_LOGF(LogD3D12RHI, Verbose, "Temp texture streaming buffer not large enough, needed %llu bytes", MipSize);
			check(TempBufferSize == ZeroBufferSize);
			TempBufferSize = MipSize;
			TempBuffer = FMemory::Malloc(TempBufferSize);
			FMemory::Memzero(TempBuffer, TempBufferSize);
		}

		SubResourceData[MipIndex].pData = TempBuffer;
		SubResourceData[MipIndex].RowPitch = RowPitch;
		SubResourceData[MipIndex].SlicePitch = MipSize;
	}

#if !PLATFORM_WINDOWS
	FRWScopeLock ReadLock(*RHIGetSuspendedLock(), SLT_ReadOnly);
#endif

	FD3D12Adapter* Adapter = &GetAdapter();
	FD3D12Texture* TextureOut = Adapter->CreateLinkedObject<FD3D12Texture>(FRHIGPUMask::All(), [&](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
	{
		FD3D12Texture* NewTexture = CreateNewD3D12Texture(CreateDesc, Device);

		SafeCreateTexture2D(Device,
			Adapter,
			TextureDesc,
			nullptr,
			&NewTexture->ResourceLocation,
			NewTexture,
			(EPixelFormat)Format,
			Flags,
			// All resources used in a COPY command list must begin in the COMMON state. 
			ED3D12Access::Common,
			ED3D12Access::SRVMask,
			nullptr);

		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Format = PlatformShaderResourceFormat;
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2D;
		SRVDesc.Texture2D.MostDetailedMip = 0;
		SRVDesc.Texture2D.MipLevels = NumMips;
		SRVDesc.Texture2D.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, SRVDesc.Format);

		// Create a wrapper for the SRV and set it on the texture
		NewTexture->EmplaceSRV(SRVDesc, FirstLinkedObject);

		return NewTexture;
	});
	FTextureRHIRef TextureOutRHI = TextureOut;

	FGraphEventArray CopyCompleteEvents;
	OutCompletionEvent = nullptr;

	if (TextureOut)
	{
		// SubResourceData is only used in async texture creation (RHIAsyncCreateTexture2D). We need to manually transition the resource to
		// its 'default state', which is what the rest of the RHI (including InitializeTexture2DData) expects for SRV-only resources.

		check((TextureDesc.Flags & D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE) == 0);

		FD3D12Device* Device = TextureOut->GetParentDevice();
		FD3D12UploadHeapAllocator& UploadHeapAllocator = Adapter->GetUploadHeapAllocator(Device->GetGPUIndex());
		uint64 Size = GetRequiredIntermediateSize(TextureOut->GetResource()->GetResource(), 0, NumMips);
		uint64 SizeLowMips = 0;

		FD3D12ResourceLocation TempResourceLocation(Device);
		FD3D12ResourceLocation TempResourceLocationLowMips(Device);

		// The allocator work in pages of 4MB. Increasing page size is undesirable from a hitching point of view because there's a performance cliff above 4MB
		// where creation time of new pages can increase by an order of magnitude. Most allocations are smaller than 4MB, but a common exception is
		// 2048x2048 BC3 textures with mips, which takes 5.33MB. To avoid this case falling into the standalone allocations fallback path and risking hitching badly,
		// we split the top mip into a separate allocation, allowing it to fit within 4MB.
		const bool bSplitAllocation = (Size > 4 * 1024 * 1024) && (NumMips > 1);

		// Data used for split allocation - Workaround for GetCopyableFootprints returning unexpected values, see UE-173385
		ID3D12Device* D3D12Device = TextureOut->GetParentDevice()->GetDevice();
		D3D12_PLACED_SUBRESOURCE_FOOTPRINT Layouts[MAX_TEXTURE_MIP_COUNT] = { };
		UINT NumRows[MAX_TEXTURE_MIP_COUNT] = { };
		UINT64 RowSizesInBytes[MAX_TEXTURE_MIP_COUNT] = { };
		UINT64 TotalBytes = 0;
		uint64 SizeMip0 = 0;
		if (bSplitAllocation)
		{
			// Setup for the copies: we get the fullmip chain here to get the offsets first
			const uint64 FirstSubresource = 0;
			D3D12Device->GetCopyableFootprints(&TextureDesc, FirstSubresource, NumMips, 0, Layouts, NumRows, RowSizesInBytes, &TotalBytes);
			
			// Mip 0
			SizeMip0 = Layouts[1].Offset;
			UploadHeapAllocator.AllocUploadResource(SizeMip0, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, TempResourceLocation);
			Layouts[0].Offset = TempResourceLocation.GetOffsetFromBaseOfResource();

			// Remaining mip chain
			SizeLowMips = TotalBytes - SizeMip0;
			UploadHeapAllocator.AllocUploadResource(SizeLowMips, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, TempResourceLocationLowMips);

			const uint64 LowMipsTotalBufferSize = TempResourceLocationLowMips.GetResource()->GetDesc().Width;
			
			const uint64 BaseOffset = Layouts[1].Offset;

			for (uint64 MipIndex = 1; MipIndex < NumMips; ++MipIndex)
			{
				check(Layouts[MipIndex].Offset >= BaseOffset);

				const uint64 RelativeMipCopyOffset = Layouts[MipIndex].Offset - BaseOffset; // Offset relative to mip1

				// The original offsets for the remaining mipchain were originally computed with mip0, so we need to remove that offset
				Layouts[MipIndex].Offset -= BaseOffset;
				// The intermediate resource we get might be already used, so we need to account for the offset within this resource
				Layouts[MipIndex].Offset += TempResourceLocationLowMips.GetOffsetFromBaseOfResource();

				// UpdateSubresources copies mip levels taking into account RowPitch (number of bytes between rows) and RowSize (number of valid texture data bytes).
				// For each row, the destination address is computed as RowIndex*RowPitch and the copy size is always RowSize.
				// If RowSize is smaller than RowPitch, the remaining bytes in the copy destination buffer are not touched.
				// See MemcpySubresource() in d3dx12_resource_helpers.h
				check(NumRows[MipIndex] != 0);
				const uint64 MipCopySize = Layouts[MipIndex].Footprint.RowPitch * (NumRows[MipIndex] - 1) + RowSizesInBytes[MipIndex];

				// Make sure that the buffer is large enough before proceeding.

				const uint64 RelativeMipCopyEndOffset = RelativeMipCopyOffset + MipCopySize;
				checkf(RelativeMipCopyEndOffset <= SizeLowMips,
					TEXT("Mip tail upload buffer allocation is too small for mip %llu. RelativeMipCopyOffset=%llu, MipCopySize=%llu, RelativeMipCopyEndOffset=%llu, SizeLowMips=%llu."),
					MipIndex, RelativeMipCopyOffset, MipCopySize, RelativeMipCopyEndOffset, SizeLowMips);

				const uint64 AbsoluteMipCopyEndOffset = Layouts[MipIndex].Offset + MipCopySize;
				checkf(AbsoluteMipCopyEndOffset <= LowMipsTotalBufferSize,
					TEXT("Mip tail upload buffer total size is too small for mip %llu. Layouts[MipIndex].Offset=%llu, MipCopySize=%llu, AbsoluteMipCopyEndOffset=%llu, LowMipsTotalBufferSize=%llu."),
					MipIndex, Layouts[MipIndex].Offset, MipCopySize, AbsoluteMipCopyEndOffset, LowMipsTotalBufferSize);
			}
		}
		else
		{
			UploadHeapAllocator.AllocUploadResource(Size, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, TempResourceLocation);
		}

		for (FD3D12Texture& CurrentTexture : *TextureOut)
		{
			// Need to get device from GPU specific copy of the texture
			Device = CurrentTexture.GetParentDevice();

			FD3D12Resource* Resource = CurrentTexture.GetResource();

			FD3D12SyncPointRef SyncPoint;
			{
				FD3D12CopyScope CopyScope(Device, ED3D12SyncPointType::GPUAndCPU);
				SyncPoint = CopyScope.GetSyncPoint();

				CopyCompleteEvents.Add(SyncPoint->GetGraphEvent());

				// NB: Do not increment NumCopies because that will count as work on the direct
				// queue, not the copy queue, possibly causing it to flush prematurely. We are
				// explicitly submitting the copy command list so there's no need to increment any
				// work counters.

				if (bSplitAllocation)
				{
					UINT64 SizeCopiedMip0 = UpdateSubresources(
						CopyScope.Context.CopyCommandList().Get(),
						Resource->GetResource(),
						TempResourceLocation.GetResource()->GetResource(),
						0, // FirstSubresource
						1, // NumSubresources
						SizeMip0, // RequiredSize
						Layouts,
						NumRows,
						RowSizesInBytes,
						SubResourceData);

					ensure(SizeCopiedMip0 == SizeMip0);

					UINT64 SizeCopiedLowMips = UpdateSubresources(
						CopyScope.Context.CopyCommandList().Get(),
						Resource->GetResource(),
						TempResourceLocationLowMips.GetResource()->GetResource(),
						1, // FirstSubresource
						NumMips-1, // NumSubresources
						SizeLowMips, // RequiredSize
						Layouts+1,
						NumRows+1,
						RowSizesInBytes+1,
						SubResourceData+1);

					ensure(SizeCopiedLowMips == SizeLowMips);

				}
				else
				{
					UpdateSubresources(
						CopyScope.Context.CopyCommandList().Get(),
						Resource->GetResource(),
						TempResourceLocation.GetResource()->GetResource(),
						TempResourceLocation.GetOffsetFromBaseOfResource(),
						0, NumMips,
						SubResourceData);
				}

				CopyScope.Context.UpdateResidency(Resource);
				CopyScope.Context.UpdateResidency(TempResourceLocation.GetResource());

				if (bSplitAllocation)
				{
					CopyScope.Context.UpdateResidency(TempResourceLocationLowMips.GetResource());
				}
			}
		}

		FD3D12TextureStats::D3D12TextureAllocated(*TextureOut);

		check(CopyCompleteEvents.Num() > 0);

		OutCompletionEvent = TGraphTask<FWaitInitialMipDataUploadTask>::CreateTask(&CopyCompleteEvents).ConstructAndDispatchWhenReady(
			TextureOut,
			MoveTemp(TempResourceLocation),
			MoveTemp(TempResourceLocationLowMips));
	}

	if (TempBufferSize != ZeroBufferSize)
	{
		FMemory::Free(TempBuffer);
	}

	return MoveTemp(TextureOutRHI);
}


/**
 * Computes the size in memory required by a given texture.
 *
 * @param	TextureRHI		- Texture we want to know the size of
 * @return					- Size in Bytes
 */
uint32 FD3D12DynamicRHI::RHIComputeMemorySize(FRHITexture* TextureRHI)
{
	if (!TextureRHI)
	{
		return 0;
	}

	FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureRHI);
	return Texture->ResourceLocation.GetSize();
}

struct FSubresourceIndices
{
	uint32 MipIndex;
	uint32 SliceIndex;
	uint32 PlaneIndex;
};

static FSubresourceIndices GetSubresourceIndices(const D3D12_RESOURCE_DESC& Desc, uint32 Index)
{
	const uint32 MipCount   = Desc.MipLevels;
	const uint32 SliceCount = Desc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D ? 1 : Desc.DepthOrArraySize;

	return FSubresourceIndices {
		.MipIndex = Index % MipCount,
		.SliceIndex = (Index / MipCount) % SliceCount,
		.PlaneIndex = Index / (MipCount * SliceCount)
	};
}

static D3D12_BOX GetSubresourceCopyBox(const D3D12_RESOURCE_DESC& Desc, const D3D12_TEXTURE_COPY_LOCATION* OtherLocation, const FSubresourceIndices& Indices, const FPixelFormatInfo& PixelInfo)
{
	// Dimensions may be derived from the counter parts footprint
	if (OtherLocation->Type == D3D12_TEXTURE_COPY_TYPE_PLACED_FOOTPRINT)
	{
		return {
			.left = 0, .top = 0, .front = 0,
			.right = OtherLocation->PlacedFootprint.Footprint.Width,
			.bottom = OtherLocation->PlacedFootprint.Footprint.Height,
			.back = OtherLocation->PlacedFootprint.Footprint.Depth
		};
	}

	// Otherwise just assume the entire (sub)resource
	return {
		.left = 0, .top = 0, .front = 0,
		.right = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, Desc.Width >> Indices.MipIndex), PixelInfo.BlockSizeX),
		.bottom = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, Desc.Height >> Indices.MipIndex), PixelInfo.BlockSizeY),
		.back = AlignArbitrary<uint32>(Desc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D ? FMath::Max<uint32>(1u, Desc.DepthOrArraySize >> Indices.MipIndex) : 1, PixelInfo.BlockSizeZ)
	};
}

static D3D12_BOX GetBoxAdjustedForPixelFormat(const D3D12_BOX& SourceBox, const FPixelFormatInfo& SourcePixelFormatInfo, const FPixelFormatInfo& DestPixelFormatInfo)
{
	return {
		.left = SourceBox.left * DestPixelFormatInfo.BlockSizeX / SourcePixelFormatInfo.BlockSizeX,
		.top = SourceBox.top * DestPixelFormatInfo.BlockSizeY / SourcePixelFormatInfo.BlockSizeY,
		.front = SourceBox.front,
		.right = SourceBox.right * DestPixelFormatInfo.BlockSizeX / SourcePixelFormatInfo.BlockSizeX,
		.bottom = SourceBox.bottom * DestPixelFormatInfo.BlockSizeY / SourcePixelFormatInfo.BlockSizeY,
		.back = SourceBox.back
	};
}

static uint64 GetPlacedBufferRequiredSize(FD3D12CommandContext& Context, const D3D12_SUBRESOURCE_FOOTPRINT& Footprint, const FPixelFormatInfo& Format, const FName& DebugName)
{
	RHI_BREADCRUMB_CHECK_SHIPPINGF(Context, (Footprint.Width % Format.BlockSizeX) == 0, TEXT("Width not aligned to block size for: '%s'"), *DebugName.ToString());
	RHI_BREADCRUMB_CHECK_SHIPPINGF(Context, (Footprint.Height % Format.BlockSizeY) == 0, TEXT("Height not aligned to block size for: '%s'"), *DebugName.ToString());
	
	const uint32 NumColumns = FMath::DivideAndRoundUp(Footprint.Width, static_cast<uint32>(Format.BlockSizeX));
	const uint32 NumRows    = FMath::DivideAndRoundUp(Footprint.Height, static_cast<uint32>(Format.BlockSizeY));

	// The last row doesn't need the full row pitch, all that matters is that the texel/block starting address is aligned to it
	const uint32 SubresourceSizeAligned   = Footprint.RowPitch * (NumRows - 1) * Footprint.Depth;
	const uint32 SubresourceSizeUnaligned = Format.BlockBytes * NumColumns * Footprint.Depth;
	
	return SubresourceSizeAligned + SubresourceSizeUnaligned;
}

FString GetTextureCopyRegionString(const FSubresourceIndices& Indices, const D3D12_BOX& CopyBox, uint32 SubresourceWidth, uint32 SubresourceHeight, uint32 SubresourceDepth)
{
	return FString::Printf(
		TEXT("Subresource (Mip: %u, Slice: %u, Plane: %u) with dimensions [%u, %u, %u], copying from [%u, %u, %u] to [%u, %u, %u]"),
		Indices.MipIndex, Indices.SliceIndex, Indices.PlaneIndex,
		SubresourceWidth, SubresourceHeight, SubresourceDepth,
		CopyBox.left, CopyBox.top, CopyBox.front,
		CopyBox.right, CopyBox.bottom, CopyBox.back
	);
}

FORCENOINLINE void FD3D12CommandContext::CopyTextureRegionChecked(
	const D3D12_TEXTURE_COPY_LOCATION* DestCopyLocation, int DestX, int DestY, int DestZ, EPixelFormat DestPixelFormat,
	const D3D12_TEXTURE_COPY_LOCATION* SourceCopyLocation, const D3D12_BOX* SourceBox, EPixelFormat SourcePixelFormat,
	const FName& DestName, const FName& SourceName
)
{
#if ENABLE_COPY_TEXTURE_REGION_CHECK
	const D3D12_RESOURCE_DESC DestDesc   = DestCopyLocation->pResource->GetDesc();
	const D3D12_RESOURCE_DESC SourceDesc = SourceCopyLocation->pResource->GetDesc();

	const FPixelFormatInfo& DestPixelFormatInfo = GPixelFormats[DestPixelFormat];
	const FPixelFormatInfo& SourcePixelFormatInfo = GPixelFormats[SourcePixelFormat];

#define LAZY_COPY_DECORATOR *GetTextureCopyRegionString(Indices, CopyBox, SubresourceWidth, SubresourceHeight, SubresourceDepth)

	switch (SourceCopyLocation->Type)
	{
		// Copy from texture
		case D3D12_TEXTURE_COPY_TYPE_SUBRESOURCE_INDEX:
		{
			const FSubresourceIndices Indices = GetSubresourceIndices(SourceDesc, SourceCopyLocation->SubresourceIndex);

			const D3D12_BOX CopyBox = SourceBox ? *SourceBox : GetSubresourceCopyBox(SourceDesc, DestCopyLocation, Indices, SourcePixelFormatInfo);

			const uint32 SubresourceWidth  = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, SourceDesc.Width >> Indices.MipIndex), SourcePixelFormatInfo.BlockSizeX);
			const uint32 SubresourceHeight = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, SourceDesc.Height >> Indices.MipIndex), SourcePixelFormatInfo.BlockSizeY);
			const uint32 SubresourceDepth  = AlignArbitrary<uint32>(SourceDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D ? FMath::Max<uint32>(1u, SourceDesc.DepthOrArraySize >> Indices.MipIndex) : 1, SourcePixelFormatInfo.BlockSizeZ);

			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, (CopyBox.left % SourcePixelFormatInfo.BlockSizeX) == 0 && (CopyBox.right % SourcePixelFormatInfo.BlockSizeX) == 0, TEXT("Width not aligned to block size for: '%s' -> '%s', %s"), *SourceName.ToString(), *DestName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, (CopyBox.top % SourcePixelFormatInfo.BlockSizeY) == 0 && (CopyBox.bottom % SourcePixelFormatInfo.BlockSizeY) == 0, TEXT("Height not aligned to block size for: '%s' -> '%s', %s"), *SourceName.ToString(), *DestName.ToString(), LAZY_COPY_DECORATOR);
				
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, CopyBox.left <= CopyBox.right && CopyBox.right <= SubresourceWidth, TEXT("Source width out of bounds for: '%s' -> '%s', %s"), *SourceName.ToString(), *DestName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, CopyBox.top <= CopyBox.bottom && CopyBox.bottom <= SubresourceHeight, TEXT("Source height out of bounds for: '%s' -> '%s', %s"), *SourceName.ToString(), *DestName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, CopyBox.front <= CopyBox.back && CopyBox.back <= SubresourceDepth, TEXT("Source depth out of bounds for:  '%s' -> '%s', %s"), *SourceName.ToString(), *DestName.ToString(), LAZY_COPY_DECORATOR);
			break;
		}

		// Copy from buffer
		case D3D12_TEXTURE_COPY_TYPE_PLACED_FOOTPRINT:
		{
			const uint32 RequiredPlacedSize = GetPlacedBufferRequiredSize(*this, SourceCopyLocation->PlacedFootprint.Footprint, SourcePixelFormatInfo, SourceName);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, SourceCopyLocation->PlacedFootprint.Offset + RequiredPlacedSize <= SourceDesc.Width, TEXT("Source placed buffer width out of bounds for: '%s' -> '%s'"), *SourceName.ToString(), *DestName.ToString());
			break;
		}
	}

	switch (DestCopyLocation->Type)
	{
		// Copy to texture
		case D3D12_TEXTURE_COPY_TYPE_SUBRESOURCE_INDEX:
		{
			const FSubresourceIndices Indices = GetSubresourceIndices(DestDesc, DestCopyLocation->SubresourceIndex);
			const D3D12_BOX CopyBox = SourceBox ? GetBoxAdjustedForPixelFormat(*SourceBox, SourcePixelFormatInfo, DestPixelFormatInfo) : GetSubresourceCopyBox(DestDesc, SourceCopyLocation, Indices, DestPixelFormatInfo);
				
			const uint32 SubresourceWidth  = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, DestDesc.Width >> Indices.MipIndex), DestPixelFormatInfo.BlockSizeX);
			const uint32 SubresourceHeight = AlignArbitrary<uint32>(FMath::Max<uint32>(1u, DestDesc.Height >> Indices.MipIndex), DestPixelFormatInfo.BlockSizeY);
			const uint32 SubresourceDepth  = AlignArbitrary<uint32>(DestDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE3D ? FMath::Max<uint32>(1u, DestDesc.DepthOrArraySize >> Indices.MipIndex) : 1, DestPixelFormatInfo.BlockSizeZ);

			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestX % DestPixelFormatInfo.BlockSizeX == 0 && (CopyBox.right - CopyBox.left) % DestPixelFormatInfo.BlockSizeX == 0, TEXT("Width not aligned to block size for: '%s' <- '%s', %s"), *DestName.ToString(), *SourceName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestY % DestPixelFormatInfo.BlockSizeY == 0 && (CopyBox.bottom - CopyBox.top) % DestPixelFormatInfo.BlockSizeY == 0, TEXT("Height not aligned to block size for: '%s' <- '%s', %s"), *DestName.ToString(), *SourceName.ToString(), LAZY_COPY_DECORATOR);
				
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestX + (CopyBox.right - CopyBox.left) <= SubresourceWidth, TEXT("Dest width out of bounds for: '%s' <- '%s', %s"), *DestName.ToString(), *SourceName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestY + (CopyBox.bottom - CopyBox.top) <= SubresourceHeight, TEXT("Dest height out of bounds for: '%s' <- '%s', %s"), *DestName.ToString(), *SourceName.ToString(), LAZY_COPY_DECORATOR);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestZ + (CopyBox.back - CopyBox.front) <= SubresourceDepth, TEXT("Dest depth out of bounds for: '%s' <- '%s', %s"), *DestName.ToString(), *SourceName.ToString(), LAZY_COPY_DECORATOR);
			break;
		}

		// Copy to buffer
		case D3D12_TEXTURE_COPY_TYPE_PLACED_FOOTPRINT:
		{
			const uint32 RequiredPlacedSize = GetPlacedBufferRequiredSize(*this, DestCopyLocation->PlacedFootprint.Footprint, DestPixelFormatInfo, DestName);
			RHI_BREADCRUMB_CHECK_SHIPPINGF(*this, DestCopyLocation->PlacedFootprint.Offset + RequiredPlacedSize <= DestDesc.Width, TEXT("Dest placed buffer width out of bounds for: '%s' <- '%s'"), *DestName.ToString(), *SourceName.ToString());
			break;
		}
	}

#undef LAZY_COPY_DECORATOR
#endif // ENABLE_COPY_TEXTURE_REGION_CHECK

	// Just pass down callchain
	GraphicsCommandList()->CopyTextureRegion(
		DestCopyLocation,
		DestX, DestY, DestZ,
		SourceCopyLocation,
		SourceBox
	);
}

///////////////////////////////////////////////////////////////////////////////////////////
// FD3D12Texture
///////////////////////////////////////////////////////////////////////////////////////////

FD3D12Texture::~FD3D12Texture()
{
	if (IsHeadLink())
	{
		// Only call this once for a LDA chain
		FD3D12TextureStats::D3D12TextureDeleted(*this);
	}
}

#if RHI_ENABLE_RESOURCE_INFO
bool FD3D12Texture::GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const
{
	OutResourceInfo = FRHIResourceInfo{};
	OutResourceInfo.Name = GetName();
	OutResourceInfo.Type = GetType();
	OutResourceInfo.VRamAllocation.AllocationSize = ResourceLocation.GetSize();
	OutResourceInfo.IsTransient = ResourceLocation.IsTransient();
#if ENABLE_RESIDENCY_MANAGEMENT
	OutResourceInfo.bResident = GetResource() && GetResource()->IsResident();
#endif

	return true;
}
#endif

void* FD3D12Texture::GetNativeResource() const
{
	void* NativeResource = nullptr;
	FD3D12Resource* Resource = GetResource();
	if (Resource)
	{
		NativeResource = Resource->GetResource();
	}
	if (!NativeResource)
	{
		FD3D12Texture* Base = GetD3D12TextureFromRHITexture((FRHITexture*)this);
		if (Base)
		{
			Resource = Base->GetResource();
			if (Resource)
			{
				NativeResource = Resource->GetResource();
			}
		}
	}
	return NativeResource;
}

FRHIDescriptorHandle FD3D12Texture::GetDefaultBindlessHandle() const
{
	if (FD3D12ShaderResourceView* View = GetShaderResourceView())
	{
		return View->GetBindlessHandle();
	}
	return FRHIDescriptorHandle();
}


void FD3D12Texture::CreateViews(FD3D12Texture* FirstLinkedObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(D3D12RHI::CreateViews);

	D3D12_RESOURCE_DESC ResourceDesc = ResourceLocation.GetResource()->GetDesc();
	const FRHITextureDesc Desc = GetDesc();

	const bool bSRGB = EnumHasAnyFlags(Desc.Flags, TexCreate_SRGB);
	const DXGI_FORMAT PlatformResourceFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat((DXGI_FORMAT)GPixelFormats[Desc.Format].PlatformFormat, Desc.Flags);
	const DXGI_FORMAT PlatformShaderResourceFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);
	const DXGI_FORMAT PlatformRenderTargetFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);
	const DXGI_FORMAT PlatformDepthStencilFormat = UE::DXGIUtilities::FindDepthStencilFormat(PlatformResourceFormat);

	const bool bTexture2D = Desc.IsTexture2D();
	const bool bTexture3D = Desc.IsTexture3D();
	const bool bCubeTexture = Desc.IsTextureCube();
	const bool bTextureArray = Desc.IsTextureArray();

	// Set up the texture bind flags.
	bool bCreateRTV = EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET);
	bool bCreateDSV = EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL);
	bool bCreateShaderResource = !EnumHasAnyFlags(ResourceDesc.Flags, D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE);

	if (EnumHasAllFlags(Desc.Flags, TexCreate_CPUReadback))
	{
		check(!EnumHasAnyFlags(Desc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ShaderResource));
		bCreateShaderResource = false;
	}

	if (EnumHasAnyFlags(Desc.Flags, TexCreate_DisableSRVCreation))
	{
		bCreateShaderResource = false;
	}

	if (Desc.Format == PF_NV12 ||
		Desc.Format == PF_P010)
	{
		bCreateRTV = false;
		bCreateShaderResource = false;
	}

	const bool bIsMultisampled = ResourceDesc.SampleDesc.Count > 1;

	FD3D12Device* Device = GetParentDevice();

	if (bCreateRTV)
	{
		if (bTexture3D)
		{
			// Create a single render-target-view for the texture.
			D3D12_RENDER_TARGET_VIEW_DESC RTVDesc;
			FMemory::Memzero(&RTVDesc, sizeof(RTVDesc));
			RTVDesc.Format = PlatformRenderTargetFormat;
			RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE3D;
			RTVDesc.Texture3D.MipSlice = 0;
			RTVDesc.Texture3D.FirstWSlice = 0;
			RTVDesc.Texture3D.WSize = Desc.Depth;

			SetNumRTVs(1);
			EmplaceRTV(RTVDesc, 0, FirstLinkedObject);
		}
		else
		{
			const bool bCreateRTVsPerSlice = EnumHasAnyFlags(Desc.Flags, TexCreate_TargetArraySlicesIndependently) && (bTextureArray || bCubeTexture);
			SetNumRTVs(bCreateRTVsPerSlice ? Desc.NumMips * ResourceDesc.DepthOrArraySize : Desc.NumMips);

			// Create a render target view for each mip
			uint32 RTVIndex = 0;
			for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; MipIndex++)
			{
				if (bCreateRTVsPerSlice)
				{
					SetCreatedRTVsPerSlice(true, ResourceDesc.DepthOrArraySize);

					for (uint32 SliceIndex = 0; SliceIndex < ResourceDesc.DepthOrArraySize; SliceIndex++)
					{
						D3D12_RENDER_TARGET_VIEW_DESC RTVDesc;
						FMemory::Memzero(RTVDesc);

						RTVDesc.Format = PlatformRenderTargetFormat;
						RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DARRAY;
						RTVDesc.Texture2DArray.FirstArraySlice = SliceIndex;
						RTVDesc.Texture2DArray.ArraySize = 1;
						RTVDesc.Texture2DArray.MipSlice = MipIndex;
						RTVDesc.Texture2DArray.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, RTVDesc.Format);

						EmplaceRTV(RTVDesc, RTVIndex++, FirstLinkedObject);
					}
				}
				else
				{
					D3D12_RENDER_TARGET_VIEW_DESC RTVDesc;
					FMemory::Memzero(RTVDesc);

					RTVDesc.Format = PlatformRenderTargetFormat;

					if (bTextureArray || bCubeTexture)
					{
						if (bIsMultisampled)
						{
							RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DMSARRAY;
							RTVDesc.Texture2DMSArray.FirstArraySlice = 0;
							RTVDesc.Texture2DMSArray.ArraySize = ResourceDesc.DepthOrArraySize;
						}
						else
						{
							RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DARRAY;
							RTVDesc.Texture2DArray.FirstArraySlice = 0;
							RTVDesc.Texture2DArray.ArraySize = ResourceDesc.DepthOrArraySize;
							RTVDesc.Texture2DArray.MipSlice = MipIndex;
							RTVDesc.Texture2DArray.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, RTVDesc.Format);
						}
					}
					else
					{
						if (bIsMultisampled)
						{
							RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DMS;
							// Nothing to set
						}
						else
						{
							RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2D;
							RTVDesc.Texture2D.MipSlice = MipIndex;
							RTVDesc.Texture2D.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, RTVDesc.Format);
						}
					}

					EmplaceRTV(RTVDesc, RTVIndex++, FirstLinkedObject);
				}
			}
		}
	}

	if (bCreateDSV)
	{
		check(!bTexture3D);

		// Create a depth-stencil-view for the texture.
		D3D12_DEPTH_STENCIL_VIEW_DESC DSVDesc = {};
		DSVDesc.Format = UE::DXGIUtilities::FindDepthStencilFormat(PlatformResourceFormat);
		if (bTextureArray || bCubeTexture)
		{
			if (bIsMultisampled)
			{
				DSVDesc.ViewDimension = D3D12_DSV_DIMENSION_TEXTURE2DMSARRAY;
				DSVDesc.Texture2DMSArray.FirstArraySlice = 0;
				DSVDesc.Texture2DMSArray.ArraySize = ResourceDesc.DepthOrArraySize;
			}
			else
			{
				DSVDesc.ViewDimension = D3D12_DSV_DIMENSION_TEXTURE2DARRAY;
				DSVDesc.Texture2DArray.FirstArraySlice = 0;
				DSVDesc.Texture2DArray.ArraySize = ResourceDesc.DepthOrArraySize;
				DSVDesc.Texture2DArray.MipSlice = 0;
			}
		}
		else
		{
			if (bIsMultisampled)
			{
				DSVDesc.ViewDimension = D3D12_DSV_DIMENSION_TEXTURE2DMS;
				// Nothing to set
			}
			else
			{
				DSVDesc.ViewDimension = D3D12_DSV_DIMENSION_TEXTURE2D;
				DSVDesc.Texture2D.MipSlice = 0;
			}
		}

		const bool HasStencil = UE::DXGIUtilities::HasStencilBits(DSVDesc.Format);
		for (uint32 AccessType = 0; AccessType < FExclusiveDepthStencil::MaxIndex; ++AccessType)
		{
			// Create a read-only access views for the texture.
			DSVDesc.Flags = (AccessType & FExclusiveDepthStencil::DepthRead_StencilWrite) ? D3D12_DSV_FLAG_READ_ONLY_DEPTH : D3D12_DSV_FLAG_NONE;
			if (HasStencil)
			{
				DSVDesc.Flags |= (AccessType & FExclusiveDepthStencil::DepthWrite_StencilRead) ? D3D12_DSV_FLAG_READ_ONLY_STENCIL : D3D12_DSV_FLAG_NONE;
			}

			EmplaceDSV(DSVDesc, AccessType, FirstLinkedObject);
		}
	}

	// Create a shader resource view for the texture.
	if (bCreateShaderResource)
	{
		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Format = PlatformShaderResourceFormat;

		if (bCubeTexture && bTextureArray)
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURECUBEARRAY;
			SRVDesc.TextureCubeArray.MostDetailedMip = 0;
			SRVDesc.TextureCubeArray.MipLevels = Desc.NumMips;
			SRVDesc.TextureCubeArray.First2DArrayFace = 0;
			SRVDesc.TextureCubeArray.NumCubes = Desc.ArraySize;
		}
		else if (bCubeTexture)
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURECUBE;
			SRVDesc.TextureCube.MostDetailedMip = 0;
			SRVDesc.TextureCube.MipLevels = Desc.NumMips;
		}
		else if (bTextureArray)
		{
			if (bIsMultisampled)
			{
				SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DMSARRAY;
				SRVDesc.Texture2DMSArray.FirstArraySlice = 0;
				SRVDesc.Texture2DMSArray.ArraySize = ResourceDesc.DepthOrArraySize;
			}
			else
			{
				SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DARRAY;
				SRVDesc.Texture2DArray.MostDetailedMip = 0;
				SRVDesc.Texture2DArray.MipLevels = Desc.NumMips;
				SRVDesc.Texture2DArray.FirstArraySlice = 0;
				SRVDesc.Texture2DArray.ArraySize = ResourceDesc.DepthOrArraySize;
				SRVDesc.Texture2DArray.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, SRVDesc.Format);
			}
		}
		else if (bTexture3D)
		{
			SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE3D;
			SRVDesc.Texture3D.MipLevels = Desc.NumMips;
			SRVDesc.Texture3D.MostDetailedMip = 0;
		}
		else
		{
			if (bIsMultisampled)
			{
				SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2DMS;
				// Nothing to set
			}
			else
			{
				SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2D;
				SRVDesc.Texture2D.MostDetailedMip = 0;
				SRVDesc.Texture2D.MipLevels = Desc.NumMips;
				SRVDesc.Texture2D.PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(PlatformResourceFormat, SRVDesc.Format);
			}
		}

		EmplaceSRV(SRVDesc, FirstLinkedObject);
	}
}


#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
void FD3D12Texture::AliasResources(FD3D12Texture* Texture)
{
	// Alias the location, will perform an addref underneath
	FD3D12ResourceLocation::Alias(ResourceLocation, Texture->ResourceLocation);

	ShaderResourceView = Texture->ShaderResourceView;

	for (uint32 Index = 0; Index < FExclusiveDepthStencil::MaxIndex; Index++)
	{
		DepthStencilViews[Index] = Texture->DepthStencilViews[Index];
	}

	bCreatedRTVsPerSlice = Texture->bCreatedRTVsPerSlice;
	RTVArraySizePerMip = Texture->RTVArraySizePerMip;
	RenderTargetViews.SetNum(Texture->RenderTargetViews.Num());
	for (int32 Index = 0; Index < Texture->RenderTargetViews.Num(); Index++)
	{
		RenderTargetViews[Index] = Texture->RenderTargetViews[Index];
	}
}
#endif // D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES

void FD3D12Texture::ReuseStagingBuffer(TUniquePtr<FD3D12LockedResource>&& LockedResource, uint32 Subresource)
{
	// If we get multiple updates in a single command list, there could already be a recycled element
	if (!LockedMap.Contains(Subresource))
	{
		LockedMap.Emplace(Subresource, MoveTemp(LockedResource));
	}
	else
	{
		// Move rvalue, so it gets destroyed
		TUniquePtr<FD3D12LockedResource> DiscardLockedResource(MoveTemp(LockedResource));
	}
}

FRHILockTextureResult FD3D12Texture::Lock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12LockTextureTime);

	const static FName RHITextureLockName("FRHITexture Lock");
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(GetName(), RHITextureLockName, GetOwnerName());

	FD3D12Device* Device = GetParentDevice();
	FD3D12Adapter* Adapter = Device->GetParentAdapter();

	const FRHITextureDesc& Desc = GetDesc();
	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Desc, Arguments);

	// Calculate the subresource index corresponding to the specified mip-map.
	const uint32 Subresource = CalcSubresource(Arguments.MipIndex, ArrayIndex, this->GetNumMips());

	const D3D12_RESOURCE_DESC& ResourceDesc = GetResource()->GetDesc();

	uint32 NumRows = 0;

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedFootprint;
	Device->GetDevice()->GetCopyableFootprints(&ResourceDesc, Subresource, 1, 0, &PlacedFootprint, &NumRows, nullptr, nullptr);

	FRHILockTextureResult Result;
	Result.Stride = PlacedFootprint.Footprint.RowPitch;
	
	const uint64 SubresourceSize = PlacedFootprint.Footprint.RowPitch * NumRows * PlacedFootprint.Footprint.Depth;
	Result.ByteCount = SubresourceSize;

	// With Dynamic set, entries in LockedMap are preserved after Unlock, and so may already exist in the map.  Check for an
	// existing entry, and pull the address from there if possible.
	if (Arguments.LockMode == RLM_WriteOnly && EnumHasAnyFlags(GetDesc().Flags, ETextureCreateFlags::Dynamic))
	{
		check(Arguments.LockMode == RLM_WriteOnly);
		TUniquePtr<FD3D12LockedResource>* ExistingLockedResource = LockedMap.Find(Subresource);
		if (ExistingLockedResource)
		{
			Result.Data = (*ExistingLockedResource)->ResourceLocation.GetMappedBaseAddress();
			return Result;
		}
	}

	check(LockedMap.Find(Subresource) == nullptr);
	TUniquePtr<FD3D12LockedResource> LockedResource = MakeUnique<FD3D12LockedResource>(Device);

	// GetCopyableFootprints returns the offset from the start of the resource to the specified subresource, but our staging buffer represents
	// only the selected subresource, so we need to reset the offset to 0.
	PlacedFootprint.Offset = 0;

	// Store the footprint information so we don't have to recompute it in Unlock.
	LockedResource->Footprint = PlacedFootprint.Footprint;

	if (FD3D12DynamicRHI::GetD3DRHI()->HandleSpecialLock(Result, this, Arguments))
	{
		// nothing left to do...
		check(Result.Data != nullptr);
	}
	else if (Arguments.LockMode == RLM_WriteOnly)
	{
		// If we're writing to the texture, allocate a system memory buffer to receive the new contents.
		// Use an upload heap to copy data to a default resource.

		const uint64 BufferSize = Align(SubresourceSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT);
		void* pData = Device->GetDefaultFastAllocator().Allocate(BufferSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, &LockedResource->ResourceLocation);
		if (nullptr == pData)
		{
			check(false);
			return Result;
		}

		Result.Data = LockedResource->ResourceLocation.GetMappedBaseAddress();
	}
	else
	{
		LockedResource->bLockedForReadOnly = true;

		//TODO: Make this work for multi-GPU (it's probably a very rare occurrence though)
		ensure(GNumExplicitGPUsForRendering == 1);

		// If we're reading from the texture, we create a staging resource, copy the texture contents to it, and map it.

		// Create the staging texture.
		FD3D12Resource* StagingTexture = nullptr;

		const FRHIGPUMask Node = Device->GetGPUMask();
		VERIFYD3D12RESULT(Adapter->CreateBuffer(D3D12_HEAP_TYPE_READBACK, Node, Node, SubresourceSize, &StagingTexture, nullptr));

		LockedResource->ResourceLocation.AsStandAlone(StagingTexture, SubresourceSize);

		CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(StagingTexture->GetResource(), PlacedFootprint);
		CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(GetResource()->GetResource(), Subresource);

		if (RHICmdList.NeedsExtraTransitions())
		{
			RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::Unknown, ERHIAccess::CopySrc, EResourceTransitionFlags::IgnoreAfterState, Arguments.MipIndex, 0, 0));
		}

		RHICmdList.EnqueueLambda([this, StagingTexture, DestCopyLocation, SourceCopyLocation](FRHICommandListBase& ExecutingCmdList)
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, 0);

			Context.FlushResourceBarriers();
			Context.CopyTextureRegionChecked(
				&DestCopyLocation,
				0, 0, 0,
				GetFormat(),
				&SourceCopyLocation,
				nullptr,
				GetFormat(),
				StagingTexture->GetName(),		// DestName
				GetName()						// SourceName
			);

			Context.UpdateResidency(StagingTexture);
			Context.UpdateResidency(GetResource());
		});

		if (RHICmdList.NeedsExtraTransitions())
		{		
			RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::CopySrc, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState, Arguments.MipIndex, 0 , 0));
		}

		// We need to execute the command list so we can read the data from the map below
		RHICmdList.GetAsImmediate().SubmitAndBlockUntilGPUIdle();

		Result.Data = LockedResource->ResourceLocation.GetMappedBaseAddress();
	}

	LockedMap.Emplace(Subresource, MoveTemp(LockedResource));

	check(Result.Data != nullptr);
	return Result;
}

void FD3D12Texture::UpdateTexture(FD3D12CommandContext& Context, uint32 MipIndex, uint32 DestX, uint32 DestY, uint32 DestZ, const D3D12_TEXTURE_COPY_LOCATION& SourceCopyLocation, FD3D12Resource* SourceResource)
{
	LLM_SCOPE_BYNAME(TEXT("D3D12CopyTextureRegion"));

	FScopedResourceBarrier ScopeResourceBarrierDest(
		Context,
		GetResource(),
		ED3D12Access::CopyDest,
		MipIndex);
	// Don't need to transition upload heaps

	CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(GetResource()->GetResource(), MipIndex);

	Context.FlushResourceBarriers();
	Context.CopyTextureRegionChecked(
		&DestCopyLocation,
		DestX, DestY, DestZ,
		GetFormat(),
		&SourceCopyLocation,
		nullptr,
		GetFormat(),
		GetName(),						// DestName
		SourceResource->GetName()		// SourceName
	);

	Context.UpdateResidency(GetResource());
	Context.UpdateResidency(SourceResource);

	Context.ConditionalSplitCommandList();

	DEBUG_EXECUTE_COMMAND_CONTEXT(Context);
}

void FD3D12Texture::CopyTextureRegion(FD3D12CommandContext& Context, uint32 DestX, uint32 DestY, uint32 DestZ, FD3D12Texture* SourceTexture, const D3D12_BOX& SourceBox)
{
	CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(GetResource()->GetResource(), 0);
	CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(SourceTexture->GetResource()->GetResource(), 0);

	FScopedResourceBarrier ConditionalScopeResourceBarrierDst(
		Context,
		GetResource(),
		ED3D12Access::CopyDest,
		DestCopyLocation.SubresourceIndex);

	FScopedResourceBarrier ConditionalScopeResourceBarrierSrc(
		Context,
		SourceTexture->GetResource(),
		ED3D12Access::CopySrc,
		SourceCopyLocation.SubresourceIndex);

	Context.FlushResourceBarriers();
	Context.CopyTextureRegionChecked(
		&DestCopyLocation,
		DestX, DestY, DestZ,
		GetFormat(),
		&SourceCopyLocation,
		&SourceBox,
		SourceTexture->GetFormat(),
		GetName(),						// DestName
		SourceTexture->GetName()		// SourceName
	);

	Context.UpdateResidency(SourceTexture->GetResource());
	Context.UpdateResidency(GetResource());
}

void FD3D12Texture::UploadInitialData(
	FRHICommandListBase& RHICmdList,
	FD3D12ResourceLocation&& SourceLocation,
	ED3D12Access InDestinationD3D12Access)
{
	RHICmdList.EnqueueLambda([Texture = this, SourceLocation = MoveTemp(SourceLocation), InDestinationD3D12Access](FRHICommandListBase& ExecutingCmdList)
	{
		const uint32 NumSubresources = CalculateSubresourceCount(Texture->GetDesc());

		size_t MemSize = NumSubresources * (sizeof(D3D12_PLACED_SUBRESOURCE_FOOTPRINT) + sizeof(UINT) + sizeof(UINT64));
		const bool bAllocateOnStack = (MemSize < 4096);
		void* Mem = bAllocateOnStack ? FMemory_Alloca(MemSize) : FMemory::Malloc(MemSize);
		ON_SCOPE_EXIT
		{
			if (!bAllocateOnStack)
			{
				FMemory::Free(Mem);
			}
		};

		D3D12_PLACED_SUBRESOURCE_FOOTPRINT * Footprints = (D3D12_PLACED_SUBRESOURCE_FOOTPRINT*)Mem;
		check(Footprints);
		UINT * Rows = (UINT*)(Footprints + NumSubresources);
		UINT64 * RowSizeInBytes = (UINT64*)(Rows + NumSubresources);

		uint64 Size = 0;
		const D3D12_RESOURCE_DESC & Desc = Texture->GetResource()->GetDesc();
		Texture->GetParentDevice()->GetDevice()->GetCopyableFootprints(&Desc, 0, NumSubresources, SourceLocation.GetOffsetFromBaseOfResource(), Footprints, Rows, RowSizeInBytes, &Size);

		D3D12_TEXTURE_COPY_LOCATION Src;
		Src.pResource = SourceLocation.GetResource()->GetResource();
		Src.Type = D3D12_TEXTURE_COPY_TYPE_PLACED_FOOTPRINT;

		// Initialize all the textures in the chain
		for (FD3D12Texture& CurrentTexture : *Texture)
		{
			FD3D12Device* Device = CurrentTexture.GetParentDevice();
			FD3D12Resource* Resource = CurrentTexture.GetResource();
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Device->GetGPUIndex());

			// resource should be in copy dest already, because it's created like that, so no transition required here

			D3D12_TEXTURE_COPY_LOCATION Dst;
			Dst.pResource = Resource->GetResource();
			Dst.Type = D3D12_TEXTURE_COPY_TYPE_SUBRESOURCE_INDEX;

			for (uint32 Subresource = 0; Subresource < NumSubresources; Subresource++)
			{
				// Src in this case is an anonymous resource location, named UploadLocation in the calling code, so use that name here.
				const static FName UploadLocationName("UploadLocation");
				Dst.SubresourceIndex = Subresource;
				Src.PlacedFootprint = Footprints[Subresource];
				Context.CopyTextureRegionChecked(&Dst, 0, 0, 0, Texture->GetFormat(), &Src, nullptr, Texture->GetFormat(), Texture->GetName(), UploadLocationName);
			}

			Context.UpdateResidency(SourceLocation.GetResource());
			Context.UpdateResidency(Resource);

			// Update the resource state after the copy has been done (will take care of updating the residency as well)
			Context.AddBarrier(
				Resource,
				ED3D12Access::CopyDest,
				InDestinationD3D12Access,
				D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
			Context.ConditionalSplitCommandList();

			// Texture is now written and ready, so unlock the block (locked after creation and can be defragmented if needed)
			CurrentTexture.ResourceLocation.UnlockPoolData();

			// If the resource is untracked, the destination state must match the default state of the resource.
			check(Resource->RequiresResourceStateTracking() || (Resource->GetDefaultAccess() == InDestinationD3D12Access));
		}
	});
}

void FD3D12Texture::Unlock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12UnlockTextureTime);
	check(IsHeadLink());

	const FRHITextureDesc& Desc = GetDesc();
	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Desc, Arguments);

	// Calculate the subresource index corresponding to the specified mip-map.
	const uint32 Subresource = CalcSubresource(Arguments.MipIndex, ArrayIndex, Desc.NumMips);

	TUniquePtr<FD3D12LockedResource> LockedResource = LockedMap.FindAndRemoveChecked(Subresource);
	check(LockedResource);

	if (FD3D12DynamicRHI::GetD3DRHI()->HandleSpecialUnlock(RHICmdList, this, Arguments))
	{
		// nothing left to do...
	}
	else if (!LockedResource->bLockedForReadOnly)
	{

		if (RHICmdList.NeedsExtraTransitions())
		{
			RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState, Arguments.MipIndex, ArrayIndex, 0));
		}

		RHICmdList.EnqueueLambda([
			RootTexture = this,
			Subresource,
			ArrayIndex,
			LockedResource = MoveTemp(LockedResource)
		](FRHICommandListBase& ExecutingCmdList) mutable		// Mutable required to allow DeferredDelete to move the captured LockedResource variable again
		{
			D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedFootprint
			{
				.Offset = LockedResource->ResourceLocation.GetOffsetFromBaseOfResource(),
				.Footprint = LockedResource->Footprint
			};

			CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(LockedResource->ResourceLocation.GetResource()->GetResource(), PlacedFootprint);

			// Copy the mip-map data from the real resource into the staging resource
			for (FD3D12Texture& Texture : *RootTexture)
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Texture.GetParentDevice()->GetGPUIndex());
				Texture.UpdateTexture(Context, Subresource, 0, 0, 0, SourceCopyLocation, LockedResource->ResourceLocation.GetResource());
			}

			// For Dynamic textures, the staging resource location is recycled back to the texture via the deferred deletion queue,
			// saving the cost of reallocating it, or allowing it to be reused again more quickly in the context of high resolution tiled
			// rendering (whenever commands get flushed, rather than at the end of the frame when pool elements are recycled).
			if (EnumHasAnyFlags(RootTexture->GetDesc().Flags, ETextureCreateFlags::Dynamic) &&
				(LockedResource->ResourceLocation.GetType() == FD3D12ResourceLocation::ResourceLocationType::eStandAlone ||
				 LockedResource->ResourceLocation.GetAllocatorType() == FD3D12ResourceLocation::EAllocatorType::AT_Pool))
			{
				FD3D12DynamicRHI::GetD3DRHI()->DeferredDelete(RootTexture, MoveTemp(LockedResource), Subresource);
			}
		});

		if (RHICmdList.NeedsExtraTransitions())
		{
			RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState, Arguments.MipIndex, ArrayIndex, 0));
		}

	}
}

void FD3D12Texture::UpdateTexture2D(FRHICommandListBase& RHICmdList, uint32 MipIndex, const FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetFormat()];

	check(UpdateRegion.Width  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.Height % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.DestX  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.DestY  % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.SrcX   % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.SrcY   % FormatInfo.BlockSizeY == 0);

	const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   FormatInfo.BlockSizeX);
	const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   FormatInfo.BlockSizeY);
	const uint32 WidthInBlocks  = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Width,  FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	// D3D12 requires specific alignments for pitch and size since we have to do the updates via buffers
	size_t StagingPitch = Align(static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	const size_t StagingBufferSize = Align(StagingPitch * HeightInBlocks, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT);

	FD3D12Device* Device = GetParentDevice();
	FD3D12ResourceLocation UploadHeapResourceLocation(Device);
	void* StagingMemory;

	// Lock full texture if possible -- Lock/Unlock takes advantage of existing entries in LockedMap if present, potentially improving performance and saving memory.
	// Besides the size matching, the command list must be immediate, otherwise it's not thread safe to access LockedMap.
	uint32 MipBlockWidth = UE::RHITextureUtils::CalculateMipBlockCount(GetDesc().Extent.X, MipIndex, FormatInfo.BlockSizeX);
	uint32 MipBlockHeight = UE::RHITextureUtils::CalculateMipBlockCount(GetDesc().Extent.Y, MipIndex, FormatInfo.BlockSizeY);
	bool bLockFullTexture = RHICmdList.IsImmediate() && UpdateRegion.DestX == 0 && UpdateRegion.DestY == 0 && MipBlockWidth == WidthInBlocks && MipBlockHeight == HeightInBlocks;
	FRHILockTextureArgs LockArgs = FRHILockTextureArgs::Lock2D(this, MipIndex, RLM_WriteOnly, false, false);

	if (bLockFullTexture)
	{
		FRHILockTextureResult LockResult = Lock(RHICmdList, LockArgs);
		StagingMemory = LockResult.Data;
		StagingPitch = LockResult.Stride;
	}
	else
	{
		StagingMemory = Device->GetDefaultFastAllocator().Allocate(StagingBufferSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, &UploadHeapResourceLocation);
	}
	check(StagingMemory);

	const uint8* CopySrc = SourceData + FormatInfo.BlockBytes * SrcXInBlocks + SourcePitch * SrcYInBlocks;
	uint8* CopyDst = (uint8*)StagingMemory;
	for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
	{
		FMemory::Memcpy(CopyDst, CopySrc, WidthInBlocks * FormatInfo.BlockBytes);
		CopySrc += SourcePitch;
		CopyDst += StagingPitch;
	}

	if (bLockFullTexture)
	{
		Unlock(RHICmdList, LockArgs);
		return;
	}

	check(StagingPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT == 0);

	D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedTexture2D
	{
		.Offset = UploadHeapResourceLocation.GetOffsetFromBaseOfResource(),
		.Footprint
		{
			.Format = (DXGI_FORMAT)FormatInfo.PlatformFormat,
			.Width = UpdateRegion.Width,
			.Height = UpdateRegion.Height,
			.Depth = 1,
			.RowPitch = (uint32)StagingPitch
		}
	};

	CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(UploadHeapResourceLocation.GetResource()->GetResource(), PlacedTexture2D);

	if (RHICmdList.NeedsExtraTransitions())
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState, MipIndex, 0, 0), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}

	RHICmdList.EnqueueLambda([
		RootTexture = this,
		SourceCopyLocation,
		UploadHeapResourceLocation = MoveTemp(UploadHeapResourceLocation),
		MipIndex,
		UpdateRegion
	](FRHICommandListBase& ExecutingCmdList)
	{
		for (FD3D12Texture& Texture : *RootTexture)
		{
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Texture.GetParentDevice()->GetGPUIndex());
			Texture.UpdateTexture(Context, MipIndex, UpdateRegion.DestX, UpdateRegion.DestY, 0, SourceCopyLocation, UploadHeapResourceLocation.GetResource());
		}
	});

	if (RHICmdList.NeedsExtraTransitions())
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(this, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState, MipIndex, 0, 0), ERHITransitionCreateFlags::AllowDuringRenderPass);
	}
}

static void GetReadBackHeapDescImpl(D3D12_PLACED_SUBRESOURCE_FOOTPRINT& OutFootprint, ID3D12Device* InDevice, D3D12_RESOURCE_DESC const& InResourceDesc, uint32 InSubresource)
{
	uint64 Offset = 0;
	if (InSubresource > 0)
	{
		InDevice->GetCopyableFootprints(&InResourceDesc, 0, InSubresource, 0, nullptr, nullptr, nullptr, &Offset);
		Offset = Align(Offset, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT);
	}
	InDevice->GetCopyableFootprints(&InResourceDesc, InSubresource, 1, Offset, &OutFootprint, nullptr, nullptr, nullptr);

	check(OutFootprint.Footprint.Width > 0 && OutFootprint.Footprint.Height > 0);
}

void FD3D12Texture::GetReadBackHeapDesc(D3D12_PLACED_SUBRESOURCE_FOOTPRINT& OutFootprint, uint32 InSubresource) const
{
	check(EnumHasAnyFlags(GetFlags(), TexCreate_CPUReadback));

	if (InSubresource == 0 && FirstSubresourceFootprint)
	{
		OutFootprint = *FirstSubresourceFootprint;
		return;
	}

	FIntVector TextureSize = GetSizeXYZ();

	D3D12_RESOURCE_DESC Desc = {};
	Desc.Dimension        = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	Desc.Width            = TextureSize.X;
	Desc.Height           = TextureSize.Y;
	Desc.DepthOrArraySize = TextureSize.Z;
	Desc.MipLevels        = GetNumMips();
	Desc.Format           = (DXGI_FORMAT) GPixelFormats[GetFormat()].PlatformFormat;
	Desc.SampleDesc.Count = GetNumSamples();

	GetReadBackHeapDescImpl(OutFootprint, GetParentDevice()->GetDevice(), Desc, InSubresource);

	if (InSubresource == 0)
	{
		FirstSubresourceFootprint = MakeUnique<D3D12_PLACED_SUBRESOURCE_FOOTPRINT>();
		*FirstSubresourceFootprint = OutFootprint;
	}
}

FRHILockTextureResult FD3D12DynamicRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(Arguments.Texture);
	return Texture->Lock(RHICmdList, Arguments);
}

void FD3D12DynamicRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(Arguments.Texture);
	Texture->Unlock(RHICmdList, Arguments);
}

void FD3D12DynamicRHI::RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	check(TextureRHI);
	FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(TextureRHI);
	Texture->UpdateTexture2D(RHICmdList, MipIndex, UpdateRegion, SourcePitch, SourceData);
}

FUpdateTexture3DData FD3D12DynamicRHI::RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
{
	return BeginUpdateTexture3D_Internal(Texture, MipIndex, UpdateRegion);
}

void FD3D12DynamicRHI::RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData)
{
	EndUpdateTexture3D_Internal(RHICmdList, UpdateData);
}

struct FD3D12RHICmdEndMultiUpdateTexture3DString
{
	static const TCHAR* TStr() { return TEXT("FD3D12RHICmdEndMultiUpdateTexture3D"); }
};
class FD3D12RHICmdEndMultiUpdateTexture3D : public FRHICommand<FD3D12RHICmdEndMultiUpdateTexture3D, FD3D12RHICmdEndMultiUpdateTexture3DString>
{
public:
	FD3D12RHICmdEndMultiUpdateTexture3D(TArray<FUpdateTexture3DData>& UpdateDataArray) :
		MipIdx(UpdateDataArray[0].MipIndex),
		DstTexture(UpdateDataArray[0].Texture)
	{
		const int32 NumUpdates = UpdateDataArray.Num();
		UpdateInfos.Empty(NumUpdates);
		UpdateInfos.AddZeroed(NumUpdates);

		for (int32 Idx = 0; Idx < UpdateInfos.Num(); ++Idx)
		{
			FUpdateInfo& UpdateInfo = UpdateInfos[Idx];
			FUpdateTexture3DData& UpdateData = UpdateDataArray[Idx];

			UpdateInfo.DstStartX = UpdateData.UpdateRegion.DestX;
			UpdateInfo.DstStartY = UpdateData.UpdateRegion.DestY;
			UpdateInfo.DstStartZ = UpdateData.UpdateRegion.DestZ;

			D3D12_SUBRESOURCE_FOOTPRINT& SubresourceFootprint = UpdateInfo.PlacedSubresourceFootprint.Footprint;
			SubresourceFootprint.Depth = UpdateData.UpdateRegion.Depth;
			SubresourceFootprint.Height = UpdateData.UpdateRegion.Height;
			SubresourceFootprint.Width = UpdateData.UpdateRegion.Width;
			SubresourceFootprint.Format = static_cast<DXGI_FORMAT>(GPixelFormats[DstTexture->GetFormat()].PlatformFormat);
			SubresourceFootprint.RowPitch = UpdateData.RowPitch;
			check(SubresourceFootprint.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT == 0);

			FD3D12UpdateTexture3DData* UpdateDataD3D12 =
				reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateData.PlatformData[0]);

			UpdateInfo.SrcResourceLocation = UpdateDataD3D12->UploadHeapResourceLocation;
			UpdateInfo.PlacedSubresourceFootprint.Offset = UpdateInfo.SrcResourceLocation->GetOffsetFromBaseOfResource();
		}
	}

	virtual ~FD3D12RHICmdEndMultiUpdateTexture3D()
	{
		for (int32 Idx = 0; Idx < UpdateInfos.Num(); ++Idx)
		{
			const FUpdateInfo& UpdateInfo = UpdateInfos[Idx];
			if (UpdateInfo.SrcResourceLocation)
			{
				delete UpdateInfo.SrcResourceLocation;
			}
		}
		UpdateInfos.Empty();
	}

	void Execute(FRHICommandListBase& ExecutingCmdList)
	{
		FD3D12Texture* NativeTexture = FD3D12DynamicRHI::ResourceCast(DstTexture.GetReference());

		for (FD3D12Texture& TextureLink : *NativeTexture)
		{
			FD3D12Device* Device = TextureLink.GetParentDevice();
			FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Device->GetGPUIndex());

			CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(TextureLink.GetResource()->GetResource(), MipIdx);

			FScopedResourceBarrier ScopeResourceBarrierDest(
				Context,
				TextureLink.GetResource(),
				ED3D12Access::CopyDest,
				DestCopyLocation.SubresourceIndex);

			Context.FlushResourceBarriers();

			for (int32 Idx = 0; Idx < UpdateInfos.Num(); ++Idx)
			{
				const FUpdateInfo& UpdateInfo = UpdateInfos[Idx];
				FD3D12Resource* UploadBuffer = UpdateInfo.SrcResourceLocation->GetResource();
				CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(UploadBuffer->GetResource(), UpdateInfo.PlacedSubresourceFootprint);

				RHI_BREADCRUMB_EVENT(Context, "EndMultiUpdateTexture3D");

				Context.CopyTextureRegionChecked(
					&DestCopyLocation,
					UpdateInfo.DstStartX,
					UpdateInfo.DstStartY,
					UpdateInfo.DstStartZ,
					TextureLink.GetFormat(),
					&SourceCopyLocation,
					nullptr,
					TextureLink.GetFormat(),
					TextureLink.GetName(),		// DestName
					UploadBuffer->GetName()		// SourceName
				);

				Context.UpdateResidency(TextureLink.GetResource());
				Context.UpdateResidency(UploadBuffer);
				DEBUG_EXECUTE_COMMAND_CONTEXT(Context);
			}

			Context.ConditionalSplitCommandList();
		}
	}

private:
	struct FUpdateInfo
	{
		uint32 DstStartX;
		uint32 DstStartY;
		uint32 DstStartZ;
		FD3D12ResourceLocation* SrcResourceLocation;
		D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedSubresourceFootprint;
	};

	uint32 MipIdx;
	FTextureRHIRef DstTexture;
	TArray<FUpdateInfo> UpdateInfos;
};

// Single pair of transition barriers instead of one pair for each update
void FD3D12DynamicRHI::RHIEndMultiUpdateTexture3D(FRHICommandListBase& RHICmdList, TArray<FUpdateTexture3DData>& UpdateDataArray)
{
	check(IsInParallelRenderingThread());
	check(UpdateDataArray.Num() > 0);
	check(GFrameNumberRenderThread == UpdateDataArray[0].FrameNumber);
#if DO_CHECK
	for (FUpdateTexture3DData& UpdateData : UpdateDataArray)
	{
		check(UpdateData.FrameNumber == UpdateDataArray[0].FrameNumber);
		check(UpdateData.MipIndex == UpdateDataArray[0].MipIndex);
		check(UpdateData.Texture == UpdateDataArray[0].Texture);
		FD3D12UpdateTexture3DData* UpdateDataD3D12 =
			reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateData.PlatformData[0]);
		check(!!UpdateDataD3D12->UploadHeapResourceLocation);
		check(UpdateDataD3D12->bComputeShaderCopy ==
			reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateDataArray[0].PlatformData[0])->bComputeShaderCopy);
	}
#endif

	bool bComputeShaderCopy = reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateDataArray[0].PlatformData[0])->bComputeShaderCopy;

	if (bComputeShaderCopy)
	{
		// TODO: implement proper EndMultiUpdate for the compute shader path
		for (int32 Idx = 0; Idx < UpdateDataArray.Num(); ++Idx)
		{
			FUpdateTexture3DData& UpdateData = UpdateDataArray[Idx];
			FD3D12UpdateTexture3DData* UpdateDataD3D12 =
				reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateData.PlatformData[0]);
			EndUpdateTexture3D_ComputeShader(static_cast<FRHIComputeCommandList&>(RHICmdList), UpdateData, UpdateDataD3D12);
		}
	}
	else
	{
		if (RHICmdList.IsBottomOfPipe())
		{
			FD3D12RHICmdEndMultiUpdateTexture3D RHICmd(UpdateDataArray);
			RHICmd.Execute(RHICmdList);
		}
		else
		{
			new (RHICmdList.AllocCommand<FD3D12RHICmdEndMultiUpdateTexture3D>()) FD3D12RHICmdEndMultiUpdateTexture3D(UpdateDataArray);
		}
	}
}

void FD3D12DynamicRHI::RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion3D& InUpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
{
	FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(TextureRHI);
	const FPixelFormatInfo& FormatInfo = GPixelFormats[Texture->GetFormat()];

	// Need to round up the height and with by block size.
	FUpdateTextureRegion3D UpdateRegion = InUpdateRegion;

	const uint32 NumBlockX = (uint32)FMath::DivideAndRoundUp<int32>(UpdateRegion.Width, FormatInfo.BlockSizeX);
	const uint32 NumBlockY = (uint32)FMath::DivideAndRoundUp<int32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	UpdateRegion.Width  = NumBlockX * FormatInfo.BlockSizeX;
	UpdateRegion.Height = NumBlockY * FormatInfo.BlockSizeY;

	FUpdateTexture3DData UpdateData = BeginUpdateTexture3D_Internal(TextureRHI, MipIndex, UpdateRegion);

	const uint32 UpdateBytesRow = NumBlockX * FormatInfo.BlockBytes;
	const uint32 UpdateBytesDepth = NumBlockY * UpdateBytesRow;

	// Copy the data into the UpdateData destination buffer
	check(nullptr != UpdateData.Data);
	check(SourceRowPitch >= UpdateBytesRow);
	check(SourceDepthPitch >= UpdateBytesDepth);

	const uint32 NumRows = UpdateRegion.Height / (uint32)FormatInfo.BlockSizeY;

	for (uint32 i = 0; i < UpdateRegion.Depth; i++)
	{
		uint8* DestRowData = UpdateData.Data + UpdateData.DepthPitch * i;
		const uint8* SourceRowData = SourceData + SourceDepthPitch * i;

		for (uint32 j = 0; j < NumRows; j++)
		{
			FMemory::Memcpy(DestRowData, SourceRowData, UpdateBytesRow);
			SourceRowData += SourceRowPitch;
			DestRowData += UpdateData.RowPitch;
		}
	}

	FD3D12UpdateTexture3DData* UpdateDataD3D12 = reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateData.PlatformData[0]);
	bool bNeedTransition = (!(UpdateDataD3D12->bComputeShaderCopy)) && RHICmdList.NeedsExtraTransitions();

	if (bNeedTransition)
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState, MipIndex, 0, 0));
	}

	EndUpdateTexture3D_Internal(RHICmdList, UpdateData);

	if (bNeedTransition)
	{
		RHICmdList.TransitionInternal(FRHITransitionInfo(TextureRHI, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState, MipIndex, 0, 0));
	}
}


FUpdateTexture3DData FD3D12DynamicRHI::BeginUpdateTexture3D_Internal(FRHITexture* TextureRHI, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
{
	check(IsInParallelRenderingThread());
	FUpdateTexture3DData UpdateData(TextureRHI, MipIndex, UpdateRegion, 0, 0, nullptr, 0, GFrameNumberRenderThread);
		
	// Initialize the platform data
	static_assert(sizeof(FD3D12UpdateTexture3DData) < sizeof(UpdateData.PlatformData), "Platform data in FUpdateTexture3DData too small to support D3D12");
	FD3D12UpdateTexture3DData* UpdateDataD3D12 = new (&UpdateData.PlatformData[0]) FD3D12UpdateTexture3DData;
	UpdateDataD3D12->bComputeShaderCopy = false;
	UpdateDataD3D12->UploadHeapResourceLocation = nullptr;

	FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(TextureRHI);
	const FPixelFormatInfo& FormatInfo = GPixelFormats[Texture->GetFormat()];
	check(FormatInfo.BlockSizeZ == 1);

	bool bDoComputeShaderCopy = false; // Compute shader can not cast compressed formats into uint
	if (CVarUseUpdateTexture3DComputeShader.GetValueOnRenderThread() != 0 && FormatInfo.BlockSizeX == 1 && FormatInfo.BlockSizeY == 1 && Texture->ResourceLocation.GetGPUVirtualAddress() && !EnumHasAnyFlags(Texture->GetFlags(), TexCreate_OfflineProcessed))
	{
		// Try a compute shader update. This does a memory allocation internally
		bDoComputeShaderCopy = BeginUpdateTexture3D_ComputeShader(UpdateData, UpdateDataD3D12);
	}

	if (!bDoComputeShaderCopy)
	{	
		const int32 NumBlockX = FMath::DivideAndRoundUp<int32>(UpdateRegion.Width, FormatInfo.BlockSizeX);
		const int32 NumBlockY = FMath::DivideAndRoundUp<int32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

		UpdateData.RowPitch = Align(NumBlockX * FormatInfo.BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
		UpdateData.DepthPitch = Align(UpdateData.RowPitch * NumBlockY, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
		const uint32 BufferSize = Align(UpdateRegion.Depth * UpdateData.DepthPitch, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT);
		UpdateData.DataSizeBytes = BufferSize;

		// This is a system memory heap so it doesn't matter which device we use.
		const uint32 HeapGPUIndex = 0;
		UpdateDataD3D12->UploadHeapResourceLocation = new FD3D12ResourceLocation(GetRHIDevice(HeapGPUIndex));

		//@TODO Probably need to use the TextureAllocator here to get correct tiling.
		// Currently the texture are allocated in linear, see hanlding around bVolume in FXboxOneTextureFormat::CompressImage(). 
		UpdateData.Data = (uint8*)GetRHIDevice(HeapGPUIndex)->GetDefaultFastAllocator().Allocate(BufferSize, D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, UpdateDataD3D12->UploadHeapResourceLocation);

		check(UpdateData.Data != nullptr);
	}
	return UpdateData;
}

void FD3D12DynamicRHI::EndUpdateTexture3D_Internal(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData)
{
	check(GFrameNumberRenderThread == UpdateData.FrameNumber);

	FD3D12UpdateTexture3DData* UpdateDataD3D12 = reinterpret_cast<FD3D12UpdateTexture3DData*>(&UpdateData.PlatformData[0]);
	check(UpdateDataD3D12->UploadHeapResourceLocation != nullptr);

	if (UpdateDataD3D12->bComputeShaderCopy)
	{
		EndUpdateTexture3D_ComputeShader(static_cast<FRHIComputeCommandList&>(RHICmdList), UpdateData, UpdateDataD3D12);
	}
	else
	{
		check(UpdateData.RowPitch % FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT == 0);
		FD3D12ResourceLocation* SrcResourceLocation = UpdateDataD3D12->UploadHeapResourceLocation;

		D3D12_PLACED_SUBRESOURCE_FOOTPRINT PlacedSubresourceFootprint
		{
			.Offset = SrcResourceLocation->GetOffsetFromBaseOfResource(),
			.Footprint
			{
				.Format   = static_cast<DXGI_FORMAT>(GPixelFormats[UpdateData.Texture->GetFormat()].PlatformFormat),
				.Width    = UpdateData.UpdateRegion.Width,
				.Height   = UpdateData.UpdateRegion.Height,
				.Depth    = UpdateData.UpdateRegion.Depth,
				.RowPitch = UpdateData.RowPitch
			}
		};

		RHICmdList.EnqueueLambda([
			MipIdx      = UpdateData.MipIndex,
			DstStartX   = UpdateData.UpdateRegion.DestX,
			DstStartY   = UpdateData.UpdateRegion.DestY,
			DstStartZ   = UpdateData.UpdateRegion.DestZ,
			RootTexture = UpdateData.Texture,
			PlacedSubresourceFootprint,
			SrcResourceLocation
		](FRHICommandListBase& ExecutingCmdList)
		{
			for (FD3D12Texture& Texture : *FD3D12DynamicRHI::ResourceCast(RootTexture))
			{
				FD3D12CommandContext& Context = FD3D12CommandContext::Get(ExecutingCmdList, Texture.GetParentDevice()->GetGPUIndex());
				RHI_BREADCRUMB_EVENT(Context, "EndUpdateTexture3D");

				CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(Texture.GetResource()->GetResource(), MipIdx);
				CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(SrcResourceLocation->GetResource()->GetResource(), PlacedSubresourceFootprint);

				FScopedResourceBarrier ScopeResourceBarrierDest(
					Context,
					Texture.GetResource(),
					ED3D12Access::CopyDest,
					DestCopyLocation.SubresourceIndex);

				Context.FlushResourceBarriers();
				Context.UpdateResidency(Texture.GetResource());
				Context.UpdateResidency(SrcResourceLocation->GetResource());

				Context.CopyTextureRegionChecked(
					&DestCopyLocation,
					DstStartX,
					DstStartY,
					DstStartZ,
					Texture.GetFormat(),
					&SourceCopyLocation,
					nullptr,
					Texture.GetFormat(),
					Texture.GetName(),									// DestName
					SrcResourceLocation->GetResource()->GetName()		// SourceName
				);

				Context.ConditionalSplitCommandList();
				DEBUG_EXECUTE_COMMAND_CONTEXT(Context);
			}

			delete SrcResourceLocation;
		});
	}
}

void FD3D12DynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, const TCHAR* Name)
{
	if (!TextureRHI || !GD3D12BindResourceLabels)
	{
		return;
	}

#if RHI_USE_RESOURCE_DEBUG_NAME
	// Also set on RHI object
	TextureRHI->SetName(Name);

	FD3D12Texture::FLinkedObjectIterator BaseTexture(GetD3D12TextureFromRHITexture(TextureRHI));

	if (GNumExplicitGPUsForRendering > 1)
	{
		// Generate string of the form "Name (GPU #)" -- assumes GPU index is a single digit.  This is called many times
		// a frame, so we want to avoid any string functions which dynamically allocate, to reduce perf overhead.
		static_assert(MAX_NUM_GPUS <= 10);

		static const TCHAR NameSuffix[] = TEXT(" (GPU #)");
		constexpr int32 NameSuffixLengthWithTerminator = (int32)UE_ARRAY_COUNT(NameSuffix);
		constexpr int32 NameBufferLength = 256;
		constexpr int32 GPUIndexSuffixOffset = 6;		// Offset of '#' character

		// Combine Name and suffix in our string buffer (clamping the length for bounds checking).  We'll replace the GPU index
		// with the appropriate digit in the loop.
		int32 NameLength = FMath::Min(FCString::Strlen(Name), NameBufferLength - NameSuffixLengthWithTerminator);
		int32 GPUIndexOffset = NameLength + GPUIndexSuffixOffset;

		TCHAR DebugName[NameBufferLength];
		FMemory::Memcpy(&DebugName[0], Name, NameLength * sizeof(TCHAR));
		FMemory::Memcpy(&DebugName[NameLength], NameSuffix, NameSuffixLengthWithTerminator * sizeof(TCHAR));

		for (; BaseTexture; ++BaseTexture)
		{
			FD3D12Resource* Resource = BaseTexture->GetResource();

			DebugName[GPUIndexOffset] = TEXT('0') + BaseTexture->GetParentDevice()->GetGPUIndex();

			SetD3D12ResourceName(Resource, DebugName);
		}
	}
	else
	{
		SetD3D12ResourceName(BaseTexture->GetResource(), Name);
	}
#endif
}

FD3D12Texture* FD3D12DynamicRHI::CreateTextureFromResource(bool bTextureArray, bool bCubeTexture, EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource)
{
	check(Resource);
	FD3D12Adapter* Adapter = &GetAdapter();

	FD3D12ResourceDesc TextureDesc = Resource->GetDesc();
	TextureDesc.bExternal = true;
	TextureDesc.Alignment = 0;

	uint32 SizeX = TextureDesc.Width;
	uint32 SizeY = TextureDesc.Height;
	uint32 SizeZ = TextureDesc.DepthOrArraySize;
	uint32 NumMips = TextureDesc.MipLevels;
	uint32 NumSamples = TextureDesc.SampleDesc.Count;
	
	check(TextureDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE2D);
	check(bTextureArray || (!bCubeTexture && SizeZ == 1) || (bCubeTexture && SizeZ == 6));

	//TODO: Somehow Oculus is creating a Render Target with 4k alignment with ovr_GetTextureSwapChainBufferDX
	//      This is invalid and causes our size calculation to fail. Oculus SDK bug?
	if (TextureDesc.Flags & D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET)
	{
		TextureDesc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
	}

	SCOPE_CYCLE_COUNTER(STAT_D3D12CreateTextureTime);

	// The state this resource will be in when it leaves this function
	const FD3D12Resource::FD3D12ResourceTypeHelper Type(TextureDesc, D3D12_HEAP_TYPE_DEFAULT);
	const ED3D12Access DestinationD3D12Access = Type.GetOptimalInitialD3D12Access(ED3D12Access::Unknown, !EnumHasAnyFlags(TexCreateFlags, TexCreate_Shared));

	FD3D12Device* Device = Adapter->GetDevice(0);
	FD3D12Resource* TextureResource = new FD3D12Resource(Device, Device->GetGPUMask(), Resource, DestinationD3D12Access, TextureDesc);
	TextureResource->AddRef();
	TextureResource->SetName(TEXT("TextureFromResource"));

	const ETextureDimension Dimension = bTextureArray
		? (bCubeTexture ? ETextureDimension::TextureCubeArray : ETextureDimension::Texture2DArray)
		: (bCubeTexture ? ETextureDimension::TextureCube      : ETextureDimension::Texture2D     );

	FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create(TEXT("TextureFromResource"), Dimension)
		.SetExtent(SizeX, SizeY)
		.SetArraySize(bCubeTexture ? SizeZ / 6 : SizeZ)
		.SetFormat(Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(TexCreateFlags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.SetInitialState(ERHIAccess::SRVMask);

	FD3D12Texture* Texture2D = Adapter->CreateLinkedObject<FD3D12Texture>(Device->GetGPUMask(), [&](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
	{
		return CreateNewD3D12Texture(CreateDesc, Device);
	});

	FD3D12ResourceLocation& Location = Texture2D->ResourceLocation;
	Location.SetType(FD3D12ResourceLocation::ResourceLocationType::eAliased);
	Location.SetResource(TextureResource);
	Location.SetGPUVirtualAddress(TextureResource->GetGPUVirtualAddress());

	Texture2D->CreateViews(nullptr);		// Always single GPU object, so FirstLinkedObject is nullptr

	FD3D12TextureStats::D3D12TextureAllocated(*Texture2D);

	return Texture2D;
}

FTextureRHIRef FD3D12DynamicRHI::RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource)
{
	return CreateTextureFromResource(false, false, Format, TexCreateFlags, ClearValueBinding, Resource);
}

FTextureRHIRef FD3D12DynamicRHI::RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource)
{
	return CreateTextureFromResource(true, false, Format, TexCreateFlags, ClearValueBinding, Resource);
}

FTextureRHIRef FD3D12DynamicRHI::RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource)
{
	return CreateTextureFromResource(false, true, Format, TexCreateFlags, ClearValueBinding, Resource);
}

#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
void FD3D12DynamicRHI::RHIAliasTextureResources(FTextureRHIRef& DestTextureRHI, FTextureRHIRef& SrcTextureRHI)
{
	FD3D12Texture* DestTexture = GetD3D12TextureFromRHITexture(DestTextureRHI);
	FD3D12Texture* SrcTexture = GetD3D12TextureFromRHITexture(SrcTextureRHI);

	// Make sure we keep a reference to the source texture we're aliasing, so we don't lose it if all other references
	// go away but we're kept around.
	DestTexture->SetAliasingSource(SrcTextureRHI);

	for (FD3D12Texture::FDualLinkedObjectIterator It(DestTexture, SrcTexture); It; ++It)
	{
		FD3D12Texture* DestLinkedTexture = It.GetFirst();
		FD3D12Texture* SrcLinkedTexture = It.GetSecond();

		DestLinkedTexture->AliasResources(SrcLinkedTexture);
	}
}

FD3D12Texture* FD3D12DynamicRHI::CreateAliasedD3D12Texture2D(FD3D12Texture* SourceTexture)
{
	FD3D12Adapter* Adapter = &GetAdapter();

	D3D12_RESOURCE_DESC TextureDesc = SourceTexture->GetResource()->GetDesc();
	TextureDesc.Alignment = 0;

	uint32 SizeX = TextureDesc.Width;
	uint32 SizeY = TextureDesc.Height;
	uint32 SizeZ = TextureDesc.DepthOrArraySize;
	uint32 NumMips = TextureDesc.MipLevels;
	uint32 NumSamples = TextureDesc.SampleDesc.Count;

	check(TextureDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE2D);

	//TODO: Somehow Oculus is creating a Render Target with 4k alignment with ovr_GetTextureSwapChainBufferDX
	//      This is invalid and causes our size calculation to fail. Oculus SDK bug?
	if (TextureDesc.Flags & D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET)
	{
		TextureDesc.Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
	}

	SCOPE_CYCLE_COUNTER(STAT_D3D12CreateTextureTime);

	FD3D12Device* Device = Adapter->GetDevice(0);

	const bool bSRGB = EnumHasAnyFlags(SourceTexture->GetFlags(), TexCreate_SRGB);

	const DXGI_FORMAT PlatformResourceFormat = TextureDesc.Format;
	const DXGI_FORMAT PlatformShaderResourceFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);
	const DXGI_FORMAT PlatformRenderTargetFormat = UE::DXGIUtilities::FindShaderResourceFormat(PlatformResourceFormat, bSRGB);

	const FString Name = SourceTexture->GetName().ToString() + TEXT("Alias");
	FRHITextureCreateDesc CreateDesc(SourceTexture->GetDesc(), ERHIAccess::SRVMask, *Name);

	FD3D12Texture* Texture2D = Adapter->CreateLinkedObject<FD3D12Texture>(Device->GetGPUMask(), [&](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
	{
		return CreateNewD3D12Texture(CreateDesc, Device);
	});

	RHIAliasTextureResources((FTextureRHIRef&)Texture2D, (FTextureRHIRef&)SourceTexture);

	return Texture2D;
}

FTextureRHIRef FD3D12DynamicRHI::RHICreateAliasedTexture(FTextureRHIRef& SourceTextureRHI)
{
	FD3D12Texture* SourceTexture = GetD3D12TextureFromRHITexture(SourceTextureRHI);
	FD3D12Texture* ReturnTexture = CreateAliasedD3D12Texture2D(SourceTexture);
	if (ReturnTexture == nullptr)
	{
		UE_LOGF(LogD3D12RHI, Error, "Currently FD3D12DynamicRHI::RHICreateAliasedTexture only supports 2D, 2D Array and Cube textures.");
		return nullptr;
	}

	return ReturnTexture;
}
#endif // D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES

///////////////////////////////////////////////////////////////////////////////////////////
// FD3D12CommandContext Texture functions
///////////////////////////////////////////////////////////////////////////////////////////

void FD3D12CommandContext::RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo)
{
	FD3D12Texture* SourceTexture = RetrieveTexture(SourceTextureRHI);
	FD3D12Texture* DestTexture = RetrieveTexture(DestTextureRHI);

	FScopedResourceBarrier ConditionalScopeResourceBarrierSrc(
		*this,
		SourceTexture->GetResource(),
		ED3D12Access::CopySrc,
		D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);
	FScopedResourceBarrier ConditionalScopeResourceBarrierDst(
		*this,
		DestTexture->GetResource(),
		ED3D12Access::CopyDest,
		D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);

	FlushResourceBarriers();

	const bool bReadback = EnumHasAnyFlags(DestTextureRHI->GetFlags(), TexCreate_CPUReadback);

	const FRHITextureDesc& SourceDesc = SourceTextureRHI->GetDesc();
	const FRHITextureDesc& DestDesc = DestTextureRHI->GetDesc();
	
	const uint16 SourceArraySize = SourceDesc.ArraySize * (SourceDesc.IsTextureCube() ? 6 : 1);
	const uint16 DestArraySize   = DestDesc.ArraySize   * (DestDesc.IsTextureCube()   ? 6 : 1);

	const bool bAllPixels =
		SourceDesc.GetSize() == DestDesc.GetSize() && (CopyInfo.Size == FIntVector::ZeroValue || CopyInfo.Size == SourceDesc.GetSize());

	const bool bAllSubresources =
		SourceDesc.NumMips   == DestDesc.NumMips   && SourceDesc.NumMips   == CopyInfo.NumMips    &&
		SourceArraySize == DestArraySize && SourceArraySize == CopyInfo.NumSlices;

	if (!bAllPixels || !bAllSubresources || bReadback)
	{
		const FIntVector SourceSize = SourceDesc.GetSize();
		const FIntVector CopySize = CopyInfo.Size == FIntVector::ZeroValue ? SourceSize >> CopyInfo.SourceMipIndex : CopyInfo.Size;

		D3D12_TEXTURE_COPY_LOCATION Src;
		Src.pResource = SourceTexture->GetResource()->GetResource();
		Src.Type = D3D12_TEXTURE_COPY_TYPE_SUBRESOURCE_INDEX;

		D3D12_TEXTURE_COPY_LOCATION Dst;
		Dst.pResource = DestTexture->GetResource()->GetResource();
		Dst.Type = bReadback ? D3D12_TEXTURE_COPY_TYPE_PLACED_FOOTPRINT : D3D12_TEXTURE_COPY_TYPE_SUBRESOURCE_INDEX;

		const FPixelFormatInfo& SourcePixelFormatInfo = GPixelFormats[SourceTextureRHI->GetFormat()];
		const FPixelFormatInfo& DestPixelFormatInfo = GPixelFormats[DestTextureRHI->GetFormat()];

		D3D12_RESOURCE_DESC DstDesc = {};
		FIntVector TextureSize = DestTextureRHI->GetSizeXYZ();
		DstDesc.Dimension = DestTextureRHI->GetTexture3D() ? D3D12_RESOURCE_DIMENSION_TEXTURE3D : D3D12_RESOURCE_DIMENSION_TEXTURE2D; 
		DstDesc.Width = TextureSize.X;
		DstDesc.Height = TextureSize.Y;
		DstDesc.DepthOrArraySize = TextureSize.Z;
		DstDesc.MipLevels = DestTextureRHI->GetNumMips();
		DstDesc.Format = (DXGI_FORMAT)DestPixelFormatInfo.PlatformFormat;
		DstDesc.SampleDesc.Count = DestTextureRHI->GetNumSamples();

		for (uint32 SliceIndex = 0; SliceIndex < CopyInfo.NumSlices; ++SliceIndex)
		{
			uint32 SourceSliceIndex = CopyInfo.SourceSliceIndex + SliceIndex;
			uint32 DestSliceIndex   = CopyInfo.DestSliceIndex   + SliceIndex;

			for (uint32 MipIndex = 0; MipIndex < CopyInfo.NumMips; ++MipIndex)
			{
				uint32 SourceMipIndex = CopyInfo.SourceMipIndex + MipIndex;
				uint32 DestMipIndex   = CopyInfo.DestMipIndex   + MipIndex;

				D3D12_BOX SrcBox;
				SrcBox.left   = CopyInfo.SourcePosition.X >> MipIndex;
				SrcBox.top    = CopyInfo.SourcePosition.Y >> MipIndex;
				SrcBox.front  = CopyInfo.SourcePosition.Z >> MipIndex;
				SrcBox.right  = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.X + CopySize.X) >> MipIndex, 1), SourcePixelFormatInfo.BlockSizeX);
				SrcBox.bottom = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.Y + CopySize.Y) >> MipIndex, 1), SourcePixelFormatInfo.BlockSizeY);
				SrcBox.back   = AlignArbitrary<uint32>(FMath::Max<uint32>((CopyInfo.SourcePosition.Z + CopySize.Z) >> MipIndex, 1), SourcePixelFormatInfo.BlockSizeZ);

				const uint32 DestX = CopyInfo.DestPosition.X >> MipIndex;
				const uint32 DestY = CopyInfo.DestPosition.Y >> MipIndex;
				const uint32 DestZ = CopyInfo.DestPosition.Z >> MipIndex;

				// RHICopyTexture is allowed to copy mip regions only if are aligned on the block size to prevent unexpected / inconsistent results.
				ensure(SrcBox.left % SourcePixelFormatInfo.BlockSizeX == 0 && SrcBox.top % SourcePixelFormatInfo.BlockSizeY == 0 && SrcBox.front % SourcePixelFormatInfo.BlockSizeZ == 0);
				ensure(DestX % DestPixelFormatInfo.BlockSizeX == 0 && DestY % DestPixelFormatInfo.BlockSizeY == 0 && DestZ % DestPixelFormatInfo.BlockSizeZ == 0);

				Src.SubresourceIndex = CalcSubresource(SourceMipIndex, SourceSliceIndex, SourceTextureRHI->GetNumMips());
				Dst.SubresourceIndex = CalcSubresource(DestMipIndex, DestSliceIndex, DestTextureRHI->GetNumMips());

				if (bReadback)
				{
					GetReadBackHeapDescImpl(Dst.PlacedFootprint, GetParentDevice()->GetDevice(), DstDesc, Dst.SubresourceIndex);
				}

				CopyTextureRegionChecked(
					&Dst,
					DestX, DestY, DestZ,
					DestTexture->GetFormat(),
					&Src,
					&SrcBox,
					SourceTexture->GetFormat(),
					DestTexture->GetName(),		// DestName
					SourceTexture->GetName()	// SourceName
				);
			}
		}
	}
	else
	{
		// Copy whole texture
		GraphicsCommandList()->CopyResource(DestTexture->GetResource()->GetResource(), SourceTexture->GetResource()->GetResource());
	}

	UpdateResidency(SourceTexture->GetResource());
	UpdateResidency(DestTexture->GetResource());
	
	ConditionalSplitCommandList();
}

#if D3D12RHI_USE_DUMMY_BACKBUFFER

///////////////////////////////////////////////////////////////////////////////////////////
// FD3D12BackBufferReferenceTexture2D functions
///////////////////////////////////////////////////////////////////////////////////////////

FRHITexture* FD3D12BackBufferReferenceTexture2D::GetBackBufferTexture() const
{
	return Viewport->GetBackBuffer_RHIThread();
}

FRHIDescriptorHandle FD3D12BackBufferReferenceTexture2D::GetDefaultBindlessHandle() const
{
	return GetBackBufferTexture()->GetDefaultBindlessHandle();
}

#endif // D3D12RHI_USE_DUMMY_BACKBUFFER
