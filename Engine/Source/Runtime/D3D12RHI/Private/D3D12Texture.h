// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Texture.h: Implementation of D3D12 Texture
=============================================================================*/
#pragma once

#include "D3D12CommandList.h"
#include "D3D12Resources.h"
#include "D3D12View.h"

/** If true, guard texture creates with SEH to log more information about a driver crash we are seeing during texture streaming. */
#define GUARDED_TEXTURE_CREATES (PLATFORM_WINDOWS && !(UE_BUILD_SHIPPING || UE_BUILD_TEST || PLATFORM_COMPILER_CLANG))

// @todo don't make global here!
void SafeCreateTexture2D(
	FD3D12Device* pDevice,
	FD3D12Adapter* Adapter,
	const FD3D12ResourceDesc& TextureDesc,
	const D3D12_CLEAR_VALUE* ClearValue,
	FD3D12ResourceLocation* OutTexture2D,
	FD3D12BaseShaderResource* Owner,
	EPixelFormat Format,
	ETextureCreateFlags Flags,
	ED3D12Access InInitialD3D12Access,
	ED3D12Access InDefaultD3D12Access,
	const TCHAR* Name);

/** D3D12 RHI Texture class */
class FD3D12Texture : public FRHITexture, public FD3D12BaseShaderResource, public FD3D12LinkedAdapterObject<FD3D12Texture>
{
public:

	// Static helper functions
	static bool CanBe4KAligned(const FD3D12ResourceDesc& Desc, EPixelFormat UEFormat);

	FD3D12Texture() = delete;

	/** Initialization constructor. */
	FD3D12Texture(
		const FRHITextureCreateDesc& InDesc,
		class FD3D12Device* InParent)
		: FRHITexture(InDesc)
		, FD3D12BaseShaderResource(InParent) {}
	virtual ~FD3D12Texture();

	// IRefCountedObject interface overrides from FD3D12BaseShaderResource
	virtual void AddRef() const override                        { FRHIResource::AddRef(); }
	virtual FReturnedRefCountValue Release() const override	    { return FReturnedRefCountValue{ FRHIResource::Release() }; }
	virtual FReturnedRefCountValue GetRefCount() const override { return  FReturnedRefCountValue{ FRHIResource::GetRefCount() }; }

	// FRHIResource overrides 
#if RHI_ENABLE_RESOURCE_INFO
	bool GetResourceInfo(FRHIResourceInfo& OutResourceInfo) const override;
#endif

	// FRHITexture overrides
#if D3D12RHI_USE_DUMMY_BACKBUFFER
	virtual void* GetTextureBaseRHI() override	{ return this; }
#else
	virtual void* GetTextureBaseRHI() override final { return this; }
#endif
	virtual void* GetNativeResource() const override final;
	virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const override;
			
	// Accessors.
	bool IsStreamable() const									{ return EnumHasAnyFlags(GetDesc().Flags, ETextureCreateFlags::Streamable); }
	bool SkipsFastClearFinalize() const                         { return EnumHasAnyFlags(GetDesc().Flags, ETextureCreateFlags::NoFastClearFinalize); }

	void GetReadBackHeapDesc(D3D12_PLACED_SUBRESOURCE_FOOTPRINT& OutFootprint, uint32 Subresource) const;

	bool HasCreatedRTVsPerSlice() const
	{
		return bCreatedRTVsPerSlice;
	}

	bool HasRenderTargetViews() const							{ return (RenderTargetViews.Num() > 0); }
	TConstArrayView<TSharedPtr<FD3D12RenderTargetView>> GetRenderTargetViews() const { return TConstArrayView<TSharedPtr<FD3D12RenderTargetView>>(RenderTargetViews); }

	FD3D12ShaderResourceView* GetShaderResourceView() const		{ return ShaderResourceView.Get(); }
	FD3D12RenderTargetView* GetRenderTargetView(int32 MipIndex, int32 ArraySliceIndex) const;
	FD3D12DepthStencilView* GetDepthStencilView(FExclusiveDepthStencil AccessType) const { return DepthStencilViews[AccessType.GetIndex()].Get(); }

#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	bool GetRequiresTypelessResourceDiscardWorkaround() const { return bRequiresTypelessResourceDiscardWorkaround; }
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
		
	// Setup functionality
	void UploadInitialData(FRHICommandListBase& RHICmdList, FD3D12ResourceLocation&& SourceLocation, ED3D12Access InDestinationD3D12Access);
	void CreateViews(FD3D12Texture* FirstLinkedObject);
	void SetCreatedRTVsPerSlice(bool Value, int32 InRTVArraySize)
	{
		bCreatedRTVsPerSlice = Value;
		RTVArraySizePerMip = InRTVArraySize;
	}

	void SetNumRTVs(int32 Num)
	{
		RenderTargetViews.SetNum(Num);
	}

	void EmplaceRTV(D3D12_RENDER_TARGET_VIEW_DESC const& RTVDesc, int32 Index, FD3D12Texture* FirstLinkedObject)
	{
		check(RenderTargetViews.IsValidIndex(Index));
		check(!RenderTargetViews[Index]);

		RenderTargetViews[Index] = MakeShared<FD3D12RenderTargetView>(GetParentDevice(), FirstLinkedObject ? FirstLinkedObject->RenderTargetViews[Index].Get() : nullptr);
		RenderTargetViews[Index]->CreateView(this, RTVDesc);
	}

	void EmplaceDSV(D3D12_DEPTH_STENCIL_VIEW_DESC const& DSVDesc, int32 Index, FD3D12Texture* FirstLinkedObject)
	{
		check(Index < FExclusiveDepthStencil::MaxIndex);
		check(!DepthStencilViews[Index]);

		DepthStencilViews[Index] = MakeShared<FD3D12DepthStencilView>(GetParentDevice(), FirstLinkedObject ? FirstLinkedObject->DepthStencilViews[Index].Get() : nullptr);
		DepthStencilViews[Index]->CreateView(this, DSVDesc);
	}

	void EmplaceSRV(D3D12_SHADER_RESOURCE_VIEW_DESC const& SRVDesc, FD3D12Texture* FirstLinkedObject)
	{
		check(!ShaderResourceView);

		const ERHIDescriptorType DescriptorType = RHIDescriptorTypeFromD3D12ViewDesc(SRVDesc);

		FD3D12ShaderResourceView::EFlags Flags = SkipsFastClearFinalize()
			? FD3D12ShaderResourceView::EFlags::SkipFastClearFinalize
			: FD3D12ShaderResourceView::EFlags::None;

		ShaderResourceView = MakeShared<FD3D12ShaderResourceView>(GetParentDevice(), FirstLinkedObject ? FirstLinkedObject->ShaderResourceView.Get() : nullptr, DescriptorType);
		ShaderResourceView->CreateView(this, SRVDesc, Flags);
	}

	// Locking/update functions
	FRHILockTextureResult Lock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments);
	void Unlock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments);

	void UpdateTexture2D(FRHICommandListBase& RHICmdList, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData);
	void UpdateTexture(FD3D12CommandContext& Context, uint32 MipIndex, uint32 DestX, uint32 DestY, uint32 DestZ, const D3D12_TEXTURE_COPY_LOCATION& SourceCopyLocation, FD3D12Resource* SourceResource);

	void CopyTextureRegion(FD3D12CommandContext& Context, uint32 DestX, uint32 DestY, uint32 DestZ, FD3D12Texture* SourceTexture, const D3D12_BOX& SourceBox);

#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
	// Resource aliasing
	void AliasResources(FD3D12Texture* Texture);
	void SetAliasingSource(FTextureRHIRef& SourceTextureRHI)
	{
		AliasingSourceTexture = SourceTextureRHI;
	}
#endif

	// Staging buffer reuse
	void ReuseStagingBuffer(TUniquePtr<FD3D12LockedResource>&& LockedResource, uint32 Subresource);

protected:
	// A shader resource view of the texture.
	TSharedPtr<FD3D12ShaderResourceView> ShaderResourceView;

	// Set when RTVs are created for each slice - TexCreate_TargetArraySlicesIndependently for TextureArrays & Cubemaps
	bool bCreatedRTVsPerSlice{ false };

#if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND
	bool bRequiresTypelessResourceDiscardWorkaround = false;
#endif // #if PLATFORM_REQUIRES_TYPELESS_RESOURCE_DISCARD_WORKAROUND

	// Number of RTVs per mip when stored per slice
	int32 RTVArraySizePerMip{};

	// A render targetable view of the texture.
	TArray<TSharedPtr<FD3D12RenderTargetView>, TInlineAllocator<1>> RenderTargetViews;

	// A depth-stencil targetable view of the texture.
	TSharedPtr<FD3D12DepthStencilView> DepthStencilViews[FExclusiveDepthStencil::MaxIndex];

	// Data for each subresource while texture is locked
	TMap<uint32, TUniquePtr<FD3D12LockedResource>> LockedMap;

	// Cached footprint size of first resource - optimization
	mutable TUniquePtr<D3D12_PLACED_SUBRESOURCE_FOOTPRINT> FirstSubresourceFootprint;

#if D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES
	// Source texture when aliased
	FTextureRHIRef AliasingSourceTexture;
#endif
};

inline FD3D12RenderTargetView* FD3D12Texture::GetRenderTargetView(int32 MipIndex, int32 ArraySliceIndex) const
{
	int32 ArrayIndex = MipIndex;

	if (bCreatedRTVsPerSlice)
	{
		check(ArraySliceIndex >= 0);
		ArrayIndex = MipIndex * RTVArraySizePerMip + ArraySliceIndex;
		check(ArrayIndex < RenderTargetViews.Num());
	}
	else
	{
		// Catch attempts to use a specific slice without having created the texture to support it
		check(ArraySliceIndex == -1 || ArraySliceIndex == 0);
	}

	if (ArrayIndex < RenderTargetViews.Num())
	{
		return RenderTargetViews[ArrayIndex].Get();
	}
	return 0;
}

template<>
struct TD3D12ResourceTraits<FRHITexture>
{
	typedef FD3D12Texture TConcreteType;
};

class FD3D12Viewport;

#if D3D12RHI_USE_DUMMY_BACKBUFFER
class FD3D12BackBufferReferenceTexture2D : public FD3D12Texture
{
public:
	FD3D12BackBufferReferenceTexture2D(const FRHITextureCreateDesc& InDesc, FD3D12Viewport* InViewPort, FD3D12Device* InDevice)
		: FD3D12Texture(InDesc, InDevice)
		, Viewport(InViewPort)
	{
	}

	FD3D12Viewport* GetViewPort() const { return Viewport; }

	FRHITexture* GetBackBufferTexture() const;
	virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const override;

	// FRHITexture overrides
	virtual void* GetTextureBaseRHI() override { return (void*)GetBackBufferTexture(); }

private:
	FD3D12Viewport* const Viewport;
};
#endif

/** Given a pointer to a RHI texture that was created by the D3D12 RHI, returns a pointer to the FD3D12Texture it encapsulates. */
FORCEINLINE FD3D12Texture* GetD3D12TextureFromRHITexture(FRHITexture* Texture)
{
	if (!Texture)
	{
		return NULL;
	}
	
	FRHITexture* RHITexture = Texture;
	FD3D12Texture* Result((FD3D12Texture*)RHITexture->GetTextureBaseRHI());
	
	return Result;
}

FORCEINLINE FD3D12Texture* GetD3D12TextureFromRHITexture(FRHITexture* Texture, uint32 GPUIndex)
{
	FD3D12Texture* Result = GetD3D12TextureFromRHITexture(Texture);
	if (Result != nullptr)
	{
		Result = Result->GetLinkedObject(GPUIndex);
		check(Result);
		return Result;
	}
	else
	{
		return Result;
	}
}

namespace FD3D12TextureStats
{
	// Note: This function can be called from many different threads
	// @param TextureSize >0 to allocate, <0 to deallocate
	// @param b3D true:3D, false:2D or cube map
	// @param bStreamable true:Streamable, false:not streamable
	void UpdateD3D12TextureStats(FD3D12Texture& Texture, const FD3D12ResourceDesc& ResourceDesc, const FRHITextureDesc& TextureDesc, uint64 TextureSize, bool bNewTexture, bool bAllocating);

	void D3D12TextureAllocated(FD3D12Texture& Texture);
	void D3D12TextureDeleted(FD3D12Texture& Texture);
};

namespace UE::D3D12RHI::TextureUtils
{
	// Fills in the ClearValue argument based on the ResourceDesc and CreateDesc flags.
	D3D12_CLEAR_VALUE* FillClearValue(D3D12_CLEAR_VALUE& ClearValue, const FD3D12ResourceDesc& ResourceDesc, const FRHITextureDesc& TextureDesc);

	void ReconcileInitialState(FRHICommandListBase& RHICmdList, FD3D12Texture* Texture, ED3D12Access CurrentD3D12Access, ED3D12Access NeededD3D12Access, bool bNeedsDiscard);

	uint64 CalculateResourceSize(FD3D12Texture* Texture);
	void CopyBulkData(void* UploadMemory, uint64 UploadMemorySize, FD3D12Texture* Texture, FResourceBulkDataInterface* BulkData);
	D3D12_PLACED_SUBRESOURCE_FOOTPRINT GetInitializerSubresourceFootprint(FD3D12Texture* Texture, uint32 Subresource, uint32& NumRows);
}
