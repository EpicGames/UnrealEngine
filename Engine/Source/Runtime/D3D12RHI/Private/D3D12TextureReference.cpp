// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12TextureReference.h"

FD3D12RHITextureReference::FD3D12RHITextureReference(FD3D12Device* InDevice, FD3D12Texture* InReferencedTexture, FD3D12RHITextureReference* FirstLinkedObject)
	: FD3D12DeviceChild(InDevice)
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, FRHITextureReference(InReferencedTexture, FirstLinkedObject ? FirstLinkedObject->BindlessHandle : InDevice->GetBindlessDescriptorAllocator().AllocateDescriptor(ERHIDescriptorType::TextureSRV))
#else
	, FRHITextureReference(InReferencedTexture)
#endif
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		InReferencedTexture->AddRenameListener(this);

		FD3D12ShaderResourceView* View = InReferencedTexture->GetShaderResourceView();
		ReferencedDescriptorVersion = View->GetOfflineCpuHandle().GetVersion();

		InDevice->GetBindlessDescriptorManager().InitializeDescriptor(BindlessHandle, View);
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
}

FD3D12RHITextureReference::~FD3D12RHITextureReference()
{
	check(!HasListeners());

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		FD3D12DynamicRHI::ResourceCast(GetReferencedTexture())->RemoveRenameListener(this);

		// Bindless handle is shared, head link object handles freeing
		if (IsHeadLink())
		{
			GetParentDevice()->GetBindlessDescriptorManager().DeferredFreeFromDestructor(BindlessHandle);
		}
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
}

void FD3D12RHITextureReference::SwitchToNewTexture(const FD3D12ContextArray& Contexts, FD3D12Texture* InNewTexture)
{
	FD3D12Texture* CurrentTexture = FD3D12DynamicRHI::ResourceCast(GetReferencedTexture());
	FD3D12Texture* NewTexture = InNewTexture ? InNewTexture : FD3D12DynamicRHI::ResourceCast(FRHITextureReference::GetDefaultTexture());
	if (CurrentTexture != NewTexture)
	{
		NotifyListeners(Contexts, CurrentTexture, NewTexture);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		FD3D12ShaderResourceView* NewView = NewTexture->GetShaderResourceView();
		const FD3D12OfflineDescriptor NewViewDescriptor = NewView->GetOfflineCpuHandle();

		if (CurrentTexture != NewTexture)
		{
			if (CurrentTexture)
			{
				CurrentTexture->RemoveRenameListener(this);
			}

			NewTexture->AddRenameListener(this);

			GetParentDevice()->GetBindlessDescriptorManager().UpdateDescriptor(Contexts, BindlessHandle, NewView);
		}
		else if (NewViewDescriptor.GetVersion() != ReferencedDescriptorVersion)
		{
			GetParentDevice()->GetBindlessDescriptorManager().UpdateDescriptor(Contexts, BindlessHandle, NewView);
		}

		ReferencedDescriptorVersion = NewViewDescriptor.GetVersion();
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	SetReferencedTexture(InNewTexture);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
void FD3D12RHITextureReference::ResourceRenamed(const FD3D12ContextArray& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation)
{
	if (ensure(BindlessHandle.IsValid()))
	{
		FD3D12Texture* RenamedTexture = static_cast<FD3D12Texture*>(InRenamedResource);
		checkSlow(RenamedTexture == ReferencedTexture);

		FD3D12ShaderResourceView* RenamedTextureView = RenamedTexture->GetShaderResourceView();
		ReferencedDescriptorVersion = RenamedTextureView->GetOfflineCpuHandle().GetVersion();

		GetParentDevice()->GetBindlessDescriptorManager().UpdateDescriptor(Contexts, BindlessHandle, RenamedTextureView);
	}
}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

FTextureReferenceRHIRef FD3D12DynamicRHI::RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture)
{
	FRHITexture* ReferencedTexture = InReferencedTexture ? InReferencedTexture : FRHITextureReference::GetDefaultTexture();

	FD3D12Adapter* Adapter = &GetAdapter();
	return Adapter->CreateLinkedObject<FD3D12RHITextureReference>(FRHIGPUMask::All(), [ReferencedTexture](FD3D12Device* Device, FD3D12RHITextureReference* FirstLinkedObject)
	{
		return new FD3D12RHITextureReference(Device, ResourceCast(ReferencedTexture, Device->GetGPUIndex()), FirstLinkedObject);
	});
}

void FD3D12DynamicRHI::RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* InTextureReference, FRHITexture* InNewTexture)
{
	// Workaround for a crash bug where FRHITextureReferences are deleted before this command is executed on the RHI thread.
	// Take a reference on the FRHITextureReference object to keep it alive.
	TRefCountPtr<FD3D12RHITextureReference> TextureReferenceRef = ResourceCast(InTextureReference);
	FD3D12Texture* NewTexture = ResourceCast(InNewTexture ? InNewTexture : FRHITextureReference::GetDefaultTexture());

	RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FD3D12DynamicRHI::RHIUpdateTextureReference"),
	[TextureReference = MoveTemp(TextureReferenceRef), NewTexture](const FD3D12ContextArray& Contexts)
	{
		for (TD3D12DualLinkedObjectIterator<FD3D12RHITextureReference, FD3D12Texture> It(TextureReference.GetReference(), NewTexture); It; ++It)
		{
			It.GetFirst()->SwitchToNewTexture(Contexts, It.GetSecond());
		}
	});
}
