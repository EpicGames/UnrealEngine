// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIPrivate.h"
#include "RHITextureReference.h"

class FD3D12RHITextureReference;

/** Resource which might needs to be notified about changes on replaced referenced textures */
struct FD3D12TextureReferenceReplaceListener
{
	virtual void TextureReplaced(FD3D12ContextArray const& Contexts, FD3D12RHITextureReference* TextureReference, FD3D12Texture* CurrentTexture, FD3D12Texture* NewTexture) = 0;
};

class FD3D12RHITextureReference : public FD3D12DeviceChild, public FRHITextureReference, public FD3D12LinkedAdapterObject<FD3D12RHITextureReference>
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, public FD3D12ShaderResourceRenameListener
#endif
{
public:
	FD3D12RHITextureReference() = delete;
	FD3D12RHITextureReference(FD3D12Device* InDevice, FD3D12Texture* InReferencedTexture, FD3D12RHITextureReference* FirstLinkedObject);
	~FD3D12RHITextureReference();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// FD3D12ShaderResourceRenameListener
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) final override;
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	void SwitchToNewTexture(FD3D12ContextArray const& Contexts, FD3D12Texture* InNewTexture);
		
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	uint32 ReferencedDescriptorVersion{};
#endif

	void AddReplaceListener(FD3D12TextureReferenceReplaceListener* InListener)
	{
		FScopeLock Lock(&TextureReplaceListenersCS);
		check(!TextureReplaceListeners.Contains(InListener));
		TextureReplaceListeners.Add(InListener);
	}

	void RemoveReplaceListener(FD3D12TextureReferenceReplaceListener* InListener)
	{
		FScopeLock Lock(&TextureReplaceListenersCS);
		uint32 Removed = TextureReplaceListeners.Remove(InListener);

		checkf(Removed == 1, TEXT("Should have exactly one registered listener during remove (same listener shouldn't registered twice and we shouldn't call this if not registered"));
	}

private:

	bool HasListeners() const
	{
		FScopeLock Lock(&TextureReplaceListenersCS);
		return TextureReplaceListeners.Num() != 0;
	}

	void NotifyListeners(FD3D12ContextArray const& Contexts, FD3D12Texture* CurrentTexture, FD3D12Texture* NewTexture)
	{
		FScopeLock Lock(&TextureReplaceListenersCS);
		for (FD3D12TextureReferenceReplaceListener* ReplaceListener : TextureReplaceListeners)
		{
			ReplaceListener->TextureReplaced(Contexts, this, CurrentTexture, NewTexture);
		}
	}

	mutable FCriticalSection TextureReplaceListenersCS;
	TArray<FD3D12TextureReferenceReplaceListener*> TextureReplaceListeners;
};

template<>
struct TD3D12ResourceTraits<FRHITextureReference>
{
	using TConcreteType = FD3D12RHITextureReference;
};
