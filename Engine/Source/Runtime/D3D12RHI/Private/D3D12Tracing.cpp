// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12Tracing.h"
#include "Trace/Trace.inl"
#include "HAL/PlatformTime.h"
#include "D3D12Device.h"
#include "D3D12Submission.h"

#if UE_RENDER_TRACING_ENABLED

namespace RenderTracing { namespace D3D12
{

RENDER_TRACE_EVENT_BEGIN(AssociateContext, Important | NoSync)
	UE_TRACE_EVENT_FIELD(uint64, PlatformContextID)
	UE_TRACE_EVENT_FIELD(uint64, RHIContextID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(PayloadCreated)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(AddPayload)
	UE_TRACE_EVENT_FIELD(uint64, PlatformContextID)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
UE_TRACE_EVENT_END()

enum ECommandListOpType
{
	CommandListOp_Open = 0,
	CommandListOp_Close = 1
};

RENDER_TRACE_EVENT_BEGIN(CommandListOp)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, PlatformCmdListID)
	UE_TRACE_EVENT_FIELD(uint8, Op)
UE_TRACE_EVENT_END()

enum EFenceOpType
{
	FenceOp_Wait = 0,
	FenceOp_Signal = 1
};

RENDER_TRACE_EVENT_BEGIN(SyncPointOp)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, SyncPointID)
	UE_TRACE_EVENT_FIELD(uint8, SyncPointType)
	UE_TRACE_EVENT_FIELD(uint8, Op)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ManualFenceOp)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, FenceID)
	UE_TRACE_EVENT_FIELD(uint64, FenceVal)
	UE_TRACE_EVENT_FIELD(uint8, Op)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ProcessSubmissionQueueEnter)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitYieldOnUnresolvedSyncPoint)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, SyncPointID)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitYieldOnUnresolvedManualFence)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, FenceID)
	UE_TRACE_EVENT_FIELD(uint64, FenceValue)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitWaitQueueFence)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint8, ExecutingQueue)
	UE_TRACE_EVENT_FIELD(uint8, WaitOnQueue)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitWaitManualFence)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint8, ExecutingQueue)
	UE_TRACE_EVENT_FIELD(uint64, FenceID)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitPlatformCommandLists)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64[], PayloadIDs)
	UE_TRACE_EVENT_FIELD(uint64[], PlatformCmdListIDs)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitSignalManualFence)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, FenceID)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitSignalQueueFence)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(SubmitResolveSyncPoint)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, PayloadID)
	UE_TRACE_EVENT_FIELD(uint64, SyncPointID)
	UE_TRACE_EVENT_FIELD(uint64, Value)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ProcessSubmissionQueueExit)
	UE_TRACE_EVENT_FIELD(uint8, Status)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ProcessInterruptQueueEnter)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(InterruptQueueFenceSignaled)
	UE_TRACE_EVENT_FIELD(uint8, Queue)
	UE_TRACE_EVENT_FIELD(uint64, PendingPayloadID)
	UE_TRACE_EVENT_FIELD(uint64, CurrentFenceValue)
	UE_TRACE_EVENT_FIELD(uint64, LastCPUSignaledFenceValue)
UE_TRACE_EVENT_END()

RENDER_TRACE_EVENT_BEGIN(ProcessInterruptQueueExit)
	UE_TRACE_EVENT_FIELD(uint8, Status)
UE_TRACE_EVENT_END()

static uint8 GetRenderTraceQueueIndex(const FD3D12Queue& Queue)
{
	// No mGPU support for now, and only one queue per type.
	switch (Queue.QueueType)
	{
	case ED3D12QueueType::Direct: return 0;
	case ED3D12QueueType::Async: return 1;
	case ED3D12QueueType::Copy: return 2;
	default: checkNoEntry(); return 0;
	}
}

void AssociateContext(const FD3D12ContextCommon* D3D12Context, const IRHIComputeContext* RHIContext)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(AssociateContext)
		<< RENDER_TRACE_VALUE(AssociateContext, PlatformContextID, reinterpret_cast<uintptr_t>(D3D12Context))
		<< RENDER_TRACE_VALUE(AssociateContext, RHIContextID, reinterpret_cast<uintptr_t>(RHIContext));
}

void PayloadCreated(const FD3D12Payload* Payload, const FD3D12Queue& Queue)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(PayloadCreated)
		<< RENDER_TRACE_VALUE(PayloadCreated, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(PayloadCreated, Queue, GetRenderTraceQueueIndex(Queue));
}

void AddPayload(const FD3D12ContextCommon* D3D12Context, const FD3D12Payload* Payload)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(AddPayload)
		<< RENDER_TRACE_VALUE(AddPayload, PlatformContextID, reinterpret_cast<uintptr_t>(D3D12Context))
		<< RENDER_TRACE_VALUE(AddPayload, PayloadID, reinterpret_cast<uintptr_t>(Payload));
}

static void CommandListOp(const FD3D12Payload* Payload, const ID3D12CommandList* D3D12CmdList, ECommandListOpType OpType)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(CommandListOp)
		<< RENDER_TRACE_VALUE(CommandListOp, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(CommandListOp, PlatformCmdListID, reinterpret_cast<uintptr_t>(D3D12CmdList))
		<< RENDER_TRACE_VALUE(CommandListOp, Op, OpType);
}

void OpenCommandList(const FD3D12Payload* Payload, const ID3D12CommandList* D3D12CmdList)
{
	CommandListOp(Payload, D3D12CmdList, CommandListOp_Open);
}

void CloseCommandList(const FD3D12Payload* Payload, const ID3D12CommandList* D3D12CmdList)
{
	CommandListOp(Payload, D3D12CmdList, CommandListOp_Close);
}

static void SyncPointOp(const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint, EFenceOpType OpType)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SyncPointOp)
		<< RENDER_TRACE_VALUE(SyncPointOp, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SyncPointOp, SyncPointID, reinterpret_cast<uintptr_t>(SyncPoint))
		<< RENDER_TRACE_VALUE(SyncPointOp, SyncPointType, static_cast<uint8>(SyncPoint->GetType()))
		<< RENDER_TRACE_VALUE(SyncPointOp, Op, OpType);
}

void WaitSyncPoint(const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint)
{
	SyncPointOp(Payload, SyncPoint, FenceOp_Wait);
}

void SignalSyncPoint(const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint)
{
	SyncPointOp(Payload, SyncPoint, FenceOp_Signal);
}

static void ManualFenceOp(const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value, EFenceOpType OpType)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(ManualFenceOp)
		<< RENDER_TRACE_VALUE(ManualFenceOp, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(ManualFenceOp, FenceID, reinterpret_cast<uintptr_t>(Fence))
		<< RENDER_TRACE_VALUE(ManualFenceOp, FenceVal, Value)
		<< RENDER_TRACE_VALUE(ManualFenceOp, Op, OpType);
}

void WaitManualFence(const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value)
{
	ManualFenceOp(Payload, Fence, Value, FenceOp_Wait);
}

void SignalManualFence(const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value)
{
	ManualFenceOp(Payload, Fence, Value, FenceOp_Signal);
}

void ProcessSubmissionQueueEnter()
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(ProcessSubmissionQueueEnter);
}

void SubmitYieldOnUnresolvedSyncPoint(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitYieldOnUnresolvedSyncPoint)
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedSyncPoint, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedSyncPoint, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedSyncPoint, SyncPointID, reinterpret_cast<uintptr_t>(SyncPoint));
}

void SubmitYieldOnUnresolvedManualFence(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitYieldOnUnresolvedManualFence)
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedManualFence, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedManualFence, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedManualFence, FenceID, reinterpret_cast<uintptr_t>(Fence))
		<< RENDER_TRACE_VALUE(SubmitYieldOnUnresolvedManualFence, FenceValue, Value);
}

void SubmitWaitQueueFence(const FD3D12Payload* Payload, const FD3D12Queue& ExecutingQueue, const FD3D12Queue& WaitOnQueue, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitWaitQueueFence)
		<< RENDER_TRACE_VALUE(SubmitWaitQueueFence, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitWaitQueueFence, ExecutingQueue, GetRenderTraceQueueIndex(ExecutingQueue))
		<< RENDER_TRACE_VALUE(SubmitWaitQueueFence, WaitOnQueue, GetRenderTraceQueueIndex(WaitOnQueue))
		<< RENDER_TRACE_VALUE(SubmitWaitQueueFence, Value, Value);
}

void SubmitWaitManualFence(const FD3D12Payload* Payload, const FD3D12Queue& ExecutingQueue, const ID3D12Fence* Fence, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitWaitManualFence)
		<< RENDER_TRACE_VALUE(SubmitWaitManualFence, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitWaitManualFence, ExecutingQueue, GetRenderTraceQueueIndex(ExecutingQueue))
		<< RENDER_TRACE_VALUE(SubmitWaitManualFence, FenceID, reinterpret_cast<uintptr_t>(Fence))
		<< RENDER_TRACE_VALUE(SubmitWaitManualFence, Value, Value);
}

void SubmitPlatformCommandLists(const FD3D12Queue& Queue, TConstArrayView<FD3D12Payload*> Payloads, TConstArrayView<ID3D12CommandList*> CmdLists)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitPlatformCommandLists)
		<< RENDER_TRACE_VALUE(SubmitPlatformCommandLists, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_ARRAY_VALUE(SubmitPlatformCommandLists, PayloadIDs, Payloads, uint64)
		<< RENDER_TRACE_ARRAY_VALUE(SubmitPlatformCommandLists, PlatformCmdListIDs, CmdLists, uint64);
}

void SubmitSignalManualFence(const FD3D12Payload* Payload, const FD3D12Queue& Queue, const ID3D12Fence* Fence, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitSignalManualFence)
		<< RENDER_TRACE_VALUE(SubmitSignalManualFence, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitSignalManualFence, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(SubmitSignalManualFence, FenceID, reinterpret_cast<uintptr_t>(Fence))
		<< RENDER_TRACE_VALUE(SubmitSignalManualFence, Value, Value);
}

void SubmitSignalQueueFence(const FD3D12Payload* Payload, const FD3D12Queue& Queue, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitSignalQueueFence)
		<< RENDER_TRACE_VALUE(SubmitSignalQueueFence, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitSignalQueueFence, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(SubmitSignalQueueFence, Value, Value);
}

void SubmitResolveSyncPoint(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint, uint64 Value)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(SubmitResolveSyncPoint)
		<< RENDER_TRACE_VALUE(SubmitResolveSyncPoint, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(SubmitResolveSyncPoint, PayloadID, reinterpret_cast<uintptr_t>(Payload))
		<< RENDER_TRACE_VALUE(SubmitResolveSyncPoint, SyncPointID, reinterpret_cast<uintptr_t>(SyncPoint))
		<< RENDER_TRACE_VALUE(SubmitResolveSyncPoint, Value, Value);
}

void ProcessSubmissionQueueExit(uint8 Status)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(ProcessSubmissionQueueExit)
		<< RENDER_TRACE_VALUE(ProcessSubmissionQueueExit, Status, Status);
}

void ProcessInterruptQueueEnter()
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(ProcessInterruptQueueEnter);
}

void InterruptQueueFenceSignaled(const FD3D12Queue& Queue, FD3D12Payload* PendingPayload, uint64 CurrentFenceValue, uint64 LastCPUSignaledFenceValue)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(InterruptQueueFenceSignaled)
		<< RENDER_TRACE_VALUE(InterruptQueueFenceSignaled, Queue, GetRenderTraceQueueIndex(Queue))
		<< RENDER_TRACE_VALUE(InterruptQueueFenceSignaled, PendingPayloadID, reinterpret_cast<uintptr_t>(PendingPayload))
		<< RENDER_TRACE_VALUE(InterruptQueueFenceSignaled, CurrentFenceValue, CurrentFenceValue)
		<< RENDER_TRACE_VALUE(InterruptQueueFenceSignaled, LastCPUSignaledFenceValue, LastCPUSignaledFenceValue);
}

void ProcessInterruptQueueExit(uint8 Status)
{
	if (!IsEnabled(ERenderTracingChannels::RHISubmission))
	{
		return;
	}

	RENDER_TRACE_LOG(ProcessInterruptQueueExit)
		<< RENDER_TRACE_VALUE(ProcessInterruptQueueExit, Status, Status);
}

} }
#endif
