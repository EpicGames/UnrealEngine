// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderTracing.h"

#if UE_RENDER_TRACING_ENABLED

class IRHIComputeContext;
class FD3D12ContextCommon;
class FD3D12Queue;
struct FD3D12Payload;
class FD3D12SyncPoint;
struct ID3D12CommandList;
struct ID3D12Fence;

namespace RenderTracing { namespace D3D12
{

void AssociateContext(const FD3D12ContextCommon* D3D12Context, const IRHIComputeContext* RHIContext);
void PayloadCreated(const FD3D12Payload* Payload, const FD3D12Queue& Queue);
void AddPayload(const FD3D12ContextCommon* D3D12Context, const FD3D12Payload* Payload);
void OpenCommandList(const FD3D12Payload* Payload, const ID3D12CommandList* D3D12CmdList);
void CloseCommandList(const FD3D12Payload* Payload, const ID3D12CommandList* D3D12CmdList);
void WaitSyncPoint(const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint);
void SignalSyncPoint(const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint);
void WaitManualFence(const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value);
void SignalManualFence(const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value);

void ProcessSubmissionQueueEnter();
void SubmitYieldOnUnresolvedSyncPoint(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint);
void SubmitYieldOnUnresolvedManualFence(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const ID3D12Fence* Fence, uint64 Value);
void SubmitWaitQueueFence(const FD3D12Payload* Payload, const FD3D12Queue& ExecutingQueue, const FD3D12Queue& WaitOnQueue, uint64 Value);
void SubmitWaitManualFence(const FD3D12Payload* Payload, const FD3D12Queue& ExecutingQueue, const ID3D12Fence* Fence, uint64 Value);
void SubmitPlatformCommandLists(const FD3D12Queue& Queue, TConstArrayView<FD3D12Payload*> Payloads, TConstArrayView<ID3D12CommandList*> CmdLists);
void SubmitSignalManualFence(const FD3D12Payload* Payload, const FD3D12Queue& Queue, const ID3D12Fence* Fence, uint64 Value);
void SubmitSignalQueueFence(const FD3D12Payload* Payload, const FD3D12Queue& Queue, uint64 Value);
void SubmitResolveSyncPoint(const FD3D12Queue& Queue, const FD3D12Payload* Payload, const FD3D12SyncPoint* SyncPoint, uint64 Value);
void ProcessSubmissionQueueExit(uint8 Status);

void ProcessInterruptQueueEnter();
void InterruptQueueFenceSignaled(const FD3D12Queue& Queue, FD3D12Payload* PendingPayload, uint64 CurrentFenceValue, uint64 LastCPUSignaledFenceValue);
void ProcessInterruptQueueExit(uint8 Status);

} }

#endif
