// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12TransientResourceAllocator.h"
#include "D3D12RHIPrivate.h"
#include "D3D12Stats.h"
#include "ProfilingDebugging/MemoryTrace.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

FD3D12TransientHeap::FD3D12TransientHeap(const FInitializer& Initializer, FD3D12Adapter* Adapter, FD3D12Device* Device, FRHIGPUMask VisibleNodeMask)
	: FRHITransientHeap(Initializer)
{
	const static FLazyName D3D12TransientHeapName(TEXT("FD3D12TransientHeap"));
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(D3D12TransientHeapName, D3D12TransientHeapName, NAME_None);

	D3D12_HEAP_FLAGS HeapFlags = D3D12_HEAP_FLAG_ALLOW_ALL_BUFFERS_AND_TEXTURES;

	if (Initializer.Flags != ERHITransientHeapFlags::AllowAll)
	{
		switch (Initializer.Flags)
		{
		case ERHITransientHeapFlags::AllowBuffers:
			HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS;
			break;

		case ERHITransientHeapFlags::AllowTextures:
			HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES;
			break;

		case ERHITransientHeapFlags::AllowRenderTargets:
			HeapFlags = D3D12_HEAP_FLAG_ALLOW_ONLY_RT_DS_TEXTURES;
			break;
		}
	}

	D3D12_HEAP_PROPERTIES HeapProperties = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT);
	HeapProperties.CreationNodeMask = FRHIGPUMask::FromIndex(Device->GetGPUIndex()).GetNative();
	HeapProperties.VisibleNodeMask = VisibleNodeMask.GetNative();

	D3D12_HEAP_DESC Desc = {};
	Desc.SizeInBytes = Initializer.Size;
	Desc.Properties = HeapProperties;
	Desc.Alignment = D3D12_DEFAULT_MSAA_RESOURCE_PLACEMENT_ALIGNMENT;
	Desc.Flags = HeapFlags;

	if (Adapter->IsHeapNotZeroedSupported())
	{
		Desc.Flags |= FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED;
	}

#if ENABLE_RESIDENCY_MANAGEMENT
	if (!GD3D12StartResourceResident)
	{
		Desc.Flags |= D3D12_HEAP_FLAG_CREATE_NOT_RESIDENT;
	}
#endif


	ID3D12Heap* D3DHeap = nullptr;
	{
		ID3D12Device* D3DDevice = Device->GetDevice();

		LLM_PLATFORM_SCOPE(ELLMTag::GraphicsPlatform);

		VERIFYD3D12RESULT(D3DDevice->CreateHeap(&Desc, IID_PPV_ARGS(&D3DHeap)));

#if PLATFORM_WINDOWS
		// On Windows there is no way to hook into the low level d3d allocations and frees.
		// This means that we must manually add the tracking here.
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, D3DHeap, Desc.SizeInBytes, ELLMTag::GraphicsPlatform));
		MemoryTrace_Alloc((uint64)D3DHeap, Desc.SizeInBytes, 0, EMemoryTraceRootHeap::VideoMemory);
		// Boost priority to make sure it's not paged out
		TRefCountPtr<ID3D12Device5> D3DDevice5;
		if (SUCCEEDED(D3DDevice->QueryInterface(IID_PPV_ARGS(D3DDevice5.GetInitReference()))))
		{
			Adapter->SetResidencyPriority(D3DHeap, D3D12_RESIDENCY_PRIORITY_HIGH, Device->GetGPUIndex(), Desc.SizeInBytes);
		}
#endif // PLATFORM_WINDOWS
	}

	Heap = new FD3D12Heap(Device, VisibleNodeMask);
	Heap->SetHeap(D3DHeap, TEXT("TransientResourceAllocator Backing Heap"), true, true);
	Heap->SetIsTransient(true);
	Heap->BeginTrackingResidency(Desc.SizeInBytes);

	INC_MEMORY_STAT_BY(STAT_D3D12TransientHeaps, Desc.SizeInBytes);
	INC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, Desc.SizeInBytes);
}

FD3D12TransientHeap::~FD3D12TransientHeap()
{
	if (Heap)
	{
		D3D12_HEAP_DESC Desc = Heap->GetHeapDesc();
		DEC_MEMORY_STAT_BY(STAT_D3D12TransientHeaps, Desc.SizeInBytes);
		DEC_MEMORY_STAT_BY(STAT_D3D12MemoryCurrentTotal, Desc.SizeInBytes);
#if PLATFORM_WINDOWS
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Heap->GetHeap()));
		MemoryTrace_Free((uint64)Heap->GetHeap(), EMemoryTraceRootHeap::VideoMemory);
#endif

		Heap->DeferDelete();
	}
}

TUniquePtr<FD3D12TransientHeapCache> FD3D12TransientHeapCache::Create(FD3D12Adapter* ParentAdapter)
{
	FRHITransientHeapCache::FInitializer Initializer = FRHITransientHeapCache::FInitializer::CreateDefault();

	// Tier2 hardware is able to mix resource types onto the same heap.
	Initializer.bSupportsAllHeapFlags = ParentAdapter->GetResourceHeapTier() == D3D12_RESOURCE_HEAP_TIER_2;

	return TUniquePtr<FD3D12TransientHeapCache>(new FD3D12TransientHeapCache(Initializer, ParentAdapter));
}

FD3D12TransientHeapCache::FD3D12TransientHeapCache(const FRHITransientHeapCache::FInitializer& Initializer, FD3D12Adapter* ParentAdapter)
	: FRHITransientHeapCache(Initializer)
	, FD3D12AdapterChild(ParentAdapter)
{}

FRHITransientHeap* FD3D12TransientHeapCache::CreateHeap(const FRHITransientHeap::FInitializer& HeapInitializer)
{
	// If heap is flagged for NNE buffers, make it visible on first GPU only.  Required by DirectML.
	FRHIGPUMask VisibleNodeMask = HeapInitializer.Flags == ERHITransientHeapFlags::AllowNNEBuffers ? FRHIGPUMask::GPU0() : FRHIGPUMask::All();

	return GetParentAdapter()->CreateLinkedObject<FD3D12TransientHeap>(VisibleNodeMask, [&](FD3D12Device* Device, FD3D12TransientHeap* FirstLinkedObject)
	{
		return new FD3D12TransientHeap(HeapInitializer, GetParentAdapter(), Device, VisibleNodeMask);
	});
}

FD3D12TransientResourceHeapAllocator::FD3D12TransientResourceHeapAllocator(FD3D12TransientHeapCache& InHeapCache)
	: FRHITransientResourceHeapAllocator(InHeapCache)
	, FD3D12AdapterChild(InHeapCache.GetParentAdapter())
	, AllocationInfoQueryDevice(GetParentAdapter()->GetDevice(0))
{}

FRHITransientTexture* FD3D12TransientResourceHeapAllocator::CreateTexture(const FRHITextureCreateInfo& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	FD3D12DynamicRHI* DynamicRHI = FD3D12DynamicRHI::GetD3DRHI();

	FD3D12ResourceDesc Desc = DynamicRHI->GetResourceDesc(InCreateInfo);
	D3D12_RESOURCE_ALLOCATION_INFO Info = AllocationInfoQueryDevice->GetResourceAllocationInfo(Desc);

	Info.Alignment = FMath::Max<uint32>(D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT, Info.Alignment);

	return CreateTextureInternal(InCreateInfo, InDebugName, Fences, Info.SizeInBytes, Info.Alignment,
		[&](const FRHITransientHeap::FResourceInitializer& Initializer)
	{
		FD3D12TransientHeap& Heap = static_cast<FD3D12TransientHeap&>(Initializer.Heap);

		return CreateTransientResource<FRHITransientTexture>([DynamicRHI, Adapter = GetParentAdapter(), &Heap, Allocation = Initializer.Allocation, Desc, InCreateInfo, InDebugName]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AllocatePlacedTexture);
			FResourceAllocatorAdapter ResourceAllocatorAdapter(Adapter, Heap, Allocation, Desc);
			FRHITextureCreateDesc CreateDesc(InCreateInfo, ERHIAccess::Discard, InDebugName);

			FD3D12Texture* Texture = DynamicRHI->CreateD3D12Texture(CreateDesc, &ResourceAllocatorAdapter);
			return FRHITransientResource::FResourceTaskResult{ Texture, ResourceAllocatorAdapter.GpuVirtualAddress };

		}, Initializer.Hash, Info.SizeInBytes, InCreateInfo);
	});
}

void FD3D12TransientResourceHeapAllocator::FResourceAllocatorAdapter::AllocateResource(
	uint32 GPUIndex,
	D3D12_HEAP_TYPE,
	const FD3D12ResourceDesc& InDesc,
	uint64 InSize,
	uint32 /*InAllocationAlignment*/,
	ED3D12Access InInitialD3D12Access,
	ED3D12ResourceStateMode InResourceStateMode,
	ED3D12Access InDefaultD3D12Access,
	const D3D12_CLEAR_VALUE* InClearValue,
	const TCHAR* InName,
	FD3D12ResourceLocation& ResourceLocation)
{
	// The D3D12_RESOURCE_DESC's are built in two different functions right now. This checks that they actually match what we expect.
#if DO_CHECK
	{
		CD3DX12_RESOURCE_DESC CreatedDesc(InDesc);
		CD3DX12_RESOURCE_DESC DerivedDesc(Desc);
		check(CreatedDesc == DerivedDesc);
	}
#endif

	FD3D12Resource* NewResource = nullptr;
	FD3D12Adapter* Adapter = GetParentAdapter();
	VERIFYD3D12RESULT(Adapter->CreatePlacedResource(
		InDesc,
		Heap.GetLinkedObject(GPUIndex)->Get(), 
		Allocation.Offset,
		InInitialD3D12Access,
		InResourceStateMode,
		InDefaultD3D12Access,
		InClearValue,
		&NewResource,
		InName));

	check(!ResourceLocation.IsValid());
	ResourceLocation.AsHeapAliased(NewResource);
	ResourceLocation.SetSize(InSize);
	ResourceLocation.SetTransient(true);

	GpuVirtualAddress = NewResource->GetGPUVirtualAddress();

#if TRACK_RESOURCE_ALLOCATIONS
	if (Adapter->IsTrackingAllAllocations())
	{
		bool bCollectCallstack = false;
		Adapter->TrackAllocationData(&ResourceLocation, Allocation.Size, bCollectCallstack);
	}
#endif
}

FRHITransientBuffer* FD3D12TransientResourceHeapAllocator::CreateBuffer(const FRHIBufferDesc& InCreateInfo, const TCHAR* InDebugName, const FRHITransientAllocationFences& Fences)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(InDebugName, InCreateInfo)
		.SetInitialState(ERHIAccess::Discard);

	D3D12_RESOURCE_DESC Desc{};
	uint32 Alignment{};
	FD3D12Buffer::GetResourceDescAndAlignment(CreateDesc, Desc, Alignment);

	Alignment = D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT;
	uint64 Size = Align(Desc.Width, Alignment);

	return CreateBufferInternal(CreateDesc, InDebugName, Fences, Size, Alignment,
		[&](const FRHITransientHeap::FResourceInitializer& Initializer)
	{
		FD3D12TransientHeap& Heap = static_cast<FD3D12TransientHeap&>(Initializer.Heap);

		return CreateTransientResource<FRHITransientBuffer>([Adapter = GetParentAdapter(), &Heap, Allocation = Initializer.Allocation, Desc, CreateDesc, InDebugName]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AllocatePlacedBuffer);
			FResourceAllocatorAdapter ResourceAllocatorAdapter(Adapter, Heap, Allocation, Desc);
			FRHIBuffer* Buffer = FD3D12DynamicRHI::GetD3DRHI()->CreateD3D12Buffer(CreateDesc, &ResourceAllocatorAdapter);
			return FRHITransientResource::FResourceTaskResult{ Buffer, ResourceAllocatorAdapter.GpuVirtualAddress };

		}, Initializer.Hash, Size, CreateDesc);
	});
}