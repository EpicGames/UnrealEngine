// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12Resources.h"
#include "RHICoreTransientResourceAllocator.h"

class FD3D12TransientHeap final
	: public FRHITransientHeap
	, public FRefCountedObject
	, public FD3D12LinkedAdapterObject<FD3D12TransientHeap>
{
public:
	FD3D12TransientHeap(const FInitializer& Initializer, FD3D12Adapter* Adapter, FD3D12Device* Device, FRHIGPUMask VisibleNodeMask);
	~FD3D12TransientHeap();

	FD3D12Heap* Get() { return Heap; }

private:
	TRefCountPtr<FD3D12Heap> Heap;
};

class FD3D12TransientHeapCache final
	: public FRHITransientHeapCache
	, public FD3D12AdapterChild
{
public:
	static TUniquePtr<FD3D12TransientHeapCache> Create(FD3D12Adapter* ParentAdapter);

	//! FRHITransientResourceSystem Overrides
	FRHITransientHeap* CreateHeap(const FRHITransientHeap::FInitializer& Initializer) override;

private:
	FD3D12TransientHeapCache(const FInitializer& Initializer, FD3D12Adapter* ParentAdapter);
};

class FD3D12TransientResourceHeapAllocator final
	: public FRHITransientResourceHeapAllocator
	, public FD3D12AdapterChild
{
public:
	FD3D12TransientResourceHeapAllocator(FD3D12TransientHeapCache& InHeapCache);

	//! IRHITransientResourceAllocator Overrides
	bool SupportsResourceType(ERHITransientResourceType InType) const override 
	{ 
		switch (InType)
		{
		case ERHITransientResourceType::Buffer: return true;
		case ERHITransientResourceType::Texture: return true;
		default: checkNoEntry(); return false;
		}
	}

	FRHITransientTexture* CreateTexture(const FRHITextureCreateInfo& CreateInfo, const TCHAR* Name, const FRHITransientAllocationFences& Fences) override;
	FRHITransientBuffer* CreateBuffer(const FRHIBufferCreateInfo& CreateInfo, const TCHAR* Name, const FRHITransientAllocationFences& Fences) override;

private:

	// This adapter mocks the D3D12 resource allocator interface and performs placed resource creation.
	class FResourceAllocatorAdapter final : public FD3D12AdapterChild, public ID3D12ResourceAllocator
	{
	public:
		FResourceAllocatorAdapter(FD3D12Adapter* Adapter, FD3D12TransientHeap& InHeap, const FRHITransientHeapAllocation& InAllocation, const D3D12_RESOURCE_DESC& InDesc)
			: FD3D12AdapterChild(Adapter)
			, Heap(InHeap)
			, Allocation(InAllocation)
			, Desc(InDesc)
		{}

		void AllocateResource(
			uint32 GPUIndex,
			D3D12_HEAP_TYPE InHeapType,
			const FD3D12ResourceDesc& InDesc,
			uint64 InSize,
			uint32 InAllocationAlignment,
			ED3D12Access InInitialD3D12Access,
			ED3D12ResourceStateMode InResourceStateMode,
			ED3D12Access InDefaultD3D12Access,
			const D3D12_CLEAR_VALUE* InClearValue,
			const TCHAR* InName,
			FD3D12ResourceLocation& ResourceLocation) override;

		FD3D12TransientHeap& Heap;
		const FRHITransientHeapAllocation& Allocation;
		const D3D12_RESOURCE_DESC Desc;
		uint64 GpuVirtualAddress = 0;
	};

	FD3D12Device* AllocationInfoQueryDevice;
};