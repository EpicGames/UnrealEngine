// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RHIPrivate.h"
#include "ClearReplacementShaders.h"

FD3D12UnorderedAccessView::FD3D12UnorderedAccessView(FD3D12Device* InDevice, FD3D12UnorderedAccessView* FirstLinkedObject, ERHIDescriptorType InDescriptorType)
	: TD3D12View(InDevice, InDescriptorType, ERHIDescriptorHeapType::Standard, FirstLinkedObject)
{
}

void FD3D12UnorderedAccessView::UpdateResourceInfo(const FResourceInfo& InResource, const D3D12_UNORDERED_ACCESS_VIEW_DESC& InD3DViewDesc, EFlags InFlags)
{
	OffsetInBytes = 0;
	StrideInBytes = 0;

	//
	// Buffer views can apply an offset in bytes from the start of the logical resource.
	//
	// Reconstruct this value and store it for later. We'll need it if the view is renamed, 
	// to determine where the view should exist within the bounds of the new resource location.
	//
	if (InD3DViewDesc.ViewDimension == D3D12_UAV_DIMENSION_BUFFER)
	{
		StrideInBytes = InD3DViewDesc.Format == DXGI_FORMAT_UNKNOWN
			? InD3DViewDesc.Buffer.StructureByteStride
			: UE::DXGIUtilities::GetFormatSizeInBytes(InD3DViewDesc.Format);

		check(StrideInBytes > 0);

		OffsetInBytes = (InD3DViewDesc.Buffer.FirstElement * StrideInBytes) - InResource.ResourceLocation->GetOffsetFromBaseOfResource();
		check((OffsetInBytes % StrideInBytes) == 0);
	}

	//
	// UAVs optionally support a hidden counter. D3D12 requires the user to allocate this as a buffer resource.
	//
	if (EnumHasAnyFlags(InFlags, EFlags::NeedsCounter))
	{
		FD3D12Device* Device = GetParentDevice();
		const FRHIGPUMask Node = Device->GetGPUMask();

		Device->GetParentAdapter()->CreateBuffer(
			CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT, Node.GetNative(), Node.GetNative())
			, Node
			, ED3D12Access::UAVMask
			, ED3D12ResourceStateMode::MultiState
			, ED3D12Access::UAVMask
			, 4
			, CounterResource.GetInitReference()
			, TEXT("Counter")
			, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS
		);
	}
}

void FD3D12UnorderedAccessView::CreateView(FResourceInfo const& InResource, D3D12_UNORDERED_ACCESS_VIEW_DESC const& InD3DViewDesc, EFlags InFlags)
{
	UpdateResourceInfo(InResource, InD3DViewDesc, InFlags);
	TD3D12View::CreateView(InResource, InD3DViewDesc);
}

void FD3D12UnorderedAccessView::UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, const D3D12_UNORDERED_ACCESS_VIEW_DESC& InD3DViewDesc, EFlags InFlags)
{
	UpdateResourceInfo(InResource, InD3DViewDesc, InFlags);
	TD3D12View::UpdateView(Contexts, InResource, InD3DViewDesc);
}

void FD3D12UnorderedAccessView::ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation)
{
	// Buffer SRV descriptors contain offsets / GPU virtual addresses which need to be updated to match the new resource location.
	if (D3DViewDesc.ViewDimension == D3D12_UAV_DIMENSION_BUFFER)
	{
		D3DViewDesc.Buffer.FirstElement = (OffsetInBytes + InNewResourceLocation->GetOffsetFromBaseOfResource()) / StrideInBytes;
	}

	TD3D12View::ResourceRenamed(Contexts, InRenamedResource, InNewResourceLocation);
}

void FD3D12UnorderedAccessView::UpdateDescriptor()
{
	ID3D12Resource* Resource = ResourceInfo.Resource->GetResource();

	ID3D12Resource* Counter = CounterResource
		? CounterResource->GetResource()
		: nullptr;

#if !D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	const D3D12_RESOURCE_DESC* AliasResourceDesc =
		ResourceInfo.Resource->GetUAVAccessResourceDesc();

	if (AliasResourceDesc)
	{
		GetParentDevice()->CreateUnorderedAccessViewAlias(
			Resource,
			Counter,
			*AliasResourceDesc,
			D3DViewDesc,
			OfflineCpuHandle);
	}
	else
#endif
	{
		GetParentDevice()->GetDevice()->CreateUnorderedAccessView(
			Resource, 
			Counter, 
			&D3DViewDesc,
			OfflineCpuHandle
		);
	}

	OfflineCpuHandle.IncrementVersion();
}

static FD3D12UnorderedAccessView::EFlags TranslateDesc(D3D12_UNORDERED_ACCESS_VIEW_DESC& UAVDesc, FD3D12Buffer* Buffer, const FRHIViewDesc::FBufferUAV::FViewInfo& Info)
{
	FD3D12UnorderedAccessView::EFlags Flags = FD3D12UnorderedAccessView::EFlags::None;
	if (!Info.bNullView)
	{
		if (Info.bAppendBuffer || Info.bAtomicCounter)
		{
			Flags = FD3D12UnorderedAccessView::EFlags::NeedsCounter;
		}

		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_BUFFER;
		UAVDesc.Format = UE::DXGIUtilities::FindUnorderedAccessFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat));
		UAVDesc.Buffer.FirstElement = (Info.OffsetInBytes + Buffer->ResourceLocation.GetOffsetFromBaseOfResource()) / Info.StrideInBytes;
		UAVDesc.Buffer.NumElements = Info.NumElements;

		switch (Info.BufferType)
		{
		case FRHIViewDesc::EBufferType::Raw:
			UAVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
			UAVDesc.Buffer.Flags = D3D12_BUFFER_UAV_FLAG_RAW;
			break;

		case FRHIViewDesc::EBufferType::Structured:
			UAVDesc.Buffer.StructureByteStride = Info.StrideInBytes;
			break;

		case FRHIViewDesc::EBufferType::Typed:
			// Nothing more to specify
			break;

		default:
			checkNoEntry(); // unsupported / unimplemented
			break;
		}
	}

	return Flags;
}

static FD3D12UnorderedAccessView::EFlags TranslateDesc(D3D12_UNORDERED_ACCESS_VIEW_DESC& UAVDesc, FD3D12Texture* Texture, const FRHIViewDesc::FTextureUAV::FViewInfo& Info)
{
	const FRHITextureDesc& TextureDesc = Texture->GetDesc();
	check(TextureDesc.NumSamples == 1);

	DXGI_FORMAT const ViewFormat = UE::DXGIUtilities::FindUnorderedAccessFormat(DXGI_FORMAT(GPixelFormats[Info.Format].PlatformFormat));
	DXGI_FORMAT const BaseFormat = UE::DXGIUtilities::GetPlatformTextureResourceFormat(DXGI_FORMAT(GPixelFormats[TextureDesc.Format].PlatformFormat), TextureDesc.Flags);

	UAVDesc.Format = ViewFormat;

	uint32 const PlaneSlice = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(BaseFormat, ViewFormat);
	FRHIRange8 const PlaneRange(PlaneSlice, 1);

	FRHIViewDesc::EDimension ViewDimension = UE::RHICore::AdjustViewInfoDimensionForNarrowing(Info, TextureDesc);
	switch (ViewDimension)
	{
	case FRHIViewDesc::EDimension::Texture2D:
		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_TEXTURE2D;
		UAVDesc.Texture2D.MipSlice = Info.MipLevel;
		UAVDesc.Texture2D.PlaneSlice = PlaneSlice;
		break;

	case FRHIViewDesc::EDimension::TextureCube:
	case FRHIViewDesc::EDimension::TextureCubeArray:
		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_TEXTURE2DARRAY;
		UAVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First * 6;
		UAVDesc.Texture2DArray.ArraySize = Info.ArrayRange.Num * 6;
		UAVDesc.Texture2DArray.MipSlice = Info.MipLevel;
		UAVDesc.Texture2DArray.PlaneSlice = PlaneSlice;
		break;

	case FRHIViewDesc::EDimension::Texture2DArray:
		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_TEXTURE2DARRAY;
		UAVDesc.Texture2DArray.FirstArraySlice = Info.ArrayRange.First;
		UAVDesc.Texture2DArray.ArraySize = Info.ArrayRange.Num;
		UAVDesc.Texture2DArray.MipSlice = Info.MipLevel;
		UAVDesc.Texture2DArray.PlaneSlice = PlaneSlice;
		break;

	case FRHIViewDesc::EDimension::Texture3D:
		UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_TEXTURE3D;
		UAVDesc.Texture3D.FirstWSlice = 0;
		UAVDesc.Texture3D.WSize = FMath::Max(TextureDesc.Depth >> Info.MipLevel, 1);
		UAVDesc.Texture3D.MipSlice = Info.MipLevel;
		break;

	default:
		checkNoEntry();
		break;
	}

	return FD3D12UnorderedAccessView::EFlags::None;
}

void FD3D12UnorderedAccessView_RHI::CreateView()
{
	if (IsBuffer())
	{
		FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(GetBuffer());

		D3D12_UNORDERED_ACCESS_VIEW_DESC UAVDesc{};
		const EFlags CreateFlags = TranslateDesc(UAVDesc, Buffer, ViewDesc.Buffer.UAV.GetViewInfo(Buffer));

		FD3D12UnorderedAccessView::CreateView(Buffer, UAVDesc, CreateFlags);
	}
	else
	{
		FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(GetTexture());

		D3D12_UNORDERED_ACCESS_VIEW_DESC UAVDesc{};
		const EFlags CreateFlags = TranslateDesc(UAVDesc, Texture, ViewDesc.Texture.UAV.GetViewInfo(Texture));

		FD3D12UnorderedAccessView::CreateView(Texture, UAVDesc, CreateFlags);
	}
}

void FD3D12UnorderedAccessView_RHI::UpdateView(FD3D12ContextArray const& Contexts)
{
	if (IsBuffer())
	{
		FD3D12Buffer* Buffer = FD3D12DynamicRHI::ResourceCast(GetBuffer());

		D3D12_UNORDERED_ACCESS_VIEW_DESC UAVDesc{};
		const EFlags CreateFlags = TranslateDesc(UAVDesc, Buffer, ViewDesc.Buffer.UAV.GetViewInfo(Buffer));

		FD3D12UnorderedAccessView::UpdateView(Contexts, Buffer, UAVDesc, CreateFlags);
	}
	else
	{
		FD3D12Texture* Texture = FD3D12DynamicRHI::ResourceCast(GetTexture());

		D3D12_UNORDERED_ACCESS_VIEW_DESC UAVDesc{};
		const EFlags CreateFlags = TranslateDesc(UAVDesc, Texture, ViewDesc.Texture.UAV.GetViewInfo(Texture));

		FD3D12UnorderedAccessView::UpdateView(Contexts, Texture, UAVDesc, CreateFlags);
	}
}

FD3D12UnorderedAccessView_RHI::FD3D12UnorderedAccessView_RHI(FD3D12Device* InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc, FD3D12UnorderedAccessView_RHI* FirstLinkedObject)
	: FRHIUnorderedAccessView(InResource, InViewDesc)
	, FD3D12UnorderedAccessView(InDevice, FirstLinkedObject, RHIDescriptorTypeFromViewDesc(InViewDesc))
{
}

// -----------------------------------------------------------------------------------------------------
//
//                                         RHI Create Functions                                             
//
// -----------------------------------------------------------------------------------------------------

FUnorderedAccessViewRHIRef FD3D12DynamicRHI::RHICreateUnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	FRHIGPUMask RelevantGPUs = ViewDesc.IsBuffer()
		? FD3D12DynamicRHI::ResourceCast(static_cast<FRHIBuffer* >(Resource))->GetLinkedObjectsGPUMask()
		: FD3D12DynamicRHI::ResourceCast(static_cast<FRHITexture*>(Resource))->GetLinkedObjectsGPUMask();

	FD3D12UnorderedAccessView_RHI* View = GetAdapter().CreateLinkedObject<FD3D12UnorderedAccessView_RHI>(RelevantGPUs, [&](FD3D12Device* Device, FD3D12UnorderedAccessView_RHI* FirstLinkedObject)
	{
		FRHIViewableResource* TargetResource = ViewDesc.IsBuffer()
			? static_cast<FRHIViewableResource*>(FD3D12DynamicRHI::ResourceCast(static_cast<FRHIBuffer* >(Resource), Device->GetGPUIndex()))
			: static_cast<FRHIViewableResource*>(FD3D12DynamicRHI::ResourceCast(static_cast<FRHITexture*>(Resource), Device->GetGPUIndex()));

		return new FD3D12UnorderedAccessView_RHI(Device, TargetResource, ViewDesc, FirstLinkedObject);
	});

	View->CreateViews(RHICmdList);
	return View;
}



// -----------------------------------------------------------------------------------------------------
//
//                                         UAV Clear Functions                                          
//
// -----------------------------------------------------------------------------------------------------

void FD3D12CommandContext::ClearUAV(TRHICommandList_RecursiveHazardous<FD3D12CommandContext>& RHICmdList, FD3D12UnorderedAccessView_RHI* UnorderedAccessView, const void* ClearValues, bool bFloat)
{
	const D3D12_RESOURCE_DESC& ResourceDesc = static_cast<FD3D12UnorderedAccessView*>(UnorderedAccessView)->GetResource()->GetDesc();
	const D3D12_UNORDERED_ACCESS_VIEW_DESC& UAVDesc = UnorderedAccessView->GetD3DDesc();

	// Only structured buffers can have an unknown format
	check(UAVDesc.ViewDimension == D3D12_UAV_DIMENSION_BUFFER || UAVDesc.Format != DXGI_FORMAT_UNKNOWN);

	EClearReplacementValueType ValueType = bFloat ? EClearReplacementValueType::Float : EClearReplacementValueType::Uint32;
	switch (UAVDesc.Format)
	{
	case DXGI_FORMAT_R32G32B32A32_SINT:
	case DXGI_FORMAT_R32G32B32_SINT:
	case DXGI_FORMAT_R16G16B16A16_SINT:
	case DXGI_FORMAT_R32G32_SINT:
	case DXGI_FORMAT_R8G8B8A8_SINT:
	case DXGI_FORMAT_R16G16_SINT:
	case DXGI_FORMAT_R32_SINT:
	case DXGI_FORMAT_R8G8_SINT:
	case DXGI_FORMAT_R16_SINT:
	case DXGI_FORMAT_R8_SINT:
		ValueType = EClearReplacementValueType::Int32;
		break;

	case DXGI_FORMAT_R32G32B32A32_UINT:
	case DXGI_FORMAT_R32G32B32_UINT:
	case DXGI_FORMAT_R16G16B16A16_UINT:
	case DXGI_FORMAT_R32G32_UINT:
	case DXGI_FORMAT_D32_FLOAT_S8X24_UINT:
	case DXGI_FORMAT_X32_TYPELESS_G8X24_UINT:
	case DXGI_FORMAT_R10G10B10A2_UINT:
	case DXGI_FORMAT_R8G8B8A8_UINT:
	case DXGI_FORMAT_R16G16_UINT:
	case DXGI_FORMAT_R32_UINT:
	case DXGI_FORMAT_D24_UNORM_S8_UINT:
	case DXGI_FORMAT_X24_TYPELESS_G8_UINT:
	case DXGI_FORMAT_R8G8_UINT:
	case DXGI_FORMAT_R16_UINT:
	case DXGI_FORMAT_R8_UINT:
		ValueType = EClearReplacementValueType::Uint32;
		break;
	}

	ensureMsgf((UAVDesc.Format == DXGI_FORMAT_UNKNOWN) || (bFloat == (ValueType == EClearReplacementValueType::Float)), TEXT("Attempt to clear a UAV using the wrong RHIClearUAV function. Float vs Integer mismatch."));

	if (UAVDesc.ViewDimension == D3D12_UAV_DIMENSION_BUFFER)
	{
		if (UAVDesc.Format == DXGI_FORMAT_UNKNOWN || (UAVDesc.Buffer.Flags & D3D12_BUFFER_UAV_FLAG_RAW) != 0)
		{
			// Structured buffer.
			RHICmdList.RunOnContext([UnorderedAccessView, ClearValues](FD3D12CommandContext& Context)
			{
				const D3D12_UNORDERED_ACCESS_VIEW_DESC& UAVDesc = UnorderedAccessView->GetD3DDesc();

				// Alias the structured buffer with an R32_UINT UAV to perform the clear.
				// We construct a temporary UAV on the offline heap, copy it to the online heap, and then call ClearUnorderedAccessViewUint.

				FD3D12Device* ParentDevice = Context.GetParentDevice();
				ID3D12Device* Device = ParentDevice->GetDevice();

				D3D12_UNORDERED_ACCESS_VIEW_DESC R32UAVDesc{};
				if ((UAVDesc.Buffer.Flags & D3D12_BUFFER_UAV_FLAG_RAW) != 0)
				{
					// Raw UAVs will already be setup correctly for us.
					check(UAVDesc.Format == DXGI_FORMAT_R32_TYPELESS);
					R32UAVDesc = UAVDesc;
				}
				else
				{
					// Structured buffer stride must be a multiple of sizeof(uint32)
					check(UAVDesc.Buffer.StructureByteStride % sizeof(uint32) == 0);
					uint32 DwordsPerElement = UAVDesc.Buffer.StructureByteStride / sizeof(uint32);

					R32UAVDesc.Format = DXGI_FORMAT_R32_UINT;
					R32UAVDesc.ViewDimension = D3D12_UAV_DIMENSION_BUFFER;
					R32UAVDesc.Buffer.FirstElement = UAVDesc.Buffer.FirstElement * DwordsPerElement;
					R32UAVDesc.Buffer.NumElements = UAVDesc.Buffer.NumElements * DwordsPerElement;
				}

				Context.OpenIfNotAlready();

				// The DescriptorType argument specifies which descriptor heap to allocate the view's descriptor from.
				// If we won't actually need the descriptor (ie bindless is disabled), we can signal to not allocate it in the first place by specifying an Invalid descriptor type.
				ERHIDescriptorType DescriptorType = ERHIDescriptorType::Invalid;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
				if (Context.StateCache.GetDescriptorCache()->IsUsingBindlessHeap())
				{
					DescriptorType = ERHIDescriptorType::BufferUAV;
				}
#endif

				// Scoped view will free the offline CPU handle once we return
				// Always single GPU object, so FirstLinkedObject is nullptr
				FD3D12UnorderedAccessView UAV(ParentDevice, nullptr, DescriptorType);
				UAV.CreateView(UnorderedAccessView->GetResourceLocation(), R32UAVDesc, FD3D12UnorderedAccessView::EFlags::None);

				FD3D12OfflineDescriptor OfflineHandle = UAV.GetOfflineCpuHandle();
				D3D12_GPU_DESCRIPTOR_HANDLE GPUHandle{};

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
				if (Context.StateCache.GetDescriptorCache()->IsUsingBindlessHeap())
				{
					check(UAV.GetBindlessHandle().IsValid());

					// We just allocated a descriptor inline. Since this could have recreated the descriptor heaps,
					// force a refresh of the current GPU heap. If it didn't change, no extra commands should be added.
					Context.GetBindlessState().RefreshDescriptorHeap();

					Context.FlushPendingDescriptorUpdates();

					// Make sure a descriptor heap is actually bound. This will noop if they were bound previously.
					Context.StateCache.GetDescriptorCache()->SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags::Bindless);

					FD3D12DescriptorHeap* BindlessHeap = Context.GetBindlessResourcesHeap();
					UE::D3D12Descriptors::CopyDescriptor(ParentDevice, BindlessHeap, UAV.GetBindlessHandle(), OfflineHandle);
					GPUHandle = BindlessHeap->GetGPUSlotHandle(UAV.GetBindlessHandle().GetIndex());
				}
				else
#endif
				{
					// Check if the view heap is full and needs to rollover.
					if (!Context.StateCache.GetDescriptorCache()->GetCurrentViewHeap()->CanReserveSlots(1))
					{
						Context.StateCache.GetDescriptorCache()->GetCurrentViewHeap()->RollOver();
					}

					// Make sure a descriptor heap is actually bound. This will noop if they were bound previously.
					Context.StateCache.GetDescriptorCache()->SetDescriptorHeaps(ED3D12SetDescriptorHeapsFlags::None);

					uint32 ReservedSlot = Context.StateCache.GetDescriptorCache()->GetCurrentViewHeap()->ReserveSlots(1);
					D3D12_CPU_DESCRIPTOR_HANDLE DestSlot = Context.StateCache.GetDescriptorCache()->GetCurrentViewHeap()->GetCPUSlotHandle(ReservedSlot);
					GPUHandle = Context.StateCache.GetDescriptorCache()->GetCurrentViewHeap()->GetGPUSlotHandle(ReservedSlot);

					Device->CopyDescriptorsSimple(1, DestSlot, OfflineHandle, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV);
				}

				Context.FlushResourceBarriers();
				Context.GraphicsCommandList()->ClearUnorderedAccessViewUint(GPUHandle, OfflineHandle, UAV.GetResource()->GetResource(), *reinterpret_cast<const UINT(*)[4]>(ClearValues), 0, nullptr);
				Context.UpdateResidency(UAV.GetResource());
				Context.ConditionalSplitCommandList();
			});
		}
		else
		{
			ClearUAVShader_T<EClearReplacementResourceType::Buffer, 4, false>(RHICmdList, UnorderedAccessView, UAVDesc.Buffer.NumElements, 1, 1, ClearValues, ValueType);
		}
	}
	else
	{
		if (UAVDesc.ViewDimension == D3D12_UAV_DIMENSION_TEXTURE2D)
		{
			uint32 Width = ResourceDesc.Width >> UAVDesc.Texture2D.MipSlice;
			uint32 Height = ResourceDesc.Height >> UAVDesc.Texture2D.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture2D, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, 1, ClearValues, ValueType);
		}
		else if (UAVDesc.ViewDimension == D3D12_UAV_DIMENSION_TEXTURE2DARRAY)
		{
			uint32 Width = ResourceDesc.Width >> UAVDesc.Texture2DArray.MipSlice;
			uint32 Height = ResourceDesc.Height >> UAVDesc.Texture2DArray.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, UAVDesc.Texture2DArray.ArraySize, ClearValues, ValueType);
		}
		else if (UAVDesc.ViewDimension == D3D12_UAV_DIMENSION_TEXTURE3D)
		{
			// @todo - is WSize / mip index handling here correct?
			uint32 Width = ResourceDesc.Width >> UAVDesc.Texture2DArray.MipSlice;
			uint32 Height = ResourceDesc.Height >> UAVDesc.Texture2DArray.MipSlice;
			ClearUAVShader_T<EClearReplacementResourceType::Texture3D, 4, false>(RHICmdList, UnorderedAccessView, Width, Height, UAVDesc.Texture3D.WSize, ClearValues, ValueType);
		}
		else
		{
			ensure(0);
		}
	}
}

void FD3D12CommandContext::RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
{
	TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
	ClearUAV(RHICmdList, RetrieveObject<FD3D12UnorderedAccessView_RHI>(UnorderedAccessViewRHI), &Values, true);
}

void FD3D12CommandContext::RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
{
	TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);
	ClearUAV(RHICmdList, RetrieveObject<FD3D12UnorderedAccessView_RHI>(UnorderedAccessViewRHI), &Values, false);
}
