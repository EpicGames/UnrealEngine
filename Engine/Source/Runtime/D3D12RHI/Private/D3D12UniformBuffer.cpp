// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12UniformBuffer.cpp: D3D uniform buffer RHI implementation.
	=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "UniformBuffer.h"
#include "ShaderParameterStruct.h"
#include "RHIUniformBufferDataShared.h"

inline bool AreBindlessUniformConstantsEnabled(FD3D12Device* Device)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FD3D12BindlessDescriptorManager& Manager = Device->GetBindlessDescriptorManager();
	if (!IsBindlessDisabled(Manager.GetConfiguration()))
	{
		return true;
	}
#endif
	return false;
}

inline bool DoesUniformBufferNeedBindlessHandle(FD3D12Device* Device, const FRHIUniformBufferLayout* Layout)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (EnumHasAnyFlags(Layout->Flags, ERHIUniformBufferFlags::BindlessAccessible))
	{
		FD3D12BindlessDescriptorManager& Manager = Device->GetBindlessDescriptorManager();
		return IsBindlessEnabledForAnyGraphics(Manager.GetConfiguration());
	}
#endif

	return false;
}

FUniformBufferRHIRef FD3D12DynamicRHI::RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	SCOPE_CYCLE_COUNTER(STAT_D3D12UpdateUniformBufferTime);

	if (Contents && Validation == EUniformBufferValidation::ValidateResources)
	{
		ValidateShaderParameterResourcesRHI(Contents, *Layout);
	}

	//Note: This is not overly efficient in the mGPU case (we create two+ upload locations) but the CPU savings of having no extra indirection to the resource are worth
	//      it in single node.
	// Create the uniform buffer
	FD3D12UniformBuffer* UniformBufferOut = GetAdapter().CreateLinkedObject<FD3D12UniformBuffer>(FRHIGPUMask::All(), [&](FD3D12Device* Device, FD3D12UniformBuffer* FirstLinkedObject) -> FD3D12UniformBuffer*
	{
		// If NumBytesActualData == 0, this uniform buffer contains no constants, only a resource table.
		FD3D12UniformBuffer* NewUniformBuffer = new FD3D12UniformBuffer(Device, Layout, Usage);
		check(nullptr != NewUniformBuffer);

		const uint32 NumBytesActualData = Layout->ConstantBufferSize;
		if (NumBytesActualData > 0)
		{
			// Is this check really needed?
			check(Align(NumBytesActualData, 16) == NumBytesActualData);
			check(Align(Contents, 16) == Contents);
			check(NumBytesActualData <= D3D12_REQ_CONSTANT_BUFFER_ELEMENT_COUNT * 16);

			const bool bCreateBindlessHandle = DoesUniformBufferNeedBindlessHandle(Device, Layout);
#if !D3D12RHI_USE_CONSTANT_BUFFER_VIEWS
			if (bCreateBindlessHandle)
#endif
			{
				// Create an offline CBV descriptor
				NewUniformBuffer->View = new FD3D12ConstantBufferView(Device, FirstLinkedObject ? FirstLinkedObject->View : nullptr, bCreateBindlessHandle ? ERHIDescriptorType::CBV : ERHIDescriptorType::Invalid);
			}

			// Uniform buffers can be created without contents and updated later.
			if (Contents)
			{
				void* MappedData = nullptr;
				if (Usage == EUniformBufferUsage::UniformBuffer_MultiFrame)
				{
					// Uniform buffers that live for multiple frames must use the more expensive and persistent allocation path
					FD3D12UploadHeapAllocator& Allocator = GetAdapter().GetUploadHeapAllocator(Device->GetGPUIndex());
					MappedData = Allocator.AllocUploadResource(NumBytesActualData, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT, NewUniformBuffer->ResourceLocation);
				}
				else
				{
					// Uniform buffers which will live for 1 frame at the max can be allocated very efficiently from a ring buffer
					FD3D12FastConstantAllocator& Allocator = GetAdapter().GetTransientUniformBufferAllocator();
					MappedData = Allocator.Allocate(NumBytesActualData, NewUniformBuffer->ResourceLocation, nullptr);
				}
				check(NewUniformBuffer->ResourceLocation.GetOffsetFromBaseOfResource() % 16 == 0);

				// Copy the data to the upload heap
				check(MappedData != nullptr);

				UE::RHICore::UpdateUniformBufferConstants(MappedData, Contents, *Layout, AreBindlessUniformConstantsEnabled(Device));

				if (NewUniformBuffer->View)
				{
					const uint32 NumBytes = Align(NumBytesActualData, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT);
					NewUniformBuffer->View->CreateView(&NewUniformBuffer->ResourceLocation, 0, NumBytes);
				}
			}
		}

		// The GPUVA is used to see if this uniform buffer contains constants or is just a resource table.
		check((Contents && NumBytesActualData > 0) ? (0 != NewUniformBuffer->ResourceLocation.GetGPUVirtualAddress()) : (0 == NewUniformBuffer->ResourceLocation.GetGPUVirtualAddress()));
		return NewUniformBuffer;
	});

	check(UniformBufferOut);

	if (Layout->Resources.Num())
	{
		const int32 NumResources = Layout->Resources.Num();

		for (FD3D12UniformBuffer& CurrentBuffer : *UniformBufferOut)
		{
			CurrentBuffer.GetResourceTable().SetNumZeroed(NumResources);

			if (Contents)
			{
				for (int32 Index = 0; Index < NumResources; ++Index)
				{
					CurrentBuffer.GetResourceTable()[Index] = GetShaderParameterResourceRHI(Contents, Layout->Resources[Index].MemberOffset, Layout->Resources[Index].MemberType);
				}
			}
		}
	}

	INC_MEMORY_STAT_BY(STAT_UniformBufferMemory, UniformBufferOut->ResourceLocation.GetSize());

	return UniformBufferOut;
}

static void DoUpdateD3D12UniformBuffer(const FD3D12ContextArray& Contexts, FD3D12UniformBuffer* UniformBuffer, FD3D12ResourceLocation&& UpdatedLocation, TArrayView<FRHIResource*> UpdatedResources)
{
	for (int32 Index = 0; Index < UpdatedResources.Num(); Index++)
	{
		UniformBuffer->GetResourceTable()[Index] = UpdatedResources[Index];

		check(UniformBuffer->GetResourceTable()[Index]);
	}
	FD3D12ResourceLocation::TransferOwnership(UniformBuffer->ResourceLocation, UpdatedLocation);

	if (UniformBuffer->View)
	{
		const uint32 NumBytes = Align(UniformBuffer->GetLayout().ConstantBufferSize, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT);
		UniformBuffer->View->UpdateView(Contexts, &UniformBuffer->ResourceLocation, 0, NumBytes);
	}

	// Notify the listeners now that the resource location on the uniform buffer has been updated
	UniformBuffer->UniformBufferUpdated(Contexts);
}

void FD3D12DynamicRHI::RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
{
	check(UniformBufferRHI);

	const FRHIUniformBufferLayout& Layout = UniformBufferRHI->GetLayout();
	ValidateShaderParameterResourcesRHI(Contents, Layout);

	const bool bBypass = RHICmdList.Bypass();

	FD3D12UniformBuffer* FirstUniformBuffer = ResourceCast(UniformBufferRHI);

	const uint32 NumBytes = Layout.ConstantBufferSize;

	TArrayView<FRHIResource*> CmdListResources;

	if (const int32 NumResources = Layout.Resources.Num())
	{
		CmdListResources = bBypass
			? TArrayView<FRHIResource*>((FRHIResource**)FMemory_Alloca(sizeof(FRHIResource*) * NumResources), NumResources)
			: RHICmdList.AllocArrayUninitialized<FRHIResource*>(NumResources);

		for (int32 Index = 0; Index < NumResources; Index++)
		{
			const FRHIUniformBufferResource& Parameter = Layout.Resources[Index];
			CmdListResources[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
		}
	}

	// Update buffers on all GPUs by looping over FD3D12LinkedAdapterObject chain
	for (FD3D12UniformBuffer& UniformBuffer : *FirstUniformBuffer)
	{
		check(UniformBuffer.GetResourceTable().Num() == CmdListResources.Num());

		FD3D12Device* Device = UniformBuffer.GetParentDevice();

		FD3D12ResourceLocation UpdatedResourceLocation(Device);

		if (NumBytes > 0)
		{
			void* MappedData = nullptr;

			if (UniformBuffer.UniformBufferUsage == UniformBuffer_MultiFrame)
			{
				FD3D12UploadHeapAllocator& Allocator = GetAdapter().GetUploadHeapAllocator(Device->GetGPUIndex());
				MappedData = Allocator.AllocUploadResource(NumBytes, D3D12_CONSTANT_BUFFER_DATA_PLACEMENT_ALIGNMENT, UpdatedResourceLocation);
			}
			else
			{
				FD3D12FastConstantAllocator& Allocator = GetAdapter().GetTransientUniformBufferAllocator();

				MappedData = Allocator.Allocate(NumBytes, UpdatedResourceLocation, nullptr);
			}

			check(MappedData != nullptr);
			UE::RHICore::UpdateUniformBufferConstants(MappedData, Contents, Layout, AreBindlessUniformConstantsEnabled(Device));
		}

		if (!bBypass)
		{
			for (FRHIResource* Resource : CmdListResources)
			{
				Resource->AddRef();
			}
		}

		//fence is required to stop parallel recording threads from recording with the old bad state of the uniformbuffer resource table.  This command MUST execute before dependent recording starts.
		RHICmdList.EnqueueLambdaMultiPipe(GetEnabledRHIPipelines(), FRHICommandListBase::EThreadFence::Enabled, TEXT("FD3D12DynamicRHI::RHIUpdateUniformBuffer"),
			[bReleaseResources = !bBypass, UniformBuffer = &UniformBuffer, UpdatedLocation = MoveTemp(UpdatedResourceLocation), CmdListResources = MoveTemp(CmdListResources)](const FRHIContextArray& Contexts) mutable
			{
				DoUpdateD3D12UniformBuffer(Contexts, UniformBuffer, MoveTemp(UpdatedLocation), CmdListResources);

				if (bReleaseResources)
				{
					for (FRHIResource* Resource : CmdListResources)
					{
						Resource->Release();
					}
				}
			});
	}
}

FD3D12UniformBuffer::~FD3D12UniformBuffer()
{
	check(IsInRHIThread() || IsInRenderingThread());

	if (!UpdateListeners.IsEmpty())
	{
		//UE_LOGF(LogD3D12RHI, Log, "Deleting uniform buffer %#016llx with GPU address: \"0x%llX\" and %d listeners still registered", this, ResourceLocation.GetGPUVirtualAddress(), UpdateListeners.Num());

		// Request remove of listener to this uniform buffer - uniform buffers can be deleted before the cached MDCs referencing
		// the uniform buffers are deleted because requests are still pending for scene proxy removal
		while (!UpdateListeners.IsEmpty())
		{
			UpdateListeners.Pop()->RemoveListener(this);
		}
	}

	int64 BufferSize = ResourceLocation.GetSize();
	DEC_MEMORY_STAT_BY(STAT_UniformBufferMemory, BufferSize);

	delete View;
}

FRHIDescriptorHandle FD3D12UniformBuffer::GetBindlessHandle() const
{
	return View ? View->GetBindlessHandle() : FRHIDescriptorHandle();
}
