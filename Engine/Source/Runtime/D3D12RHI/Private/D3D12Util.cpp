// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3D12Util.h: D3D RHI utility implementation.
=============================================================================*/

#include "D3D12Util.h"
#include "D3D12RHIPrivate.h"
#include "D3D12ShaderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RendererInterface.h"
#include "CoreGlobals.h"
#if WITH_ENGINE
#include "Engine/GameEngine.h"
#include "EngineModule.h"
#endif
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/ExceptionHandling.h"
#if PLATFORM_WINDOWS
#include "HAL/PlatformCrashContext.h"
#endif
#include "GenericPlatform/GenericPlatformCrashContext.h"

#define D3DERR(x) case x: ErrorCodeText = TEXT(#x); break;
#define LOCTEXT_NAMESPACE "Developer.MessageLog"

// GPU crashes are nonfatal on windows/nonshipping so as not to interfere with GPU crash dump processing
#if PLATFORM_WINDOWS || !UE_BUILD_SHIPPING
  #define D3D12RHI_GPU_CRASH_LOG_VERBOSITY Error
#else
  #define D3D12RHI_GPU_CRASH_LOG_VERBOSITY Fatal
#endif

template<typename PerDeviceFunction>
void FD3D12DynamicRHI::ForEachDevice(ID3D12Device* inDevice, const PerDeviceFunction& pfPerDeviceFunction)
{
	if (FD3D12Adapter* const D3D12Adapter = ChosenAdapter.Get())
	{
		for (uint32 GPUIndex : FRHIGPUMask::All())
		{
			FD3D12Device* D3D12Device = D3D12Adapter->GetDevice(GPUIndex);
			if (inDevice == nullptr || D3D12Device->GetDevice() == inDevice)
			{
				pfPerDeviceFunction(D3D12Device);
			}
		}
	}
}

bool ShouldSetD3D12ResourceName(const FD3D12ResourceLocation& ResourceLocation)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	// only rename the underlying d3d12 resource if it's not sub allocated (requires resource state tracking or stand alone allocated)
	return ResourceLocation.GetResource() && (ResourceLocation.GetResource()->RequiresResourceStateTracking() || ResourceLocation.GetType() == FD3D12ResourceLocation::ResourceLocationType::eStandAlone);
#else
	return false;
#endif
}

void SetD3D12ObjectName(ID3D12Object* Object, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Object && Name)
	{
		VERIFYD3D12RESULT(Object->SetName(Name));
	}
#endif
}

void SetD3D12ResourceName(FD3D12Resource* Resource, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (Resource)
	{
		Resource->SetName(Name);
	}
#endif
}

void SetD3D12ResourceName(FD3D12ResourceLocation& ResourceLocation, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	if (ShouldSetD3D12ResourceName(ResourceLocation))
	{
		SetD3D12ResourceName(ResourceLocation.GetResource(), Name);
	}
#endif
}

void SetD3D12ResourceName(FD3D12Buffer* Buffer, const TCHAR* Name)
{
	if (Buffer)
	{
		SetD3D12ResourceName(Buffer->ResourceLocation, Name);
	}
}

void SetD3D12ResourceName(FD3D12Texture* Texture, const TCHAR* Name)
{
	if (Texture)
	{
		SetD3D12ResourceName(Texture->ResourceLocation, Name);
	}
}

FString GetD312ObjectName(ID3D12Object* const Object)
{
#if NAME_OBJECTS
	if (Object == nullptr)
		return FString("Unknown Resource");

	#define MAX_OBJECT_NAME_LEN 512

	TCHAR OutName[MAX_OBJECT_NAME_LEN];
	UINT size = MAX_OBJECT_NAME_LEN;
	HRESULT hr = Object->GetPrivateData(WKPDID_D3DDebugObjectNameW, &size, OutName);
	check(size <= MAX_OBJECT_NAME_LEN);

	if (hr != S_OK)
		return FString("Unknown Resource");
	
	return FString(OutName);
#else
	return FString();
#endif
}

static FString GetD3D12DeviceHungErrorString(HRESULT ErrorCode)
{
	FString ErrorCodeText;

	switch (ErrorCode)
	{
		D3DERR(DXGI_ERROR_DEVICE_HUNG)
		D3DERR(DXGI_ERROR_DEVICE_REMOVED)
		D3DERR(DXGI_ERROR_DEVICE_RESET)
		D3DERR(DXGI_ERROR_DRIVER_INTERNAL_ERROR)
		D3DERR(DXGI_ERROR_INVALID_CALL)
		default:
			ErrorCodeText = FString::Printf(TEXT("%08X"), (int32)ErrorCode);
	}

	return ErrorCodeText;
}

static FString GetD3D12ErrorString(HRESULT ErrorCode, ID3D12Device* Device)
{
	FString ErrorCodeText;

	switch (ErrorCode)
	{
		D3DERR(S_OK);
		D3DERR(D3D11_ERROR_FILE_NOT_FOUND)
		D3DERR(D3D11_ERROR_TOO_MANY_UNIQUE_STATE_OBJECTS)
		D3DERR(E_FAIL)
		D3DERR(E_INVALIDARG)
		D3DERR(E_OUTOFMEMORY)
		D3DERR(DXGI_ERROR_INVALID_CALL)
		D3DERR(DXGI_ERROR_WAS_STILL_DRAWING)
		D3DERR(E_NOINTERFACE)
		D3DERR(DXGI_ERROR_DEVICE_REMOVED)
#if PLATFORM_WINDOWS
		EMBED_DXGI_ERROR_LIST(D3DERR, )
#endif
		default:
			ErrorCodeText = FString::Printf(TEXT("%08X"), (int32)ErrorCode);
	}

	if (ErrorCode == DXGI_ERROR_DEVICE_REMOVED && Device)
	{
		HRESULT hResDeviceRemoved = Device->GetDeviceRemovedReason();
		ErrorCodeText += FString(" with Reason: ") + GetD3D12DeviceHungErrorString(hResDeviceRemoved);
	}

	return ErrorCodeText;
}

#undef D3DERR

static FString GetD3D12TextureFlagString(uint32 TextureFlags)
{
	FString TextureFormatText;

	if (TextureFlags & D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET)
	{
		TextureFormatText += TEXT("D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET ");
	}

	if (TextureFlags & D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL)
	{
		TextureFormatText += TEXT("D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL ");
	}

	if (TextureFlags & D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE)
	{
		TextureFormatText += TEXT("D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE ");
	}

	if (TextureFlags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS)
	{
		TextureFormatText += TEXT("D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS ");
	}
	return TextureFormatText;
}

#if PLATFORM_WINDOWS

static TArrayView<D3D12_DRED_BREADCRUMB_CONTEXT> GetBreadcrumbContexts(const D3D12_AUTO_BREADCRUMB_NODE* Node)
{
	return {};
}

static TArrayView<D3D12_DRED_BREADCRUMB_CONTEXT> GetBreadcrumbContexts(const D3D12_AUTO_BREADCRUMB_NODE1* Node)
{
	return MakeArrayView<D3D12_DRED_BREADCRUMB_CONTEXT>(Node->pBreadcrumbContexts, Node->BreadcrumbContextsCount);
}

struct FDred_1_1
{
	FDred_1_1(ID3D12Device* Device)
	{
		if (SUCCEEDED(Device->QueryInterface(IID_PPV_ARGS(Data.GetInitReference()))))
		{
			D3D12_DRED_AUTO_BREADCRUMBS_OUTPUT DredAutoBreadcrumbsOutput;
			if (SUCCEEDED(Data->GetAutoBreadcrumbsOutput(&DredAutoBreadcrumbsOutput)))
			{
				BreadcrumbHead = DredAutoBreadcrumbsOutput.pHeadAutoBreadcrumbNode;
			}
		}
	}
	TRefCountPtr<ID3D12DeviceRemovedExtendedData> Data;
	const D3D12_AUTO_BREADCRUMB_NODE* BreadcrumbHead = nullptr;
};

struct FDred_1_2
{
	FDred_1_2(ID3D12Device* Device)
	{
		if (SUCCEEDED(Device->QueryInterface(IID_PPV_ARGS(Data.GetInitReference()))))
		{
			D3D12_DRED_AUTO_BREADCRUMBS_OUTPUT1 DredAutoBreadcrumbsOutput;
			if (SUCCEEDED(Data->GetAutoBreadcrumbsOutput1(&DredAutoBreadcrumbsOutput)))
			{
				BreadcrumbHead = DredAutoBreadcrumbsOutput.pHeadAutoBreadcrumbNode;
			}
		}
	}
	TRefCountPtr<ID3D12DeviceRemovedExtendedData1> Data;
	const D3D12_AUTO_BREADCRUMB_NODE1* BreadcrumbHead = nullptr;
};

// Should match all values from D3D12_AUTO_BREADCRUMB_OP
static const TCHAR* BreadcrumbOpNames[] =
{
	TEXT("SetMarker"),
	TEXT("BeginEvent"),
	TEXT("EndEvent"),
	TEXT("DrawInstanced"),
	TEXT("DrawIndexedInstanced"),
	TEXT("ExecuteIndirect"),
	TEXT("Dispatch"),
	TEXT("CopyBufferRegion"),
	TEXT("CopyTextureRegion"),
	TEXT("CopyResource"),
	TEXT("CopyTiles"),
	TEXT("ResolveSubresource"),
	TEXT("ClearRenderTargetView"),
	TEXT("ClearUnorderedAccessView"),
	TEXT("ClearDepthStencilView"),
	TEXT("ResourceBarrier"),
	TEXT("ExecuteBundle"),
	TEXT("Present"),
	TEXT("ResolveQueryData"),
	TEXT("BeginSubmission"),
	TEXT("EndSubmission"),
	TEXT("DecodeFrame"),
	TEXT("ProcessFrames"),
	TEXT("AtomicCopyBufferUint"),
	TEXT("AtomicCopyBufferUint64"),
	TEXT("ResolveSubresourceRegion"),
	TEXT("WriteBufferImmediate"),
	TEXT("DecodeFrame1"),
	TEXT("SetProtectedResourceSession"),
	TEXT("DecodeFrame2"),
	TEXT("ProcessFrames1"),
	TEXT("BuildRaytracingAccelerationStructure"),
	TEXT("EmitRaytracingAccelerationStructurePostBuildInfo"),
	TEXT("CopyRaytracingAccelerationStructure"),
	TEXT("DispatchRays"),
	TEXT("InitializeMetaCommand"),
	TEXT("ExecuteMetaCommand"),
	TEXT("EstimateMotion"),
	TEXT("ResolveMotionVectorHeap"),
	TEXT("SetPipelineState1"),
	TEXT("InitializeExtensionCommand"),
	TEXT("ExecuteExtensionCommand"),
	TEXT("DispatchMesh"),
	TEXT("EncodeFrame"),
	TEXT("ResolveEncoderOutputMetadata"),
	TEXT("Barrier"),
	TEXT("BeginCommandList"),
	TEXT("DispatchGraph"),
	TEXT("SetProgram"),
};
static_assert(UE_ARRAY_COUNT(BreadcrumbOpNames) == D3D12_AUTO_BREADCRUMB_OP_SETPROGRAM + 1, "OpNames array length mismatch");

/** 
 * Calculate the number of active scopes in the case of a DRED history where the number of 
 * EndEvent operations does not match the number of BeginEvent operations.
 * Practically, this would be the number of "missing" BeginEvent operations that, if added at
 * the beginning of the history, would balance out all EndEvent operations found later on.
 */
template <typename FDredNode_T>
static uint32 CalculateDREDUnknownActiveScopes(const FDredNode_T* DredNode)
{
	check(DredNode);

	int32 NumOpenEvents = 0;
	int32 MaxUnknownActiveScopes = 0;
	for (uint32 Op = 0; Op < DredNode->BreadcrumbCount; ++Op)
	{
		D3D12_AUTO_BREADCRUMB_OP BreadcrumbOp = DredNode->pCommandHistory[Op];
		if (BreadcrumbOp == D3D12_AUTO_BREADCRUMB_OP_BEGINEVENT)
		{
			NumOpenEvents++;
		}
		else if (BreadcrumbOp == D3D12_AUTO_BREADCRUMB_OP_ENDEVENT)
		{
			NumOpenEvents--;
		}

		MaxUnknownActiveScopes = FMath::Min(NumOpenEvents, MaxUnknownActiveScopes);
	}

	return FMath::Abs(MaxUnknownActiveScopes);
}

template <typename FDredNode_T>
static FGPUBreadcrumbCrashData::FQueueData CollectDREDBreadcrumbNodes(const FDredNode_T* DredNode)
{
	check(DredNode && DredNode->pLastBreadcrumbValue);
	uint32 LastCompletedOp = *DredNode->pLastBreadcrumbValue;
	if (LastCompletedOp == DredNode->BreadcrumbCount || LastCompletedOp == 0)
	{
		return {};
	}

	TMap<uint32, const wchar_t*> ContextStrings;
	for (const D3D12_DRED_BREADCRUMB_CONTEXT& Context : GetBreadcrumbContexts(DredNode))
	{
		ContextStrings.Add(Context.BreadcrumbIndex, Context.pContextString);
	}

	using EState = FGPUBreadcrumbCrashData::EState;
	struct FBreadcrumbNode
	{
		TOptional<EState> State {};
		FString Name;
		TArray<FBreadcrumbNode> Children;
	};

	// Create a root node that will hold all events as children. The root itself will be discarded.
	FBreadcrumbNode Root;
	TArray<FBreadcrumbNode*> ParentChain = { &Root };

	// If we have open scopes, create them now as "Unknown events".
	uint32 NumOpenScopes = CalculateDREDUnknownActiveScopes(DredNode);
	for (uint32 i = 0; i < NumOpenScopes; ++i)
	{
		FBreadcrumbNode& UnknownNode = ParentChain.Last()->Children.Emplace_GetRef();
		UnknownNode.Name = TEXT("Unknown event");
		UnknownNode.State = EState::Active;
		ParentChain.Push(&UnknownNode);
	}

	for (uint32 Op = 0; Op < DredNode->BreadcrumbCount; ++Op)
	{
		D3D12_AUTO_BREADCRUMB_OP BreadcrumbOp = DredNode->pCommandHistory[Op];
		bool bCompleted = Op < LastCompletedOp;
		auto OpContextStr = ContextStrings.Find(Op);

		if (BreadcrumbOp == D3D12_AUTO_BREADCRUMB_OP_BEGINEVENT)
		{
			// This is a begin event, potentially with children events.
			FBreadcrumbNode& BreadcrumbNode = ParentChain.Last()->Children.Emplace_GetRef();
			BreadcrumbNode.Name = OpContextStr ? *OpContextStr : TEXT("Unknown event");
			BreadcrumbNode.State = bCompleted ? EState::Active : EState::NotStarted;

			ParentChain.Push(&BreadcrumbNode);
		}
		else if (BreadcrumbOp == D3D12_AUTO_BREADCRUMB_OP_ENDEVENT)
		{
			FBreadcrumbNode* Parent = ParentChain.Pop();
			if (!Parent->State.IsSet())
			{
				// If we reach this point, the DRED breadcrumbs are malformed, and some
				// basic invariants around matching BeginEvent/EndEvent do not hold.
				// Return gracefully and do not attempt to process further.
				return {};
			}

			// This is the end event for the parent node. Mark the whole event as finished
			// if this end event was completed.
			if (bCompleted && Parent->State == EState::Active)
			{
				Parent->State = EState::Finished;
			}
		}
		else
		{
			// This is a miscellaneous event between a BeginEvent and an EndEvent.
			const TCHAR* OpName = (BreadcrumbOp < UE_ARRAY_COUNT(BreadcrumbOpNames)) ? BreadcrumbOpNames[BreadcrumbOp] : TEXT("Unknown Op");

			FBreadcrumbNode& BreadcrumbNode = ParentChain.Last()->Children.Emplace_GetRef();
			if (OpContextStr)
			{
				BreadcrumbNode.Name = FString::Printf(TEXT("%s [%s]"), OpName, *OpContextStr);
			}
			else
			{
				BreadcrumbNode.Name = OpName;
			}
			BreadcrumbNode.State = bCompleted ? EState::Finished : EState::NotStarted;
		}
	}

	FGPUBreadcrumbCrashData::FQueueData Result {};

	if (!Root.Children.IsEmpty())
	{
		FGPUBreadcrumbCrashData::FSerializer Serializer;
		for (FBreadcrumbNode const& ActualRoot : Root.Children)
		{
			auto Recurse = [&](FBreadcrumbNode const& Current, auto& Recurse) -> void
			{
				Serializer.BeginNode(Current.Name, *Current.State);

				for (FBreadcrumbNode const& Child : Current.Children)
				{
					Recurse(Child, Recurse);
				}

				Serializer.EndNode();
			};
			Recurse(ActualRoot, Recurse);
		}

		Result = Serializer.GetResult();
	}	

	return Result;
}

/** Log the DRED data to Error log if available */
template <typename FDred_T>
static bool LogDREDData(ID3D12Device* Device, bool bTrackingAllAllocations, D3D12_GPU_VIRTUAL_ADDRESS& OutPageFaultGPUAddress)
{
	// Should match all valid values from D3D12_DRED_ALLOCATION_TYPE
	static const TCHAR* AllocTypesNames[] =
	{
		TEXT("CommandQueue"),
		TEXT("CommandAllocator"),
		TEXT("PipelineState"),
		TEXT("CommandList"),
		TEXT("Fence"),
		TEXT("DescriptorHeap"),
		TEXT("Heap"),
		TEXT("Unknown"),				// Unknown type - missing enum value in D3D12_DRED_ALLOCATION_TYPE
		TEXT("QueryHeap"),
		TEXT("CommandSignature"),
		TEXT("PipelineLibrary"),
		TEXT("VideoDecoder"),
		TEXT("Unknown"),				// Unknown type - missing enum value in D3D12_DRED_ALLOCATION_TYPE
		TEXT("VideoProcessor"),
		TEXT("Unknown"),				// Unknown type - missing enum value in D3D12_DRED_ALLOCATION_TYPE
		TEXT("Resource"),
		TEXT("Pass"),
		TEXT("CryptoSession"),
		TEXT("CryptoSessionPolicy"),
		TEXT("ProtectedResourceSession"),
		TEXT("VideoDecoderHeap"),
		TEXT("CommandPool"),
		TEXT("CommandRecorder"),
		TEXT("StateObjectr"),
		TEXT("MetaCommand"),
		TEXT("SchedulingGroup"),
		TEXT("VideoMotionEstimator"),
		TEXT("VideoMotionVectorHeap"),
		TEXT("VideoExtensionCommand"),
	};
	static_assert(UE_ARRAY_COUNT(AllocTypesNames) == D3D12_DRED_ALLOCATION_TYPE_VIDEO_EXTENSION_COMMAND - D3D12_DRED_ALLOCATION_TYPE_COMMAND_QUEUE + 1, "AllocTypes array length mismatch");

	bool bHasValidBreadcrumbData = false;
	FDred_T Dred(Device);
	if (Dred.Data.IsValid())
	{
		if (Dred.BreadcrumbHead)
		{
			UE_LOGF(LogD3D12RHI, Error, "DRED: Last tracked GPU operations:");

			FGPUBreadcrumbCrashData CrashData(TEXT("DRED"));

			FString ContextStr;
			TMap<int32, const wchar_t*> ContextStrings;

			uint32 TracedCommandLists = 0;
			auto Node = Dred.BreadcrumbHead;
			while (Node && Node->pLastBreadcrumbValue)
			{
				int32 LastCompletedOp = *Node->pLastBreadcrumbValue;

				if (LastCompletedOp != Node->BreadcrumbCount && LastCompletedOp != 0)
				{
					bHasValidBreadcrumbData = true;
					UE_LOGF(LogD3D12RHI, Error, "DRED: Commandlist \"%ls\" on CommandQueue \"%ls\", %d completed of %d", Node->pCommandListDebugNameW, Node->pCommandQueueDebugNameW, LastCompletedOp, Node->BreadcrumbCount);
					TracedCommandLists++;

					int32 FirstOp = FMath::Max(LastCompletedOp - 100, 0);
					int32 LastOp = FMath::Min(LastCompletedOp + 20, int32(Node->BreadcrumbCount) - 1);

					ContextStrings.Reset();
					for (const D3D12_DRED_BREADCRUMB_CONTEXT& Context : GetBreadcrumbContexts(Node))
					{
						ContextStrings.Add(Context.BreadcrumbIndex, Context.pContextString);
					}

					for (int32 Op = FirstOp; Op <= LastOp; ++Op)
					{
						D3D12_AUTO_BREADCRUMB_OP BreadcrumbOp = Node->pCommandHistory[Op];

						auto OpContextStr = ContextStrings.Find(Op);
						if (OpContextStr)
						{
							ContextStr = " [";
							ContextStr += *OpContextStr;
							ContextStr += "]";
						}
						else
						{
							ContextStr.Reset();
						}

						const TCHAR* OpName = (BreadcrumbOp < UE_ARRAY_COUNT(BreadcrumbOpNames)) ? BreadcrumbOpNames[BreadcrumbOp] : TEXT("Unknown Op");
						UE_LOGF(LogD3D12RHI, Error, "\tOp: %d, %ls%ls%ls", Op, OpName, *ContextStr, (Op + 1 == LastCompletedOp) ? TEXT(" - LAST COMPLETED") : TEXT(""));
					}

					// Collect and export breadcrumb data separately as part of the crash payload.
					if (FGPUBreadcrumbCrashData::FQueueData QueueData = CollectDREDBreadcrumbNodes(Node))
					{
						CrashData.Queues.FindOrAdd(Node->pCommandQueueDebugNameW, MoveTemp(QueueData));
					}
				}

				Node = Node->pNext;
			}

			if (CrashData.Queues.Num())
			{
				FGenericCrashContext::SetGPUBreadcrumbs(MoveTemp(CrashData));
			}

			if (TracedCommandLists == 0)
			{
				UE_LOGF(LogD3D12RHI, Error, "DRED: No command list found with active outstanding operations (all finished or not started yet).");
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "DRED: No breadcrumb head found.");
		}

		FPlatformCrashContext::SetEngineData(TEXT("RHI.DREDHasBreadcrumbData"), bHasValidBreadcrumbData ? TEXT("true") : TEXT("false"));

		bool bHasValidPageFaultData = false;
		D3D12_DRED_PAGE_FAULT_OUTPUT DredPageFaultOutput;
		if (SUCCEEDED(Dred.Data->GetPageFaultAllocationOutput(&DredPageFaultOutput)) && DredPageFaultOutput.PageFaultVA != 0)
		{
			bHasValidPageFaultData = true;
			OutPageFaultGPUAddress = DredPageFaultOutput.PageFaultVA;
			UE_LOGF(LogD3D12RHI, Error, "DRED: PageFault at VA GPUAddress \"0x%llX\"", (long long)DredPageFaultOutput.PageFaultVA);
			
			const D3D12_DRED_ALLOCATION_NODE* Node = DredPageFaultOutput.pHeadExistingAllocationNode;
			if (Node)
			{
				UE_LOGF(LogD3D12RHI, Error, "DRED: Active objects with VA ranges that match the faulting VA:");
				while (Node)
				{
					// When tracking all allocations then empty named dummy resources (heap & buffer)
					// are created for each texture to extract the GPUBaseAddress so don't write these out
					if (!bTrackingAllAllocations || Node->ObjectNameW)
					{
						int32 alloc_type_index = Node->AllocationType - D3D12_DRED_ALLOCATION_TYPE_COMMAND_QUEUE;
						const TCHAR* AllocTypeName = (alloc_type_index < UE_ARRAY_COUNT(AllocTypesNames)) ? AllocTypesNames[alloc_type_index] : TEXT("Unknown Alloc");
						UE_LOGF(LogD3D12RHI, Error, "\tName: %ls (Type: %ls)", Node->ObjectNameW, AllocTypeName);
					}
					Node = Node->pNext;
				}
			}

			Node = DredPageFaultOutput.pHeadRecentFreedAllocationNode;
			if (Node)
			{
				UE_LOGF(LogD3D12RHI, Error, "DRED: Recent freed objects with VA ranges that match the faulting VA:");
				while (Node)
				{
					// See comments above
					if (!bTrackingAllAllocations || Node->ObjectNameW)
					{
						int32 alloc_type_index = Node->AllocationType - D3D12_DRED_ALLOCATION_TYPE_COMMAND_QUEUE;
						const TCHAR* AllocTypeName = (alloc_type_index < UE_ARRAY_COUNT(AllocTypesNames)) ? AllocTypesNames[alloc_type_index] : TEXT("Unknown Alloc");
						UE_LOGF(LogD3D12RHI, Error, "\tName: %ls (Type: %ls)", Node->ObjectNameW, AllocTypeName);
					}

					Node = Node->pNext;
				}
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "DRED: No PageFault data.");
		}

		FPlatformCrashContext::SetEngineData(TEXT("RHI.DREDHasPageFaultData"), bHasValidPageFaultData ? TEXT("true") : TEXT("false"));

		return true;
	}
	else
	{
		return false;
	}
}


namespace D3D12RHI
{


static FString MakeResourceDescDebugString(const D3D12_RESOURCE_DESC& Desc)
{
	FString ResourceDescString;
	switch (Desc.Dimension)
	{
	default:
		ResourceDescString = TEXT("Unknown");
		break;
	case D3D12_RESOURCE_DIMENSION_BUFFER:
		ResourceDescString = FString::Printf(TEXT("Buffer %" UINT64_FMT " bytes"), Desc.Width);
		break;
	case D3D12_RESOURCE_DIMENSION_TEXTURE1D:
	case D3D12_RESOURCE_DIMENSION_TEXTURE2D:
	case D3D12_RESOURCE_DIMENSION_TEXTURE3D:
		ResourceDescString = FString::Printf(TEXT("Texture %" UINT64_FMT "x%dx%d %s"), Desc.Width, Desc.Height, Desc.DepthOrArraySize, LexToString(Desc.Format));
	}
	return ResourceDescString;
}

void LogPageFaultData(FD3D12Adapter* InAdapter, FD3D12Device* InDevice, D3D12_GPU_VIRTUAL_ADDRESS InPageFaultAddress)
{
	if (InPageFaultAddress == 0)
	{
		return;
	}

	FD3D12ManualFence& FrameFence = InAdapter->GetFrameFence();

	UE_LOGF(LogD3D12RHI, Error, "PageFault: PageFault at VA GPUAddress \"0x%llX\" (GPU %d)", (long long)InPageFaultAddress, InDevice->GetGPUIndex());
	uint64 CachedFenceValue = FrameFence.GetCompletedFenceValue(false);
	uint64 ActualFenceValue = FrameFence.GetCompletedFenceValue(true);
	uint64 NextFenceValue = FrameFence.GetNextFenceToSignal();
	UE_LOGF(LogD3D12RHI, Error, "PageFault: Last completed frame ID: %lld (cached: %lld) - Current frame ID: %lld", ActualFenceValue, CachedFenceValue, NextFenceValue);
	UE_LOGF(LogD3D12RHI, Error, "PageFault: Logging all resource enabled: %ls", InAdapter->IsTrackingAllAllocations() ? TEXT("Yes") : TEXT("No"));

	// Try and find all current allocations near that range
	static const int64 CheckRangeRadius = 16 * 1024 * 1024;
	TArray<FD3D12Adapter::FAllocatedResourceResult> OverlappingResources;
	InAdapter->FindResourcesNearGPUAddress(InPageFaultAddress, CheckRangeRadius, OverlappingResources);
	UE_LOGF(LogD3D12RHI, Error, "PageFault: Found %d active tracked resources in %3.2f MB range of page fault address", OverlappingResources.Num(), CheckRangeRadius / (1024.0f * 1024));
	if (OverlappingResources.Num() > 0)
	{
		uint32 PrintCount = FMath::Min(OverlappingResources.Num(), 100);
		for (uint32 Index = 0; Index < PrintCount; ++Index)
		{
			FD3D12Adapter::FAllocatedResourceResult OverlappingResource = OverlappingResources[Index];
			D3D12_GPU_VIRTUAL_ADDRESS ResourceAddress = OverlappingResource.Allocation->GetGPUVirtualAddress();

			const FD3D12Resource* Resource = OverlappingResource.Allocation->GetResource();
			FString ResourceDescString = MakeResourceDescDebugString(Resource->GetDesc());

			UE_LOGF(LogD3D12RHI, Error, "\tGPU Address: [0x%llX .. 0x%llX] - Size: %lld bytes, %3.2f MB - Distance to page fault: %lld bytes, %3.2f MB - Transient: %d - Name: %ls - Desc: %ls",
				(uint64)ResourceAddress,
				(uint64)ResourceAddress + OverlappingResource.Allocation->GetSize(),
				OverlappingResource.Allocation->GetSize(),
				OverlappingResource.Allocation->GetSize() / (1024.0f * 1024), 
				OverlappingResource.Distance,
				OverlappingResource.Distance / (1024.0f * 1024), 
				OverlappingResource.Allocation->IsTransient(), 
				*Resource->GetName().ToString(),
				*ResourceDescString);
		}
	}

	// Try and find all current heaps containing the page fault address
	TArray<FD3D12Heap*> OverlappingHeaps;
	InAdapter->FindHeapsContainingGPUAddress(InPageFaultAddress, OverlappingHeaps);
	UE_LOGF(LogD3D12RHI, Error, "PageFault: Found %d active heaps containing page fault address", OverlappingHeaps.Num());
	for (int32 Index = 0; Index < OverlappingHeaps.Num(); ++Index)
	{
		FD3D12Heap* Heap = OverlappingHeaps[Index];
		UE_LOGF(LogD3D12RHI, Error, "\tGPU Address: \"0x%llX\" - Size: %3.2f MB - Name: %ls",
			(long long)Heap->GetGPUVirtualAddress(), Heap->GetHeapDesc().SizeInBytes / (1024.0f * 1024), *(Heap->GetName().ToString()));
	}

	// Try and find all released allocations within the faulting address
	TArray<FD3D12Adapter::FReleasedAllocationData> ReleasedResources;
	InAdapter->FindReleasedAllocationData(InPageFaultAddress, ReleasedResources);
	UE_LOGF(LogD3D12RHI, Error, "PageFault: Found %d released resources containing the page fault address during last 100 frames", ReleasedResources.Num());
	if (ReleasedResources.Num() > 0)
	{
		uint32 PrintCount = FMath::Min(ReleasedResources.Num(), 100);
		for (uint32 Index = 0; Index < PrintCount; ++Index)
		{
			FD3D12Adapter::FReleasedAllocationData& AllocationData = ReleasedResources[Index];

			FString ResourceDescString = MakeResourceDescDebugString(AllocationData.ResourceDesc);

			UE_LOGF(LogD3D12RHI, Error, "\tGPU Address: [0x%llX .. 0x%llX] - Size: %lld bytes, %3.2f MB - FrameID: %4lld - DefragFree: %d - Transient: %d - Heap: %d - Name: %ls - Desc: %ls",
				(uint64)AllocationData.GPUVirtualAddress,
				(uint64)AllocationData.GPUVirtualAddress + AllocationData.AllocationSize,
				AllocationData.AllocationSize,
				AllocationData.AllocationSize / (1024.0f * 1024),
				AllocationData.ReleasedFrameID,
				AllocationData.bDefragFree,
				AllocationData.bTransient,
				AllocationData.bHeap,
				*AllocationData.ResourceName.ToString(),
				*ResourceDescString);
		}
	}
}

} // namespace D3D12RHI

void LogMemoryStats(FD3D12Adapter* InAdapter)
{	
	const FRHIMemoryStats& MemoryStats = InAdapter->GetMemoryStats();

	UE_LOGF(LogD3D12RHI, Error, "Video Memory Stats from frame ID %lld:", InAdapter->GetMemoryStatsUpdateFrame());
	UE_LOGF(LogD3D12RHI, Error, "\tLocal Budget:\t%7.2f MB", MemoryStats.BudgetLocal / (1024.0f * 1024));
	UE_LOGF(LogD3D12RHI, Error, "\tLocal Used:\t%7.2f MB", MemoryStats.UsedLocal / (1024.0f * 1024));
	UE_LOGF(LogD3D12RHI, Error, "\tSystem Budget:\t%7.2f MB", MemoryStats.BudgetSystem / (1024.0f * 1024));
	UE_LOGF(LogD3D12RHI, Error, "\tSystem Used:\t%7.2f MB", MemoryStats.UsedSystem / (1024.0f * 1024));
}

#endif  // PLATFORM_WINDOWS

extern CORE_API bool GIsGPUCrashed;

void FD3D12DynamicRHI::TerminateOnOutOfMemory(ID3D12Device* InDevice, HRESULT D3DResult, bool bCreatingTextures)
{
#if PLATFORM_WINDOWS
	// send telemetry event with current adapter's memory info
	FD3D12Adapter* Adapter = nullptr;
	ForEachDevice(InDevice, [&](FD3D12Device* IterationDevice)
	{
		if (InDevice == IterationDevice->GetDevice())
		{
			Adapter = IterationDevice->GetParentAdapter();
		}
	});

	// if InDevice == nullptr, just pick the chosen adapter
	if (!Adapter && ChosenAdapter.IsValid())
	{
		check(!InDevice);
		Adapter = ChosenAdapter.Get();
	}

	if (Adapter)
	{
		const auto& CurrentMemoryStats = Adapter->GetMemoryStats();
		FCoreDelegates::GetGPUOutOfMemoryDelegate().Broadcast(CurrentMemoryStats.BudgetLocal, CurrentMemoryStats.UsedLocal);
	}

	if (!FApp::IsUnattended())
	{
		if (bCreatingTextures)
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *LOCTEXT("OutOfVideoMemoryTextures", "Out of video memory trying to allocate a texture! Make sure your video card has the minimum required memory, try lowering the resolution and/or closing other applications that are running. Exiting...").ToString(), TEXT("Error"));
		}
		else
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *NSLOCTEXT("D3D12RHI", "OutOfMemory", "Out of video memory trying to allocate a rendering resource. Make sure your video card has the minimum required memory, try lowering the resolution and/or closing other applications that are running. Exiting...").ToString(), TEXT("Error"));
		}
	}

#if STATS
#if WITH_ENGINE
	GetRendererModule().DebugLogOnCrash();
#endif
#endif

	static IConsoleVariable* GPUCrashOOM = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GPUCrashOnOutOfMemory"));
	const bool bGPUCrashOOM = GPUCrashOOM && GPUCrashOOM->GetInt();
	// If no device provided then log the memory information for each device.
	ForEachDevice(InDevice, [&](FD3D12Device* IterationDevice)
	{
		FD3D12Adapter* Adapter = IterationDevice->GetParentAdapter();
		LogMemoryStats(Adapter);
	});

	// Also log Windows memory stats.
	FPlatformMemory::DumpStats(*GLog);
	FPlatformMemory::bIsOOM = true;

	UE_LOGF(LogD3D12RHI, Fatal, "Out of video memory trying to allocate a rendering resource");
	if (!bGPUCrashOOM)
	{
		// Exit silently without reporting a crash because an OOM is not necessarily our fault		
		FPlatformMisc::RequestExit(true, TEXT("D3D12Util.TerminateOnOutOfMemory"));
	}

#else // PLATFORM_WINDOWS
	UE_LOGF(LogInit, Fatal, "Out of video memory trying to allocate a rendering resource");
#endif // !PLATFORM_WINDOWS
}

#if WITH_RHI_BREADCRUMBS
void FD3D12DynamicRHI::DumpActiveBreadcrumbs(FRHIBreadcrumbState::EVerbosity Verbosity)
{
	check(IsInInterruptThread());

	if (UE::RHI::UseGPUCrashBreadcrumbs())
	{
		TMap<FRHIBreadcrumbState::FQueueID, TArray<FRHIBreadcrumbRange>> QueueRanges;
		FRHIBreadcrumbState BreadcrumbState;

		ForEachQueue([&](FD3D12Queue& Queue)
		{
			if (!Queue.DiagnosticBuffer)
				return;

			uint32 const DeviceIndex = Queue.Device->GetGPUIndex();

			ERHIPipeline Pipeline;
			switch (Queue.QueueType)
			{
			default: return; // Skip pipelines that the RHI doesn't handle
			case ED3D12QueueType::Direct: Pipeline = ERHIPipeline::Graphics; break;
			case ED3D12QueueType::Async: Pipeline = ERHIPipeline::AsyncCompute; break;
			}

			TArray<FRHIBreadcrumbRange>& Ranges = QueueRanges.Add({ DeviceIndex, Pipeline });

			// Extract the breadcrumb ranges for the payloads still in the interrupt queue.
			for (FD3D12Payload* Payload : Queue.PendingInterrupt)
			{
				if (Payload->BreadcrumbRange)
				{
					Ranges.AddUnique(Payload->BreadcrumbRange);
				}
			}

			BreadcrumbState.Devices[DeviceIndex].Pipelines[Pipeline].MarkerOut = Queue.DiagnosticBuffer->ReadMarkerOut();
			BreadcrumbState.Devices[DeviceIndex].Pipelines[Pipeline].MarkerIn = Queue.DiagnosticBuffer->ReadMarkerIn();
		});

		// Traverse the breadcrumb tree and log active GPU work
		if (!QueueRanges.IsEmpty())
		{
			BreadcrumbState.DumpActiveBreadcrumbs(QueueRanges, Verbosity);
		}
	}
}
#endif // WITH_RHI_BREADCRUMBS

void FD3D12DynamicRHI::OutputGPUCrashReport(FTextBuilder& ErrorMessage)
{
	// Mark critical and gpu crash
	GIsCriticalError = true;
	GIsGPUCrashed = true;

	// Log which devices were removed and their reason strings
	{
		FString RemovedReasons;
		TConstArrayView<FD3D12Device*> Devices = GetAdapter().GetDevices();
		for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
		{
			HRESULT Reason = Devices[DeviceIndex]->GetDevice()->GetDeviceRemovedReason();
			if (FAILED(Reason))
			{
				FString ReasonString = GetD3D12DeviceHungErrorString(Reason);
				RemovedReasons += FString::Printf(TEXT("\r\n\t- Device %d Removed: %s"), DeviceIndex, *ReasonString);
			}
			else
			{
				RemovedReasons += FString::Printf(TEXT("\r\n\t- Device %d OK (no device removed reason)"), DeviceIndex);
			}
		}

		UE_LOGF(LogD3D12RHI, Error, "GPU crash detected:%ls\r\n", *RemovedReasons);
	}


	// Log shader asserts / prints
	{
		FString ShaderDiagnostics;
		ForEachQueue([&](FD3D12Queue& Queue)
		{
			if (!Queue.DiagnosticBuffer)
				return;

			ShaderDiagnostics += Queue.DiagnosticBuffer->GetShaderDiagnosticMessages(Queue.Device->GetGPUIndex(), Queue.QueueIndex, GetD3DCommandQueueTypeName(Queue.QueueType));
		});

		if (!ShaderDiagnostics.IsEmpty())
		{
			UE_LOGF(LogD3D12RHI, Error, "Shader diagnostic messages and asserts:%ls\r\n", *ShaderDiagnostics);
		}
	}

#if WITH_RHI_BREADCRUMBS
	//
	// Log RHI breadcrumb data
	// 
	// Don't collect breadcrumb ranges if we don't have breadcrumbs enabled. The breadcrumbs
	// will not have meaningful GPU state information because GPU markers are not written.
	//
	DumpActiveBreadcrumbs(FRHIBreadcrumbState::EVerbosity::Error);
#endif

#if NV_AFTERMATH
	TArray<UE::RHICore::Nvidia::Aftermath::FCrashResult> AftermathResults;
	UE::RHICore::Nvidia::Aftermath::OnGPUCrash(AftermathResults);

	for (const UE::RHICore::Nvidia::Aftermath::FCrashResult& AftermathResult : AftermathResults)
	{
		if (AftermathResult.GPUFaultAddress.IsSet())
		{
			ForEachDevice(nullptr, [&](FD3D12Device* Device)
			{
				D3D12RHI::LogPageFaultData(Device->GetParentAdapter(), Device, D3D12_GPU_VIRTUAL_ADDRESS(*AftermathResult.GPUFaultAddress));
			});
		}
	}
#endif
#if INTEL_GPU_CRASH_DUMPS
	UE::RHICore::Intel::GPUCrashDumps::OnGPUCrash();
#endif

#if PLATFORM_WINDOWS
	ForEachDevice(nullptr, [&](FD3D12Device* Device)
	{
		D3D12_GPU_VIRTUAL_ADDRESS PageFaultAddress = 0;
		bool bIsTrackingAllAllocations = Device->GetParentAdapter()->IsTrackingAllAllocations();
		if (!LogDREDData<FDred_1_2>(Device->GetDevice(), bIsTrackingAllAllocations, PageFaultAddress))
		{
			if (!LogDREDData<FDred_1_1>(Device->GetDevice(), bIsTrackingAllAllocations, PageFaultAddress))
			{
				UE_LOGF(LogD3D12RHI, Error, "DRED: could not find DRED data (might not be enabled or available). Run with -dred or -gpucrashdebugging to enable dred if available.");
			}						
		}

		FD3D12Adapter* Adapter = Device->GetParentAdapter();
		LogPageFaultData(Adapter, Device, PageFaultAddress);
		LogMemoryStats(Adapter);
	});
#endif  // PLATFORM_WINDOWS

	// Make sure the log is flushed
	GLog->Panic();

	// Build the error message
	ErrorMessage.AppendLine(LOCTEXT("GPU Crashed", "GPU Crashed or D3D Device Removed.\n"));
	ErrorMessage.AppendLine(LOCTEXT("GPU Crash Debugging enabled", "Check log for GPU state information."));
#if NV_AFTERMATH
	for (const UE::RHICore::Nvidia::Aftermath::FCrashResult& AftermathResult : AftermathResults)
	{
		if (AftermathResult.DumpPath.IsSet())
		{
			FFormatOrderedArguments Args;
			Args.Add(FText::FromString(*AftermathResult.DumpPath));
			ErrorMessage.AppendLineFormat(LOCTEXT("GPU CrashDump", "\nA GPU mini dump has been written to \"{0}\"."), Args);
		}
	}
#endif
}

void FD3D12DynamicRHI::TerminateOnGPUCrash(TOptional<HRESULT> D3DResult)
{
	// Include the D3D HRESULT that alerted the application of the GPU crash and the device removal reason, if present.
	// In some cases, such as when the runtime signals the device is lost through fence values, there is no HRESULT
	// but there might be a device removal reason.
	if (D3DResult.IsSet())
	{
		FGenericCrashContext::SetEngineData(TEXT("GPUCrash.D3DResult"), FString::FromInt(*D3DResult));
	}

	TConstArrayView<FD3D12Device*> Devices = GetAdapter().GetDevices();
	for (FD3D12Device* Device : Devices)
	{
		HRESULT Reason = Device->GetDevice()->GetDeviceRemovedReason();
		if (Reason != S_OK)
		{
			FGenericCrashContext::SetEngineData(TEXT("GPUCrash.D3DDeviceRemovedReason"), FString::FromInt(Reason));

			// Just include one device's removal reason in the crash report payload.
			break;
		}
	}

#if WITH_NVAPI_RAYTRACING_VALIDATION
	FlushRaytracingValidationMessages();
#endif

	FTextBuilder ErrorMessage;
	OutputGPUCrashReport(ErrorMessage);

#if WITH_ENGINE
	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine)
	{
		GameEngine->OnGPUCrash();
	}
#endif

	// Show message box or trace information
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!FApp::IsUnattended() && !IsDebuggerPresent())
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage.ToText().ToString(), TEXT("Error"));
	}
	else
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		UE_LOGF(LogD3D12RHI, D3D12RHI_GPU_CRASH_LOG_VERBOSITY, "%ls", *ErrorMessage.ToText().ToString());
	}

	// hard break here when the debugger is attached
	if (IsDebuggerPresent())
	{
		UE_DEBUG_BREAK();
	}

#if PLATFORM_WINDOWS
	ReportGPUCrash(TEXT("GPU Crash dump Triggered"), nullptr);
#endif

	// Force shutdown, we can't do anything useful anymore.
	FPlatformMisc::RequestExit(true, TEXT("D3D12Util.TerminateOnGPUCrash"));
}

// It's possible for multiple threads to catch GPU crashes or other D3D errors at the same time. Make sure we only log the error once
// by acquiring this critical section inside HandleFailedD3D12Result (and never releasing it, because those functions don't return).
static FCriticalSection GD3DCallFailedCS;

void FD3D12DynamicRHI::HandleFailedD3D12Result(HRESULT D3DResult, ID3D12Device* Device, bool bCreatingTextures, const TCHAR* Message)
{
	auto LogMessage = [Message](bool bFatal)
	{
		if (bFatal)
		{
			UE_LOGF(LogD3D12RHI, Fatal, "%ls", Message);
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "%ls", Message);
		}
	};

	if (D3DResult == E_OUTOFMEMORY)
	{
		// Which ever thread wins the race gets to log the OOM error
		GD3DCallFailedCS.Lock();
		LogMessage(false);

		// This function does not return.
		TerminateOnOutOfMemory(Device, D3DResult, bCreatingTextures);
	}
	else if (D3DResult == DXGI_ERROR_DEVICE_REMOVED || D3DResult == DXGI_ERROR_DEVICE_HUNG || D3DResult == DXGI_ERROR_DEVICE_RESET)
	{
		if (IsInInterruptThread())
		{
			//
			// We're already on the interrupt thread. We must not block on the lock, as the interrupt thread is responsible
			// for reporting the GPU crash (we can only safely access the RHI breadcrumbs / active payloads from the interrupt thread).
			//
			// Attempt to take the crash lock to prevent other threads from log spamming, but don't give up if we fail to do so.
			//
			if (GD3DCallFailedCS.TryLock())
			{
				// We're the first thread to take the lock... log the error
				LogMessage(false);
			}

			// This function does not return.
			TerminateOnGPUCrash(D3DResult);
		}
		else
		{
			// Take the lock to ensure we report the error only once
			GD3DCallFailedCS.Lock();
			LogMessage(false);

			// The interrupt thread must be the one to handle DXGI_ERROR_DEVICE_REMOVED etc.
			// This function does not return.
			ProcessInterruptQueueOnGPUCrash(D3DResult);
		}
	}
	else
	{
		// For all other errors, take the lock to make sure we only report once
		GD3DCallFailedCS.Lock();
		LogMessage(false);
	}

	//
	// We'll end up here for any D3D error not covered above, or if any of those functions happen to return (they shouldn't do).
	//

	// Make sure the log is flushed!
	GLog->Panic();

	// Make one final (fatal) log attempt
	LogMessage(true);

	// Force shutdown, we can't do anything useful anymore.
	GIsCriticalError = true;
	FPlatformMisc::RequestExit(true, TEXT("D3D12Util.HandleFailedD3D12Result"));
}

namespace D3D12RHI
{
	void VerifyD3D12Result(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D12Device* Device, FString Message)
	{
		const FString& ErrorString = GetD3D12ErrorString(D3DResult, Device);

		FD3D12DynamicRHI::GetD3DRHI()->HandleFailedD3D12Result(D3DResult, Device, false, *FString::Printf(
			TEXT("%s failed \n at %s:%u \n with error %s\n%s"), 
			ANSI_TO_TCHAR(Code),
			ANSI_TO_TCHAR(Filename),
			Line,
			*ErrorString,
			*Message
		));
	}

	void VerifyD3D12CreateTextureResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, const D3D12_RESOURCE_DESC& TextureDesc, ID3D12Device* Device)
	{
		const FString ErrorString = GetD3D12ErrorString(D3DResult, Device);
		const TCHAR* D3DFormatString = UE::DXGIUtilities::GetFormatString(TextureDesc.Format);

		FD3D12DynamicRHI::GetD3DRHI()->HandleFailedD3D12Result(D3DResult, Device, true, *FString::Printf(
			TEXT("%s failed \n at %s:%u \n with error %s, \n Size=%llix%ix%i Format=%s(0x%08X), NumMips=%i, Flags=%s"),
			ANSI_TO_TCHAR(Code),
			ANSI_TO_TCHAR(Filename),
			Line,
			*ErrorString,
			TextureDesc.Width,
			TextureDesc.Height,
			TextureDesc.DepthOrArraySize,
			D3DFormatString,
			TextureDesc.Format,
			TextureDesc.MipLevels,
			*GetD3D12TextureFlagString(TextureDesc.Flags)
		));
	}
}

void FD3D12QuantizedBoundShaderState::InitShaderRegisterCounts(const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier, const FShaderCodePackedResourceCounts& Counts, FShaderRegisterCounts& Shader, bool bAllowUAVs)
{
	uint32 MaxSRVs = MAX_SRVS;
	uint32 MaxSamplers = MAX_SAMPLERS;
	uint32 MaxUAVs = MAX_UAVS;
	uint32 MaxCBs = MAX_CBS;

	// On tier 1 & 2 HW the actual descriptor table size used during the draw/dispatch must match that of the
	// root signature so we round the size up to the closest power of 2 to accomplish 2 goals: 1) keep the size of
	// the table closer to the required size to limit descriptor heap usage due to required empty descriptors,
	// 2) encourage root signature reuse by having other shader root signature table sizes fall within the size rounding.
	// Sampler and Shader resouce view table sizes must match signature on Tier 1 hardware and Constant buffer and
	// Unorded access views table sizes must match signature on tier 2 hardware. On hardware > tier 2 the actual descriptor
	// table size used during the draw/dispatch doesn't need to match the root signature size so we encourage reuse by using
	// the max size. More info here: https://learn.microsoft.com/en-us/windows/win32/direct3d12/hardware-support,
	// https://en.wikipedia.org/wiki/Feature_levels_in_Direct3D

	// To reduce the size of the root signature, we only allow UAVs for certain shaders. 
	check(bAllowUAVs || (!bAllowUAVs && Counts.NumUAVs == 0));

	if (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_1)
	{
		Shader.SamplerCount = (Counts.NumSamplers > 0) ? FMath::Min(MaxSamplers, FMath::RoundUpToPowerOfTwo(Counts.NumSamplers)) : Counts.NumSamplers;
		Shader.ShaderResourceCount = (Counts.NumSRVs > 0) ? FMath::Min(MaxSRVs, FMath::RoundUpToPowerOfTwo(Counts.NumSRVs)) : Counts.NumSRVs;
	}
	else
	{
		Shader.SamplerCount = Counts.NumSamplers > 0 ? MaxSamplers : 0;
		Shader.ShaderResourceCount = Counts.NumSRVs > 0 ? MaxSRVs : 0;
	}

	if (ResourceBindingTier <= D3D12_RESOURCE_BINDING_TIER_2)
	{
		Shader.ConstantBufferCount = (Counts.NumCBs > MAX_ROOT_CBVS) ? FMath::Min(MaxCBs, FMath::RoundUpToPowerOfTwo(Counts.NumCBs)) : Counts.NumCBs;
		Shader.UnorderedAccessCount = (Counts.NumUAVs > 0 && bAllowUAVs) ? FMath::Min(MaxUAVs, FMath::RoundUpToPowerOfTwo(Counts.NumUAVs)) : 0;
	}
	else
	{
		Shader.ConstantBufferCount = (Counts.NumCBs > MAX_ROOT_CBVS) ? MaxCBs : Counts.NumCBs;
		Shader.UnorderedAccessCount = (Counts.NumUAVs > 0 && bAllowUAVs) ? MaxUAVs : 0;
	}
}

bool NeedsAgsIntrinsicsSpace(const FD3D12ShaderData& ShaderData)
{
#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	for (const FShaderCodeVendorExtension& Extension : ShaderData.VendorExtensions)
	{
		if (Extension.VendorId == EGpuVendorId::Amd)
		{
			// https://github.com/GPUOpen-LibrariesAndSDKs/AGS_SDK/blob/master/ags_lib/hlsl/ags_shader_intrinsics_dx12.hlsl
			return true;
		}
	}
#endif

	return false;
}

bool NeedsNvIntrinsicsSpace(const FD3D12ShaderData& ShaderData)
{
#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	for (const FShaderCodeVendorExtension& Extension : ShaderData.VendorExtensions)
	{
		if (Extension.VendorId == EGpuVendorId::Nvidia)
		{
			// https://developer.nvidia.com/unlocking-gpu-intrinsics-hlsl
			return true;
		}
	}
#endif

	return false;
}

static void SetBoundShaderStateFlags(FD3D12QuantizedBoundShaderState& OutQBSS, const FD3D12ShaderData* ShaderData)
{
	if (ShaderData)
	{
		OutQBSS.bUseDiagnosticBuffer |= ShaderData->UsesDiagnosticBuffer();
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		OutQBSS.bUseDirectlyIndexedResourceHeap |= ShaderData->UsesBindlessResources();
		OutQBSS.bUseDirectlyIndexedSamplerHeap |= ShaderData->UsesBindlessSamplers();
#endif
		if (GRHISupportsShaderRootConstants)
		{
			OutQBSS.bUseRootConstants |= ShaderData->UsesRootConstants();
		}
	}
}

static void QuantizeBoundShaderStateCommon(
	FD3D12QuantizedBoundShaderState& OutQBSS,
	const FD3D12ShaderData* ShaderData,
	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier,
	EShaderVisibility ShaderVisibility,
	bool bAllowUAVs = false
)
{
	if (ShaderData)
	{
		FD3D12QuantizedBoundShaderState::InitShaderRegisterCounts(ResourceBindingTier, ShaderData->ResourceCounts, OutQBSS.RegisterCounts[ShaderVisibility], bAllowUAVs);
		OutQBSS.bNeedsAgsIntrinsicsSpace |= NeedsAgsIntrinsicsSpace(*ShaderData);
		OutQBSS.bNeedsNvIntrinsicsSpace |= NeedsNvIntrinsicsSpace(*ShaderData);
	}

	SetBoundShaderStateFlags(OutQBSS, ShaderData);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
static bool IsCompatibleWithBindlessSamplers(const FD3D12ShaderData* ShaderData)
{
	if (ensure(ShaderData))
	{
		return ShaderData->UsesBindlessSamplers()
			|| ShaderData->ResourceCounts.NumSamplers == 0;
	}
	return true;
}

static bool IsCompatibleWithBindlessResources(const FD3D12ShaderData* ShaderData)
{
	if (ensure(ShaderData))
	{
		return ShaderData->UsesBindlessResources()
			|| (ShaderData->ResourceCounts.NumSRVs + ShaderData->ResourceCounts.NumUAVs) == 0;
	}
	return true;
}
#endif

#if USE_STATIC_ROOT_SIGNATURE

inline bool BSSUsesRootConstants(const FBoundShaderStateInput& BSS)
{
	if (!GRHISupportsShaderRootConstants)
	{
		return false;
	}

	TArray<const FD3D12ShaderData*, TInlineAllocator<5>> ShaderData;

	ShaderData.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetVertexShader()));
#if PLATFORM_SUPPORTS_MESH_SHADERS
	ShaderData.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetMeshShader()));
	ShaderData.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetAmplificationShader()));
#endif
	ShaderData.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetPixelShader()));
	ShaderData.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetGeometryShader()));

	bool bUsesRootConstants = false;
	for (int32 DataIndex = 0; DataIndex < ShaderData.Num(); ++DataIndex)
	{
		if (ShaderData[DataIndex] == nullptr)
		{
			continue;
		}

		bUsesRootConstants = EnumHasAnyFlags(ShaderData[DataIndex]->ResourceCounts.UsageFlags, EShaderResourceUsageFlags::RootConstants);

		if (bUsesRootConstants)
		{
			break;
		}
	}

	return bUsesRootConstants;
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
inline bool BSSUsesBindless(const FBoundShaderStateInput& BSS)
{
	if (!GRHIGlobals.bSupportsBindless)
	{
		return false;
	}

	TArray<const FD3D12ShaderData*, TInlineAllocator<5>> ShaderDatas;

	ShaderDatas.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetVertexShader()));
#if PLATFORM_SUPPORTS_MESH_SHADERS
	ShaderDatas.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetMeshShader()));
	ShaderDatas.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetAmplificationShader()));
#endif
	ShaderDatas.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetPixelShader()));
	ShaderDatas.Add(FD3D12DynamicRHI::ResourceCast(BSS.GetGeometryShader()));

	bool bAnyBindless = false;
	bool bAnyBindful = false;
	for (const FD3D12ShaderData* ShaderData : ShaderDatas)
	{
		if (ShaderData)
		{
			if (ShaderData->UsesBindlessResources())
			{
				// These should both be set at the same time for static root signatures
				checkSlow(ShaderData->UsesBindlessSamplers());
				bAnyBindless = true;
			}
			else
			{
				bAnyBindful = true;
			}
		}
	}

	check(bAnyBindful != bAnyBindless);

	return bAnyBindless;
}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

#endif
	
const FD3D12RootSignature* FD3D12Adapter::GetRootSignature(const FBoundShaderStateInput& BSS)
{
#if USE_STATIC_ROOT_SIGNATURE

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BSSUsesBindless(BSS))
	{
		if (BSSUsesRootConstants(BSS))
		{
			return &StaticBindlessGraphicsWithConstantsRootSignature;
		}
		else
		{
			return &StaticBindlessGraphicsRootSignature;
		}
	}
	else
#endif
	{
		if (BSSUsesRootConstants(BSS))
		{
			return &StaticGraphicsWithConstantsRootSignature;
		}
		else
		{
			return &StaticGraphicsRootSignature;
		}
	}

#else //! USE_STATIC_ROOT_SIGNATURE

	// BSS quantizer. There is a 1:1 mapping of quantized bound shader state objects to root signatures.
	// The objective is to allow a single root signature to represent many bound shader state objects.
	// The bigger the quantization step sizes, the fewer the root signatures.
	FD3D12QuantizedBoundShaderState QBSS{};

	QBSS.bAllowIAInputLayout = BSS.VertexDeclarationRHI != nullptr;	// Does the root signature need access to vertex buffers?

	const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier = GetResourceBindingTier();

	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetVertexShader()),        ResourceBindingTier, SV_Vertex, true /*bAllowUAVs*/);
#if PLATFORM_SUPPORTS_MESH_SHADERS
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetMeshShader()),          ResourceBindingTier, SV_Mesh, true /*bAllowUAVs*/);
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetAmplificationShader()), ResourceBindingTier, SV_Amplification, true /*bAllowUAVs*/);
#endif
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetPixelShader()),         ResourceBindingTier, SV_Pixel, true /*bAllowUAVs*/);
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetGeometryShader()),      ResourceBindingTier, SV_Geometry);

#if DO_CHECK && PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (QBSS.bUseDirectlyIndexedResourceHeap || QBSS.bUseDirectlyIndexedSamplerHeap)
	{
		struct FGenericShaderPair
		{
			const FD3D12ShaderData* Data;
			const FRHIGraphicsShader* RHI;
		};
		const FGenericShaderPair ShaderDatas[] =
		{
			{ FD3D12DynamicRHI::ResourceCast(BSS.GetVertexShader()), BSS.GetVertexShader() },
#if PLATFORM_SUPPORTS_MESH_SHADERS
			{ FD3D12DynamicRHI::ResourceCast(BSS.GetMeshShader()), BSS.GetMeshShader() },
			{ FD3D12DynamicRHI::ResourceCast(BSS.GetAmplificationShader()), BSS.GetAmplificationShader() },
#endif
			{ FD3D12DynamicRHI::ResourceCast(BSS.GetPixelShader()), BSS.GetPixelShader() },
			{ FD3D12DynamicRHI::ResourceCast(BSS.GetGeometryShader()), BSS.GetGeometryShader() },
		};

		for (const FGenericShaderPair& ShaderPair : ShaderDatas)
		{
			if (ShaderPair.RHI)
			{
				if (QBSS.bUseDirectlyIndexedResourceHeap)
				{
					checkf(IsCompatibleWithBindlessResources(ShaderPair.Data), TEXT("Mismatched dynamic resource usage. %s doesn't support binding with stages that use dynamic resources"), ShaderPair.RHI->GetShaderName());
				}
				if (QBSS.bUseDirectlyIndexedSamplerHeap)
				{
					checkf(IsCompatibleWithBindlessSamplers(ShaderPair.Data), TEXT("Mismatched dynamic resource usage. %s doesn't support binding with stages that use dynamic samplers"), ShaderPair.RHI->GetShaderName());
				}
			}
		}
	}
#endif

	return RootSignatureManager.GetRootSignature(QBSS);

#endif //! USE_STATIC_ROOT_SIGNATURE
}

const FD3D12RootSignature* FD3D12Adapter::GetRootSignature(const FD3D12ComputeShader* ComputeShader)
{
#if USE_STATIC_ROOT_SIGNATURE

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (ComputeShader->UsesBindlessResources() && GRHIGlobals.bSupportsBindless)
	{
		if (ComputeShader->UsesRootConstants() && GRHISupportsShaderRootConstants)
		{
			return &StaticBindlessComputeWithConstantsRootSignature;
		}
		else
		{
			return &StaticBindlessComputeRootSignature;
		}
	}
	else
#endif
	{
		if (ComputeShader->UsesRootConstants() && GRHISupportsShaderRootConstants)
		{
			return &StaticComputeWithConstantsRootSignature;
		}
		else
		{
			return &StaticComputeRootSignature;
		}
	}

#else //! USE_STATIC_ROOT_SIGNATURE

	check(ComputeShader);

	// BSS quantizer. There is a 1:1 mapping of quantized bound shader state objects to root signatures.
	// The objective is to allow a single root signature to represent many bound shader state objects.
	// The bigger the quantization step sizes, the fewer the root signatures.
	FD3D12QuantizedBoundShaderState QBSS{};

	QuantizeBoundShaderStateCommon(QBSS, ComputeShader, GetResourceBindingTier(), SV_All, true /*bAllowUAVs*/);

	check(QBSS.bAllowIAInputLayout == false); // No access to vertex buffers needed

	return RootSignatureManager.GetRootSignature(QBSS);

#endif //! USE_STATIC_ROOT_SIGNATURE
}

const FD3D12RootSignature* FD3D12Adapter::GetRootSignature(ERHIShaderBundleMode ShaderBundleMode, bool bBindless)
{
	// Determine the root signature for the bundle
#if USE_STATIC_ROOT_SIGNATURE
	if (ShaderBundleMode == ERHIShaderBundleMode::CS)
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindless)
		{
			return &StaticBindlessComputeWithConstantsRootSignature;
		}
#endif
		return &StaticComputeWithConstantsRootSignature;
	}

	if (ShaderBundleMode == ERHIShaderBundleMode::MSPS || ShaderBundleMode == ERHIShaderBundleMode::VSPS)
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindless)
		{
			return &StaticBindlessGraphicsWithConstantsRootSignature;
		}
#endif

		return &StaticGraphicsWithConstantsRootSignature;
	}
#endif // USE_STATIC_ROOT_SIGNATURE

	checkNoEntry();
	return nullptr;
}

const FD3D12RootSignature* FD3D12Adapter::GetRootSignature(const FD3D12WorkGraphShader* WorkGraphShader)
{
	check(WorkGraphShader);

	FD3D12QuantizedBoundShaderState QBSS{};
	QuantizeBoundShaderStateCommon(QBSS, WorkGraphShader, GetResourceBindingTier(), SV_All, true /*bAllowUAVs*/);

	QBSS.RootSignatureType = WorkGraphShader->GetFrequency() == SF_WorkGraphRoot ? RS_WorkGraphGlobal : RS_WorkGraphLocalCompute;
	check(QBSS.bAllowIAInputLayout == false); // No access to vertex buffers needed

	return RootSignatureManager.GetRootSignature(QBSS);
}

const FD3D12RootSignature* FD3D12Adapter::GetGlobalWorkGraphRootSignature(const FRHIShaderBindingLayout& ShaderBindingLayout)
{
	FD3D12QuantizedBoundShaderState QBSS{};
	FShaderRegisterCounts& QBSSRegisterCounts = QBSS.RegisterCounts[SV_All];

	QBSS.ShaderBindingLayout = ShaderBindingLayout;
	QBSS.RootSignatureType = RS_WorkGraphGlobal;
	QBSS.bUseDiagnosticBuffer = true;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	QBSS.bUseDirectlyIndexedResourceHeap = GetBindlessDescriptorAllocator().AreResourcesBindless();
	QBSS.bUseDirectlyIndexedSamplerHeap = GetBindlessDescriptorAllocator().AreSamplersBindless();
#endif

	QBSSRegisterCounts.SamplerCount = MAX_SAMPLERS;
	QBSSRegisterCounts.ShaderResourceCount = MAX_SRVS;
	QBSSRegisterCounts.ConstantBufferCount = MAX_CBS;
	QBSSRegisterCounts.UnorderedAccessCount = MAX_UAVS;

	return RootSignatureManager.GetRootSignature(QBSS);
}

const FD3D12RootSignature* FD3D12Adapter::GetWorkGraphGraphicsRootSignature(const FBoundShaderStateInput& BSS)
{
	FD3D12QuantizedBoundShaderState QBSS{};

	const D3D12_RESOURCE_BINDING_TIER ResourceBindingTier = GetResourceBindingTier();

#if PLATFORM_SUPPORTS_MESH_SHADERS
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetWorkGraphShader()), ResourceBindingTier, SV_Mesh);
#endif
	QuantizeBoundShaderStateCommon(QBSS, FD3D12DynamicRHI::ResourceCast(BSS.GetPixelShader()), ResourceBindingTier, SV_Pixel, true /*bAllowUAVs*/);

	QBSS.RootSignatureType = RS_WorkGraphLocalRaster;

	return RootSignatureManager.GetRootSignature(QBSS);
}

#if D3D12_RHI_RAYTRACING

const FD3D12RootSignature* FD3D12Adapter::GetGlobalRayTracingRootSignature(const FRHIShaderBindingLayout& ShaderBindingLayout)
{
#if USE_STATIC_ROOT_SIGNATURE

	return &StaticRayTracingGlobalRootSignature;

#else //!USE_STATIC_ROOT_SIGNATURE

	FD3D12QuantizedBoundShaderState QBSS{};
	FShaderRegisterCounts& QBSSRegisterCounts = QBSS.RegisterCounts[SV_All];

	QBSS.ShaderBindingLayout = ShaderBindingLayout;
	QBSS.RootSignatureType = RS_RayTracingGlobal;
	QBSS.bUseDiagnosticBuffer = true;
	// On NVIDIA, the NV intrinsics space is always needed for NVAPI shader extensions.
	// On non-NVIDIA, it's still needed when the platform compiles shaders with cluster ops support
	// (PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS=1), because hit group shaders will reference g_NvidiaExt behind a dynamic branch (UsingCLAS). 
	// The root signature must include this space even on AMD/Intel so the D3D12 runtime accepts the shader bytecode.
	QBSS.bNeedsNvIntrinsicsSpace = IsRHIDeviceNVIDIA() || FDataDrivenShaderPlatformInfo::GetSupportsRayTracingClusterOps(GMaxRHIShaderPlatform);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	QBSS.bUseDirectlyIndexedResourceHeap = GetBindlessDescriptorAllocator().AreResourcesBindless();
	QBSS.bUseDirectlyIndexedSamplerHeap = GetBindlessDescriptorAllocator().AreSamplersBindless();
#endif

	QBSSRegisterCounts.SamplerCount = MAX_SAMPLERS;
	QBSSRegisterCounts.ShaderResourceCount = MAX_SRVS;
	QBSSRegisterCounts.ConstantBufferCount = MAX_CBS;
	QBSSRegisterCounts.UnorderedAccessCount = MAX_UAVS;

	return RootSignatureManager.GetRootSignature(QBSS);

#endif //! USE_STATIC_ROOT_SIGNATURE
}

const FD3D12RootSignature* FD3D12Adapter::GetLocalRootSignature(const FD3D12RayTracingShader* RayTracingShader)
{
#if USE_STATIC_ROOT_SIGNATURE

	switch (RayTracingShader->GetFrequency())
	{
	default:
		checkNoEntry(); // Unexpected shader target frequency
		return nullptr;

	case SF_RayGen:
		return &StaticRayTracingGlobalRootSignature;

	case SF_RayHitGroup:
	case SF_RayCallable:
	case SF_RayMiss:
		return &StaticRayTracingLocalRootSignature;
	}

#else //! USE_STATIC_ROOT_SIGNATURE

	FD3D12QuantizedBoundShaderState QBSS{};

	FShaderRegisterCounts& QBSSRegisterCounts = QBSS.RegisterCounts[SV_All];

	switch (RayTracingShader->GetFrequency())
	{
	case SF_RayGen:
		// Ray gen only uses global root signature and needs the RHIShaderBindingLayout which is provided through the RTPSO initializer 
		// and verified against hash stored in the RHIShader data
		checkNoEntry();
		break;
	case SF_RayHitGroup:
	case SF_RayCallable:
	case SF_RayMiss:
	{
		// Local root signature is used for hit group shaders, using the exact number of resources to minimize shader binding table record size.
		check(RayTracingShader);
		const FShaderCodePackedResourceCounts& Counts = RayTracingShader->ResourceCounts;

		QBSS.RootSignatureType = RS_RayTracingLocal;

		QBSSRegisterCounts.SamplerCount = Counts.NumSamplers;
		QBSSRegisterCounts.ShaderResourceCount = Counts.NumSRVs;
		QBSSRegisterCounts.ConstantBufferCount = Counts.NumCBs;
		QBSSRegisterCounts.UnorderedAccessCount = Counts.NumUAVs;

		check(QBSSRegisterCounts.SamplerCount <= MAX_SAMPLERS);
		check(QBSSRegisterCounts.ShaderResourceCount <= MAX_SRVS);
		check(QBSSRegisterCounts.ConstantBufferCount <= MAX_CBS);
		check(QBSSRegisterCounts.UnorderedAccessCount <= MAX_UAVS);

		break;
	}
	default:
		checkNoEntry(); // Unexpected shader target frequency
	}

	SetBoundShaderStateFlags(QBSS, RayTracingShader);

	return RootSignatureManager.GetRootSignature(QBSS);

#endif //! USE_STATIC_ROOT_SIGNATURE
}
#endif // D3D12_RHI_RAYTRACING

FD3D12BoundRenderTargets::FD3D12BoundRenderTargets(FD3D12RenderTargetView** RTArray, uint32 NumActiveRTs, FD3D12DepthStencilView* DSView)
{
	FMemory::Memcpy(RenderTargetViews, RTArray, sizeof(RenderTargetViews));
	DepthStencilView = DSView;
	NumActiveTargets = NumActiveRTs;
}

FD3D12BoundRenderTargets::~FD3D12BoundRenderTargets()
{
}

void LogExecuteCommandLists(uint32 NumCommandLists, ID3D12CommandList* const* ppCommandLists)
{
	for (uint32 i = 0; i < NumCommandLists; i++)
	{
		ID3D12CommandList* const pCurrentCommandList = ppCommandLists[i];
		UE_LOGF(LogD3D12RHI, Log, "*** [tid:%08x] EXECUTE (CmdList: %016llX) %u/%u ***", FPlatformTLS::GetCurrentThreadId(), pCurrentCommandList, i + 1, NumCommandLists);
	}
}


#if ASSERT_RESOURCE_STATES
// Forward declarations are required for the template functions
// template bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12View* pView, const D3D12_RESOURCE_STATES& State);
// 
// bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12View* pView, const D3D12_RESOURCE_STATES& State)
// {
// 	// Check the view
// 	if (!pView)
// 	{
// 		// No need to check null views
// 		return true;
// 	}
// 
// 	return AssertResourceState(pCommandList, pView->GetResource(), State, pView->GetViewSubresourceSubset());
// }

bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12Resource* pResource, const D3D12_RESOURCE_STATES& State, uint32 Subresource)
{
	// Check the resource
	if (!pResource)
	{
		// No need to check null resources
		// Some dynamic SRVs haven't been mapped and updated yet so they actually don't have any backing resources.
		return true;
	}

	FD3D12ViewSubset ViewSubset(Subresource, pResource->GetMipLevels(), pResource->GetArraySize(), pResource->GetPlaneCount());
	return AssertResourceState(pCommandList, pResource, State, ViewSubset);
}

bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12Resource* pResource, const D3D12_RESOURCE_STATES& State, const FD3D12ViewSubset& ViewSubset)
{
#if PLATFORM_WINDOWS
	// Check the resource
	if (!pResource)
	{
		// No need to check null resources
		// Some dynamic SRVs haven't been mapped and updated yet so they actually don't have any backing resources.
		return true;
	}

	// Can only verify resource states if the debug layer is used
	static const bool bWithD3DDebug = GRHIGlobals.IsDebugLayerEnabled;
	if (!bWithD3DDebug)
	{
		UE_LOGF(LogD3D12RHI, Fatal, "*** AssertResourceState requires the debug layer ***");
		return false;
	}

	// Get the debug command queue
	TRefCountPtr<ID3D12DebugCommandList> pDebugCommandList;
	VERIFYD3D12RESULT(pCommandList->QueryInterface(pDebugCommandList.GetInitReference()));

	// Get the underlying resource
	ID3D12Resource* pD3D12Resource = pResource->GetResource();
	check(pD3D12Resource);

	// For each subresource in the view...
	for (uint32 SubresourceIndex : ViewSubset)
	{
		const bool bGoodState = !!pDebugCommandList->AssertResourceState(pD3D12Resource, SubresourceIndex, State);
		if (!bGoodState)
		{
			return false;
		}
	}
#endif // PLATFORM_WINDOWS

	return true;
}
#endif

//
// Stat declarations.
//

DEFINE_STAT(STAT_D3D12PresentTime);
DEFINE_STAT(STAT_D3D12CustomPresentTime);

DEFINE_STAT(STAT_D3D12NumCommandAllocators);
DEFINE_STAT(STAT_D3D12NumCommandLists);
DEFINE_STAT(STAT_D3D12NumQueryHeaps);
DEFINE_STAT(STAT_D3D12NumPSOs);
DEFINE_STAT(STAT_D3D12ExecutedCommandLists);
DEFINE_STAT(STAT_D3D12ExecutedCommandListBatches);

DEFINE_STAT(STAT_D3D12TexturesAllocated);
DEFINE_STAT(STAT_D3D12TexturesReleased);
DEFINE_STAT(STAT_D3D12CreateTextureTime);
DEFINE_STAT(STAT_D3D12LockTextureTime);
DEFINE_STAT(STAT_D3D12UnlockTextureTime);
DEFINE_STAT(STAT_D3D12CreateBufferTime);
DEFINE_STAT(STAT_D3D12CopyToStagingBufferTime);
DEFINE_STAT(STAT_D3D12LockBufferTime);
DEFINE_STAT(STAT_D3D12UnlockBufferTime);
DEFINE_STAT(STAT_D3D12CommitTransientResourceTime);
DEFINE_STAT(STAT_D3D12DecommitTransientResourceTime);

DEFINE_STAT(STAT_D3D12UAVBarriers);

DEFINE_STAT(STAT_D3D12BindlessResourceHeapsAllocated);
DEFINE_STAT(STAT_D3D12BindlessResourceHeapsActive);
DEFINE_STAT(STAT_D3D12BindlessResourceHeapsInUseByGPU);
DEFINE_STAT(STAT_D3D12BindlessResourceHeapsVersioned);
DEFINE_STAT(STAT_D3D12BindlessResourceDescriptorsInitialized);
DEFINE_STAT(STAT_D3D12BindlessResourceDescriptorsUpdated);
DEFINE_STAT(STAT_D3D12BindlessResourceGPUDescriptorsCopied);
DEFINE_STAT(STAT_D3D12BindlessResourceHeapGPUMemoryUsage);

DEFINE_STAT(STAT_D3D12NewBoundShaderStateTime);
DEFINE_STAT(STAT_D3D12CreateBoundShaderStateTime);
DEFINE_STAT(STAT_D3D12NumBoundShaderState);
DEFINE_STAT(STAT_D3D12SetBoundShaderState);

DEFINE_STAT(STAT_D3D12UpdateUniformBufferTime);

DEFINE_STAT(STAT_D3D12CommitResourceTables);
DEFINE_STAT(STAT_D3D12SetTextureInTableCalls);

DEFINE_STAT(STAT_D3D12DispatchShaderBundle);

DEFINE_STAT(STAT_D3D12ClearShaderResourceViewsTime);
DEFINE_STAT(STAT_D3D12SetShaderResourceViewTime);
DEFINE_STAT(STAT_D3D12SetUnorderedAccessViewTime);
DEFINE_STAT(STAT_D3D12CommitGraphicsConstants);
DEFINE_STAT(STAT_D3D12CommitComputeConstants);
DEFINE_STAT(STAT_D3D12SetShaderUniformBuffer);

DEFINE_STAT(STAT_D3D12ApplyStateTime);
DEFINE_STAT(STAT_D3D12ApplyStateRebuildPSOTime);
DEFINE_STAT(STAT_D3D12ApplyStateFindPSOTime);
DEFINE_STAT(STAT_D3D12ApplyStateSetSRVTime);
DEFINE_STAT(STAT_D3D12ApplyStateSetUAVTime);
DEFINE_STAT(STAT_D3D12ApplyStateSetVertexBufferTime);
DEFINE_STAT(STAT_D3D12ApplyStateSetConstantBufferTime);
DEFINE_STAT(STAT_D3D12ClearMRT);

DEFINE_STAT(STAT_D3D12ExecuteCommandListTime);
DEFINE_STAT(STAT_D3D12WaitForFenceTime);

DEFINE_STAT(STAT_D3D12MemoryCurrentTotal);
DEFINE_STAT(STAT_D3D12RenderTargets);
DEFINE_STAT(STAT_D3D12UAVTextures);
DEFINE_STAT(STAT_D3D12Textures);
DEFINE_STAT(STAT_D3D12UAVBuffers);
DEFINE_STAT(STAT_D3D12RTBuffers);
DEFINE_STAT(STAT_D3D12Buffer);
DEFINE_STAT(STAT_D3D12TransientHeaps);

DEFINE_STAT(STAT_D3D12RenderTargetStandAloneAllocated);
DEFINE_STAT(STAT_D3D12UAVTextureStandAloneAllocated);
DEFINE_STAT(STAT_D3D12TextureStandAloneAllocated);
DEFINE_STAT(STAT_D3D12UAVBufferStandAloneAllocated);
DEFINE_STAT(STAT_D3D12BufferStandAloneAllocated);

DEFINE_STAT(STAT_D3D12RenderTargetStandAloneCount);
DEFINE_STAT(STAT_D3D12UAVTextureStandAloneCount);
DEFINE_STAT(STAT_D3D12TextureStandAloneCount);
DEFINE_STAT(STAT_D3D12UAVBufferStandAloneCount);
DEFINE_STAT(STAT_D3D12BufferStandAloneCount);

DEFINE_STAT(STAT_D3D12TextureAllocatorAllocated);
DEFINE_STAT(STAT_D3D12TextureAllocatorUnused);
DEFINE_STAT(STAT_D3D12TextureAllocatorCount);

DEFINE_STAT(STAT_D3D12BufferPoolMemoryAllocated);
DEFINE_STAT(STAT_D3D12BufferPoolMemoryUsed);
DEFINE_STAT(STAT_D3D12BufferPoolMemoryFree);
DEFINE_STAT(STAT_D3D12BufferPoolAlignmentWaste);
DEFINE_STAT(STAT_D3D12BufferPoolPageCount);
DEFINE_STAT(STAT_D3D12BufferPoolFullPages);
DEFINE_STAT(STAT_D3D12BufferPoolFragmentation);
DEFINE_STAT(STAT_D3D12BufferPoolFragmentationPercentage);

DEFINE_STAT(STAT_D3D12UploadPoolMemoryAllocated);
DEFINE_STAT(STAT_D3D12UploadPoolMemoryUsed);
DEFINE_STAT(STAT_D3D12UploadPoolMemoryFree);
DEFINE_STAT(STAT_D3D12UploadPoolAlignmentWaste);
DEFINE_STAT(STAT_D3D12UploadPoolPageCount);
DEFINE_STAT(STAT_D3D12UploadPoolFullPages);

DEFINE_STAT(STAT_D3D12ReservedResourcePhysical);

DEFINE_STAT(STAT_UniqueSamplers);

DEFINE_STAT(STAT_ViewHeapChanged);
DEFINE_STAT(STAT_SamplerHeapChanged);

DEFINE_STAT(STAT_NumViewOnlineDescriptorHeaps);
DEFINE_STAT(STAT_NumSamplerOnlineDescriptorHeaps);
DEFINE_STAT(STAT_NumReuseableSamplerOnlineDescriptorTables);
DEFINE_STAT(STAT_NumReuseableSamplerOnlineDescriptors);
DEFINE_STAT(STAT_NumReservedViewOnlineDescriptors);
DEFINE_STAT(STAT_NumReservedSamplerOnlineDescriptors);
DEFINE_STAT(STAT_NumReusedSamplerOnlineDescriptors);

DEFINE_STAT(STAT_GlobalViewHeapFreeDescriptors);
DEFINE_STAT(STAT_GlobalViewHeapReservedDescriptors);
DEFINE_STAT(STAT_GlobalViewHeapUsedDescriptors);
DEFINE_STAT(STAT_GlobalViewHeapWastedDescriptors);
DEFINE_STAT(STAT_GlobalViewHeapBlockAllocations);

DEFINE_STAT(STAT_ViewOnlineDescriptorHeapMemory);
DEFINE_STAT(STAT_SamplerOnlineDescriptorHeapMemory);

DEFINE_STAT(STAT_ExplicitSamplerDescriptorHeaps);
DEFINE_STAT(STAT_ExplicitSamplerDescriptors);

DEFINE_STAT(STAT_ExplicitViewDescriptorHeaps);
DEFINE_STAT(STAT_ExplicitViewDescriptors);

DEFINE_STAT(STAT_ExplicitMaxUsedSamplerDescriptors);
DEFINE_STAT(STAT_ExplicitUsedSamplerDescriptors);
DEFINE_STAT(STAT_ExplicitUsedViewDescriptors);


//--------------------------------------------------------------------------------------------------
// BuildComputeShaderRootSignature — Build a serialized root signature from a UE shader blob
// Uses the same root signature construction logic as RHICreateComputeShader.
// Can be called without an initialized D3D12 RHI.
//--------------------------------------------------------------------------------------------------

bool BuildComputeShaderRootSignature(const TConstArrayView<uint8>& ShaderCode, D3D12_RESOURCE_BINDING_TIER ResourceBindingTier, TRefCountPtr<ID3DBlob>& OutRootSigBlob, int32& OutNativeBytecodeOffset)
{
	if (ShaderCode.Num() == 0)
	{
		return false;
	}

	// Parse the shader blob - same as InitStandardShader
	FShaderCodeReader CodeReader(ShaderCode);

	// Extract resource counts from optional data
	const FShaderCodePackedResourceCounts* PackedResourceCounts = CodeReader.FindOptionalData<FShaderCodePackedResourceCounts>();
	if (!PackedResourceCounts)
	{
		UE_LOGF(LogD3D12RHI, Warning, "BuildComputeShaderRootSignature: No resource counts found in shader blob");
		return false;
	}

	// Deserialize SRT to find where native bytecode starts
	FMemoryReaderView Ar(ShaderCode, true);
	FShaderResourceTable SRT;
	Ar << SRT;
	OutNativeBytecodeOffset = Ar.Tell();

	// Build quantized bound shader state - same as GetRootSignature(ComputeShader)
	FD3D12QuantizedBoundShaderState QBSS{};

	// Use a temporary FD3D12ShaderData to hold the extracted data
	// so we can call the existing quantization functions
	FD3D12ShaderData TempShaderData;
	TempShaderData.ResourceCounts = *PackedResourceCounts;

#if D3D12RHI_NEEDS_VENDOR_EXTENSIONS
	// Extract vendor extensions from optional data
	int32 VendorExtensionTableSize = 0;
	auto* VendorExtensionData = CodeReader.FindOptionalDataAndSize(FShaderCodeVendorExtension::Key, VendorExtensionTableSize);
	if (VendorExtensionData && VendorExtensionTableSize > 0)
	{
		FBufferReader VendorAr((void*)VendorExtensionData, VendorExtensionTableSize, false);
		VendorAr << TempShaderData.VendorExtensions;
	}
#endif

	// Root constants are always supported on D3D12 - force enable for tool context
	// where GRHISupportsShaderRootConstants is not initialized by device init
	GRHISupportsShaderRootConstants = true;

	// Quantize for compute: SV_All visibility, UAVs allowed
	QuantizeBoundShaderStateCommon(QBSS, &TempShaderData, ResourceBindingTier, SV_All, true /*bAllowUAVs*/);

	// Build root signature descriptor
	FD3D12RootSignatureDesc RootSigDesc(QBSS, ResourceBindingTier);

	// Serialize root signature
	TRefCountPtr<ID3DBlob> ErrorBlob;
	HRESULT hr = D3DX12SerializeVersionedRootSignature(
		&RootSigDesc.GetDesc(),
		D3D_ROOT_SIGNATURE_VERSION_1_1,
		OutRootSigBlob.GetInitReference(),
		ErrorBlob.GetInitReference());

	if (FAILED(hr))
	{
		UE_LOGF(LogD3D12RHI, Error, "BuildComputeShaderRootSignature: Failed to serialize root signature: 0x%08X %ls",
			hr, ErrorBlob ? ANSI_TO_TCHAR((const char*)ErrorBlob->GetBufferPointer()) : TEXT(""));
		return false;
	}

	return true;
}

//--------------------------------------------------------------------------------------------------
// BuildGraphicsShaderRootSignature -- Build a serialized root signature from multiple UE shader blobs.
// Merges resource counts across all active stages (VS, PS, GS, MS, AS).
// Can be called without an initialized D3D12 RHI.
//--------------------------------------------------------------------------------------------------

bool BuildGraphicsShaderRootSignature(const TConstArrayView<TConstArrayView<uint8>> ShaderBlobs,
	D3D12_RESOURCE_BINDING_TIER ResourceBindingTier, TRefCountPtr<ID3DBlob>& OutRootSigBlob)
{
	if (ShaderBlobs.IsEmpty())
	{
		return false;
	}

	// Merge FShaderCodePackedResourceCounts across all active shader stages.
	// The root signature must cover the union of all resources required by any stage.
	FShaderCodePackedResourceCounts MergedCounts = {};
	bool bFoundAny = false;

	for (const TConstArrayView<uint8>& ShaderCode : ShaderBlobs)
	{
		if (ShaderCode.IsEmpty())
		{
			continue;
		}

		FShaderCodeReader CodeReader(ShaderCode);

		const FShaderCodePackedResourceCounts* PackedCounts = CodeReader.FindOptionalData<FShaderCodePackedResourceCounts>();
		if (!PackedCounts)
		{
			// Not fatal -- some stages may not have resource counts (e.g. pass-through GS)
			continue;
		}

		// Merge by taking the max of each resource count across stages
		MergedCounts.NumSamplers = FMath::Max(MergedCounts.NumSamplers, PackedCounts->NumSamplers);
		MergedCounts.NumSRVs     = FMath::Max(MergedCounts.NumSRVs,     PackedCounts->NumSRVs);
		MergedCounts.NumCBs      = FMath::Max(MergedCounts.NumCBs,      PackedCounts->NumCBs);
		MergedCounts.NumUAVs     = FMath::Max(MergedCounts.NumUAVs,     PackedCounts->NumUAVs);
		MergedCounts.UsageFlags |= PackedCounts->UsageFlags;
		bFoundAny = true;
	}

	if (!bFoundAny)
	{
		UE_LOGF(LogD3D12RHI, Warning, "BuildGraphicsShaderRootSignature: No resource counts found in any shader stage.");
		return false;
	}

	// Build quantized bound shader state from merged resource counts -- same path as compute
	GRHISupportsShaderRootConstants = true;

	FD3D12ShaderData TempShaderData;
	TempShaderData.ResourceCounts = MergedCounts;
	// NOTE: VendorExtensions (AGS/NV intrinsics) are not yet extracted from graphics shader stages
	// and are therefore not forwarded to QuantizeBoundShaderStateCommon. Graphics PSOs that rely on
	// vendor intrinsics will get a root signature without the extra descriptor space those require.
	// This is acceptable for the current ASD tool scope -- support can be added when needed.

	FD3D12QuantizedBoundShaderState QBSS{};
	// Use SV_All visibility so the root signature covers all stages
	QuantizeBoundShaderStateCommon(QBSS, &TempShaderData, ResourceBindingTier, SV_All, true /*bAllowUAVs*/);

	FD3D12RootSignatureDesc RootSigDesc(QBSS, ResourceBindingTier);

	TRefCountPtr<ID3DBlob> ErrorBlob;
	HRESULT hr = D3DX12SerializeVersionedRootSignature(
		&RootSigDesc.GetDesc(),
		D3D_ROOT_SIGNATURE_VERSION_1_1,
		OutRootSigBlob.GetInitReference(),
		ErrorBlob.GetInitReference());

	if (FAILED(hr))
	{
		UE_LOGF(LogD3D12RHI, Error, "BuildGraphicsShaderRootSignature: Failed to serialize root signature: 0x%08X %ls",
			hr, ErrorBlob ? ANSI_TO_TCHAR((const char*)ErrorBlob->GetBufferPointer()) : TEXT(""));
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
