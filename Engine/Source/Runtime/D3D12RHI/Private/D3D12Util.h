// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Util.h: D3D RHI utility definitions.
	=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "D3D12RHICommon.h"
#include "D3D12Queue.h"
#include "D3D12Access.h"
#include "DXGIUtilities.h"
#include "RenderUtils.h"
#include "ShaderCore.h"

namespace D3D12RHI
{
	/**
	 * Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
	 * @param	Result - The result code to check
	 * @param	Code - The code which yielded the result.
	 * @param	Filename - The filename of the source file containing Code.
	 * @param	Line - The line number of Code within Filename.
	 */
	extern void VerifyD3D12Result(HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, ID3D12Device* Device, FString Message = FString());

	/**
	* Checks that the given result isn't a failure.  If it is, the application exits with an appropriate error message.
	* @param	Result - The result code to check
	* @param	Code - The code which yielded the result.
	* @param	Filename - The filename of the source file containing Code.
	* @param	Line - The line number of Code within Filename.
	*/
	extern void VerifyD3D12CreateTextureResult(HRESULT D3DResult, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, const D3D12_RESOURCE_DESC& TextureDesc, ID3D12Device* Device);

	/**
	 * A macro for using VERIFYD3D12RESULT that automatically passes in the code and filename/line.
	 */
#define VERIFYD3D12RESULT_LAMBDA(x, Device, Lambda)	{HRESULT hres = x; if (FAILED(hres)) { VerifyD3D12Result(hres, #x, __FILE__, __LINE__, Device, Lambda()); }}
#define VERIFYD3D12RESULT_EX(x, Device)	{HRESULT hres = x; if (FAILED(hres)) { VerifyD3D12Result(hres, #x, __FILE__, __LINE__, Device); }}
#define VERIFYD3D12RESULT(x)			{HRESULT hres = x; if (FAILED(hres)) { VerifyD3D12Result(hres, #x, __FILE__, __LINE__, nullptr); }}
#define VERIFYD3D12CREATETEXTURERESULT(x, Desc, Device) {HRESULT hres = x; if (FAILED(hres)) { VerifyD3D12CreateTextureResult(hres, #x, __FILE__, __LINE__, Desc, Device); }}

	/** Checks if given GPU virtual address corresponds to any known resource allocations and logs results */
	void LogPageFaultData(class FD3D12Adapter* InAdapter, FD3D12Device* InDevice, D3D12_GPU_VIRTUAL_ADDRESS InPageFaultAddress);
	
} // namespace D3D12RHI

using namespace D3D12RHI;

class FD3D12Buffer;
class FD3D12Resource;
class FD3D12ResourceLocation;
class FD3D12Texture;

bool ShouldSetD3D12ResourceName(const FD3D12ResourceLocation& ResourceLocation);

void SetD3D12ObjectName(ID3D12Object* Object, const TCHAR* Name);

void SetD3D12ResourceName(FD3D12ResourceLocation& ResourceLocation, const TCHAR* Name);
void SetD3D12ResourceName(FD3D12Resource* Resource, const TCHAR* Name);
void SetD3D12ResourceName(FD3D12Buffer* Resource, const TCHAR* Name);
void SetD3D12ResourceName(FD3D12Texture* Resource, const TCHAR* Name);

FString GetD312ObjectName(ID3D12Object* const Object);

enum EShaderVisibility
{
	SV_Vertex,
	SV_Pixel,
	SV_Geometry,
#if PLATFORM_SUPPORTS_MESH_SHADERS
	SV_Mesh,
	SV_Amplification,
#endif
	SV_All,
	SV_ShaderVisibilityCount
};

enum ERootSignatureType
{
	RS_Raster,
	RS_RayTracingGlobal,
	RS_RayTracingLocal,
	RS_WorkGraphGlobal,
	RS_WorkGraphLocalCompute,
	RS_WorkGraphLocalRaster,
};

struct FShaderRegisterCounts
{
	uint8 SamplerCount;
	uint8 ConstantBufferCount;
	uint8 ShaderResourceCount;
	uint8 UnorderedAccessCount;
	
	inline bool operator==(const FShaderRegisterCounts& RHS) const
	{
		return SamplerCount == RHS.SamplerCount &&
			ConstantBufferCount == RHS.ConstantBufferCount &&
			ShaderResourceCount == RHS.ShaderResourceCount &&
			UnorderedAccessCount == RHS.UnorderedAccessCount;
	}
	
	friend uint32 GetTypeHash(const FShaderRegisterCounts& Counts)
	{
		uint32 Hash = GetTypeHash(Counts.SamplerCount);
		Hash = HashCombineFast(Hash, GetTypeHash(Counts.ConstantBufferCount));
		Hash = HashCombineFast(Hash, GetTypeHash(Counts.ShaderResourceCount));
		Hash = HashCombineFast(Hash, GetTypeHash(Counts.UnorderedAccessCount));
		return Hash;
	}
};

struct FD3D12QuantizedBoundShaderState
{
	FRHIShaderBindingLayout ShaderBindingLayout;
	TStaticArray<FShaderRegisterCounts, SV_ShaderVisibilityCount> RegisterCounts;
	ERootSignatureType RootSignatureType = RS_Raster;
	union
	{
		struct
		{
			uint8 bAllowIAInputLayout : 1;
			uint8 bNeedsAgsIntrinsicsSpace : 1;
			uint8 bNeedsNvIntrinsicsSpace : 1;
			uint8 bUseDiagnosticBuffer : 1;
			uint8 bUseDirectlyIndexedResourceHeap : 1;
			uint8 bUseDirectlyIndexedSamplerHeap : 1;
			uint8 bUseRootConstants : 1;
			uint8 Padding : 1;
		};
		uint8 Flags;
	};

	inline bool operator==(const FD3D12QuantizedBoundShaderState& RHS) const
	{
		return ShaderBindingLayout == RHS.ShaderBindingLayout &&
			RegisterCounts == RHS.RegisterCounts &&
			RootSignatureType == RHS.RootSignatureType &&
			Flags == RHS.Flags;
	}
	
	friend uint32 GetTypeHash(const FD3D12QuantizedBoundShaderState& State)
	{
		uint32 Hash = GetTypeHash(State.ShaderBindingLayout);
		for (int32 Index = 0; Index < SV_ShaderVisibilityCount; ++Index)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(State.RegisterCounts[Index]));
		}
		Hash = HashCombineFast(Hash, GetTypeHash(State.RootSignatureType));
		Hash = HashCombineFast(Hash, GetTypeHash(State.Flags));
		return Hash;
	}

	static void InitShaderRegisterCounts(D3D12_RESOURCE_BINDING_TIER ResourceBindingTier, const FShaderCodePackedResourceCounts& Counts, FShaderRegisterCounts& Shader, bool bAllowUAVs = false);
};

/**
* Convert from ECubeFace to D3DCUBEMAP_FACES type
* @param Face - ECubeFace type to convert
* @return D3D cube face enum value
*/
FORCEINLINE uint32 GetD3D12CubeFace(ECubeFace Face)
{
	switch (Face)
	{
	case CubeFace_PosX:
	default:
		return 0;//D3DCUBEMAP_FACE_POSITIVE_X;
	case CubeFace_NegX:
		return 1;//D3DCUBEMAP_FACE_NEGATIVE_X;
	case CubeFace_PosY:
		return 2;//D3DCUBEMAP_FACE_POSITIVE_Y;
	case CubeFace_NegY:
		return 3;//D3DCUBEMAP_FACE_NEGATIVE_Y;
	case CubeFace_PosZ:
		return 4;//D3DCUBEMAP_FACE_POSITIVE_Z;
	case CubeFace_NegZ:
		return 5;//D3DCUBEMAP_FACE_NEGATIVE_Z;
	};
}

/**
* Calculate a subresource index for a texture
*/
FORCEINLINE uint32 CalcSubresource(uint32 MipSlice, uint32 ArraySlice, uint32 MipLevels)
{
	return MipSlice + ArraySlice * MipLevels;
}

/**
 * Keeps track of Locks for D3D12 objects
 */
class FD3D12LockedKey
{
public:
	void* SourceObject;
	uint32 Subresource;

public:
	FD3D12LockedKey() : SourceObject(NULL)
		, Subresource(0)
	{}
	FD3D12LockedKey(FD3D12Resource* source, uint32 subres = 0) : SourceObject((void*)source)
		, Subresource(subres)
	{}
	FD3D12LockedKey(class FD3D12ResourceLocation* source, uint32 subres = 0) : SourceObject((void*)source)
		, Subresource(subres)
	{}

	template<class ClassType>
	FD3D12LockedKey(ClassType* source, uint32 subres = 0) : SourceObject((void*)source)
		, Subresource(subres)
	{}
	bool operator==(const FD3D12LockedKey& Other) const
	{
		return SourceObject == Other.SourceObject && Subresource == Other.Subresource;
	}
	bool operator!=(const FD3D12LockedKey& Other) const
	{
		return SourceObject != Other.SourceObject || Subresource != Other.Subresource;
	}
	uint32 GetHash() const
	{
		return PointerHash(SourceObject);
	}

	/** Hashing function. */
	friend uint32 GetTypeHash(const FD3D12LockedKey& K)
	{
		return K.GetHash();
	}
};

class FD3D12RenderTargetView;
class FD3D12DepthStencilView;

/**
 * Class for retrieving render targets currently bound to the device context.
 */
class FD3D12BoundRenderTargets
{
public:
	/** Initialization constructor: requires the state cache. */
	explicit FD3D12BoundRenderTargets(FD3D12RenderTargetView** RTArray, uint32 NumActiveRTs, FD3D12DepthStencilView* DSView);

	/** Destructor. */
	~FD3D12BoundRenderTargets();

	/** Accessors. */
	FORCEINLINE int32 GetNumActiveTargets() const { return NumActiveTargets; }
	FORCEINLINE FD3D12RenderTargetView* GetRenderTargetView(int32 TargetIndex) { return RenderTargetViews[TargetIndex]; }
	FORCEINLINE FD3D12DepthStencilView* GetDepthStencilView() { return DepthStencilView; }

private:
	/** Active render target views. */
	FD3D12RenderTargetView* RenderTargetViews[MaxSimultaneousRenderTargets];

	/** Active depth stencil view. */
	FD3D12DepthStencilView* DepthStencilView;

	/** The number of active render targets. */
	int32 NumActiveTargets;
};

void LogExecuteCommandLists(uint32 NumCommandLists, ID3D12CommandList *const *ppCommandLists);

/**
 * The base class of threadsafe reference counted objects.
 */
template <class Type>
struct FThreadsafeQueue
{
private:
	mutable FCriticalSection	SynchronizationObject; // made this mutable so this class can have const functions and still be thread safe
	TQueue<Type>				Items;
	uint32						Size = 0;
public:

	inline const uint32 GetSize() const { return Size; }

	void Enqueue(const Type& Item)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Enqueue(Item);
		Size++;
	}

	bool Dequeue(Type& Result)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Size--;
		return Items.Dequeue(Result);
	}

	template <typename CompareFunc>
	bool Dequeue(Type& Result, const CompareFunc& Func)
	{
		FScopeLock ScopeLock(&SynchronizationObject);

		if (Items.Peek(Result))
		{
			if (Func(Result))
			{
				Items.Dequeue(Result);
				Size--;

				return true;
			}
		}

		return false;
	}

	template <typename ResultType, typename CompareFunc>
	bool BatchDequeue(TArray<ResultType>& Result, const CompareFunc& Func, uint32 MaxItems)
	{
		FScopeLock ScopeLock(&SynchronizationObject);

		uint32 i = 0;
		Type Item;
		while (Items.Peek(Item) && i <= MaxItems)
		{
			if (Func(Item))
			{
				Items.Dequeue(Item);
				Size--;
				Result.Emplace(Item);

				i++;
			}
			else
			{
				break;
			}
		}

		return i > 0;
	}

	bool Peek(Type& Result)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		return Items.Peek(Result);
	}

	bool IsEmpty()
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		return Items.IsEmpty();
	}

	void Empty()
	{
		FScopeLock ScopeLock(&SynchronizationObject);

		Type Result;
		while (Items.Dequeue(Result)) {}
	}
};

inline bool IsCPUWritable(D3D12_HEAP_TYPE HeapType, const D3D12_HEAP_PROPERTIES *pCustomHeapProperties = nullptr)
{
	check(HeapType == D3D12_HEAP_TYPE_CUSTOM ? pCustomHeapProperties != nullptr : true);
	return HeapType == D3D12_HEAP_TYPE_UPLOAD ||
		(HeapType == D3D12_HEAP_TYPE_CUSTOM &&
			(pCustomHeapProperties->CPUPageProperty == D3D12_CPU_PAGE_PROPERTY_WRITE_COMBINE || pCustomHeapProperties->CPUPageProperty == D3D12_CPU_PAGE_PROPERTY_WRITE_BACK));
}

inline bool IsGPUOnly(D3D12_HEAP_TYPE HeapType, const D3D12_HEAP_PROPERTIES *pCustomHeapProperties = nullptr)
{
	check(HeapType == D3D12_HEAP_TYPE_CUSTOM ? pCustomHeapProperties != nullptr : true);
	return HeapType == D3D12_HEAP_TYPE_DEFAULT ||
		(HeapType == D3D12_HEAP_TYPE_CUSTOM &&
		(pCustomHeapProperties->CPUPageProperty == D3D12_CPU_PAGE_PROPERTY_NOT_AVAILABLE));
}

inline bool IsCPUAccessible(D3D12_HEAP_TYPE HeapType, const D3D12_HEAP_PROPERTIES* pCustomHeapProperties = nullptr)
{
	return !IsGPUOnly(HeapType, pCustomHeapProperties);
}

inline ED3D12Access DetermineInitialBufferD3D12Access(D3D12_HEAP_TYPE HeapType, const D3D12_HEAP_PROPERTIES* pCustomHeapProperties = nullptr)
{
	if (HeapType == D3D12_HEAP_TYPE_DEFAULT || IsCPUWritable(HeapType, pCustomHeapProperties))
	{
		return ED3D12Access::GenericRead;
	}
	else
	{
		check(HeapType == D3D12_HEAP_TYPE_READBACK);
		return ED3D12Access::CopyDest;
	}
}

static inline uint64 GetTilesNeeded(uint32 Width, uint32 Height, uint32 Depth, const D3D12_TILE_SHAPE& Shape)
{
	return uint64((Width + Shape.WidthInTexels - 1) / Shape.WidthInTexels) *
		((Height + Shape.HeightInTexels - 1) / Shape.HeightInTexels) *
		((Depth + Shape.DepthInTexels - 1) / Shape.DepthInTexels);
}

static void Get4KTileShape(D3D12_TILE_SHAPE* pTileShape, DXGI_FORMAT DXGIFormat, EPixelFormat UEFormat, D3D12_RESOURCE_DIMENSION Dimension, uint32 SampleCount)
{
	//Bits per unit
	uint32 BPU = GPixelFormats[UEFormat].BlockBytes * 8;

	switch (Dimension)
	{
	case D3D12_RESOURCE_DIMENSION_BUFFER:
	case D3D12_RESOURCE_DIMENSION_TEXTURE1D:
	{
		check(!UE::DXGIUtilities::IsBlockCompressedFormat(DXGIFormat));
		pTileShape->WidthInTexels = (BPU == 0) ? 4096 : 4096 * 8 / BPU;
		pTileShape->HeightInTexels = 1;
		pTileShape->DepthInTexels = 1;
	}
	break;
	case D3D12_RESOURCE_DIMENSION_TEXTURE2D:
	{
		pTileShape->DepthInTexels = 1;
		if (UE::DXGIUtilities::IsBlockCompressedFormat(DXGIFormat))
		{
			// Currently only supported block sizes are 64 and 128.
			// These equations calculate the size in texels for a tile. It relies on the fact that 16*16*16 blocks fit in a tile if the block size is 128 bits.
			check(BPU == 64 || BPU == 128);
			pTileShape->WidthInTexels = 16 * UE::DXGIUtilities::GetWidthAlignment(DXGIFormat);
			pTileShape->HeightInTexels = 16 * UE::DXGIUtilities::GetHeightAlignment(DXGIFormat);
			if (BPU == 64)
			{
				// If bits per block are 64 we double width so it takes up the full tile size.
				// This is only true for BC1 and BC4
				check((DXGIFormat >= DXGI_FORMAT_BC1_TYPELESS && DXGIFormat <= DXGI_FORMAT_BC1_UNORM_SRGB) ||
					(DXGIFormat >= DXGI_FORMAT_BC4_TYPELESS && DXGIFormat <= DXGI_FORMAT_BC4_SNORM));
				pTileShape->WidthInTexels *= 2;
			}
		}
		else
		{
			if (BPU <= 8)
			{
				pTileShape->WidthInTexels = 64;
				pTileShape->HeightInTexels = 64;
			}
			else if (BPU <= 16)
			{
				pTileShape->WidthInTexels = 64;
				pTileShape->HeightInTexels = 32;
			}
			else if (BPU <= 32)
			{
				pTileShape->WidthInTexels = 32;
				pTileShape->HeightInTexels = 32;
			}
			else if (BPU <= 64)
			{
				pTileShape->WidthInTexels = 32;
				pTileShape->HeightInTexels = 16;
			}
			else if (BPU <= 128)
			{
				pTileShape->WidthInTexels = 16;
				pTileShape->HeightInTexels = 16;
			}
			else
			{
				check(false);
			}

			if (SampleCount <= 1)
			{ /* Do nothing */
			}
			else if (SampleCount <= 2)
			{
				pTileShape->WidthInTexels /= 2;
				pTileShape->HeightInTexels /= 1;
			}
			else if (SampleCount <= 4)
			{
				pTileShape->WidthInTexels /= 2;
				pTileShape->HeightInTexels /= 2;
			}
			else if (SampleCount <= 8)
			{
				pTileShape->WidthInTexels /= 4;
				pTileShape->HeightInTexels /= 2;
			}
			else if (SampleCount <= 16)
			{
				pTileShape->WidthInTexels /= 4;
				pTileShape->HeightInTexels /= 4;
			}
			else
			{
				check(false);
			}

			check(UE::DXGIUtilities::GetWidthAlignment(DXGIFormat) == 1);
			check(UE::DXGIUtilities::GetHeightAlignment(DXGIFormat) == 1);
		}

		break;
	}
	case D3D12_RESOURCE_DIMENSION_TEXTURE3D:
	{
		if (UE::DXGIUtilities::IsBlockCompressedFormat(DXGIFormat))
		{
			// Currently only supported block sizes are 64 and 128.
			// These equations calculate the size in texels for a tile. It relies on the fact that 16*16*16 blocks fit in a tile if the block size is 128 bits.
			check(BPU == 64 || BPU == 128);
			pTileShape->WidthInTexels = 8 * UE::DXGIUtilities::GetWidthAlignment(DXGIFormat);
			pTileShape->HeightInTexels = 8 * UE::DXGIUtilities::GetHeightAlignment(DXGIFormat);
			pTileShape->DepthInTexels = 4;
			if (BPU == 64)
			{
				// If bits per block are 64 we double width so it takes up the full tile size.
				// This is only true for BC1 and BC4
				check((DXGIFormat >= DXGI_FORMAT_BC1_TYPELESS && DXGIFormat <= DXGI_FORMAT_BC1_UNORM_SRGB) ||
					(DXGIFormat >= DXGI_FORMAT_BC4_TYPELESS && DXGIFormat <= DXGI_FORMAT_BC4_SNORM));
				pTileShape->DepthInTexels *= 2;
			}
		}
		else
		{
			if (BPU <= 8)
			{
				pTileShape->WidthInTexels = 16;
				pTileShape->HeightInTexels = 16;
				pTileShape->DepthInTexels = 16;
			}
			else if (BPU <= 16)
			{
				pTileShape->WidthInTexels = 16;
				pTileShape->HeightInTexels = 16;
				pTileShape->DepthInTexels = 8;
			}
			else if (BPU <= 32)
			{
				pTileShape->WidthInTexels = 16;
				pTileShape->HeightInTexels = 8;
				pTileShape->DepthInTexels = 8;
			}
			else if (BPU <= 64)
			{
				pTileShape->WidthInTexels = 8;
				pTileShape->HeightInTexels = 8;
				pTileShape->DepthInTexels = 8;
			}
			else if (BPU <= 128)
			{
				pTileShape->WidthInTexels = 8;
				pTileShape->HeightInTexels = 8;
				pTileShape->DepthInTexels = 4;
			}
			else
			{
				check(false);
			}

			check(UE::DXGIUtilities::GetWidthAlignment(DXGIFormat) == 1);
			check(UE::DXGIUtilities::GetHeightAlignment(DXGIFormat) == 1);
		}
	}
	break;
	}
}

#define ASSERT_RESOURCE_STATES 0	// Disabled for now.

#if ASSERT_RESOURCE_STATES
class FD3D12View;
struct FD3D12ViewSubset;

// template <class TView>
// bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12View* pView, const D3D12_RESOURCE_STATES& State);

bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12Resource* pResource, const D3D12_RESOURCE_STATES& State, uint32 Subresource);
bool AssertResourceState(ID3D12CommandList* pCommandList, FD3D12Resource* pResource, const D3D12_RESOURCE_STATES& State, const FD3D12ViewSubset& ViewSubset);
#endif

FORCEINLINE_DEBUGGABLE D3D12_PRIMITIVE_TOPOLOGY_TYPE TranslatePrimitiveTopologyType(EPrimitiveTopologyType TopologyType)
{
	switch (TopologyType)
	{
	case EPrimitiveTopologyType::Triangle:	return D3D12_PRIMITIVE_TOPOLOGY_TYPE_TRIANGLE;
	case EPrimitiveTopologyType::Patch:		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_PATCH;
	case EPrimitiveTopologyType::Line:		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_LINE;
	case EPrimitiveTopologyType::Point:		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_POINT;
	default:
		ensure(0);
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_UNDEFINED;
	}
}

FORCEINLINE_DEBUGGABLE D3D_PRIMITIVE_TOPOLOGY TranslatePrimitiveType(EPrimitiveType PrimitiveType)
{
	switch (PrimitiveType)
	{
	case PT_TriangleList:				return D3D_PRIMITIVE_TOPOLOGY_TRIANGLELIST;
	case PT_TriangleStrip:				return D3D11_PRIMITIVE_TOPOLOGY_TRIANGLESTRIP;
	case PT_LineList:					return D3D_PRIMITIVE_TOPOLOGY_LINELIST;
	case PT_PointList:					return D3D_PRIMITIVE_TOPOLOGY_POINTLIST;
	#if defined(D3D12RHI_PRIMITIVE_TOPOLOGY_RECTLIST)
		case PT_RectList:				return D3D12RHI_PRIMITIVE_TOPOLOGY_RECTLIST;
	#endif

	default:
		ensure(0);
		return D3D_PRIMITIVE_TOPOLOGY_UNDEFINED;
	}
}

#pragma warning(push)
#pragma warning(disable: 4063)
FORCEINLINE_DEBUGGABLE D3D12_PRIMITIVE_TOPOLOGY_TYPE D3D12PrimitiveTypeToTopologyType(D3D_PRIMITIVE_TOPOLOGY PrimitiveType)
{
	switch (PrimitiveType)
	{
	case D3D_PRIMITIVE_TOPOLOGY_POINTLIST:
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_POINT;

	case D3D_PRIMITIVE_TOPOLOGY_LINELIST:
	case D3D_PRIMITIVE_TOPOLOGY_LINESTRIP:
	case D3D_PRIMITIVE_TOPOLOGY_LINELIST_ADJ:
	case D3D_PRIMITIVE_TOPOLOGY_LINESTRIP_ADJ:
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_LINE;

	case D3D_PRIMITIVE_TOPOLOGY_TRIANGLELIST:
	case D3D_PRIMITIVE_TOPOLOGY_TRIANGLESTRIP:
	case D3D_PRIMITIVE_TOPOLOGY_TRIANGLELIST_ADJ:
	case D3D_PRIMITIVE_TOPOLOGY_TRIANGLESTRIP_ADJ:
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_TRIANGLE;

	#if defined(D3D12RHI_PRIMITIVE_TOPOLOGY_RECTLIST)
		case D3D12RHI_PRIMITIVE_TOPOLOGY_RECTLIST:
			return D3D12RHI_PRIMITIVE_TOPOLOGY_TYPE_RECT;
	#endif

	case D3D_PRIMITIVE_TOPOLOGY_UNDEFINED:
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_UNDEFINED;

	default:
		return D3D12_PRIMITIVE_TOPOLOGY_TYPE_PATCH;
	}
}
#pragma warning(pop)

// @return 0xffffffff if not not supported
FORCEINLINE_DEBUGGABLE uint32 GetMaxMSAAQuality(uint32 SampleCount)
{
	if (SampleCount <= DX_MAX_MSAA_COUNT)
	{
		// 0 has better quality (a more even distribution)
		// higher quality levels might be useful for non box filtered AA or when using weighted samples 
		return 0;
	}

	// not supported
	return 0xffffffff;
}

struct FD3D12ScopeLock
{
public:
	FD3D12ScopeLock(FCriticalSection* CritSec) : CS(CritSec) { CS->Lock(); }
	~FD3D12ScopeLock() { CS->Unlock(); }
private:
	FCriticalSection* CS;
};

struct FD3D12ScopeNoLock
{
public:
	FD3D12ScopeNoLock(FCriticalSection* CritSec) { /* Do Nothing! */ }
	~FD3D12ScopeNoLock() { /* Do Nothing! */ }
};


FORCEINLINE_DEBUGGABLE bool CheckResourceStateCompatibility(D3D12_RESOURCE_STATES State, D3D12_RESOURCE_FLAGS Flags, FString& IncompatibilityReason)
{
	// Check for ALLOW_RENDER_TARGET compatibility
	if (State & D3D12_RESOURCE_STATE_RENDER_TARGET)
	{
		if (!(Flags & D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET))
		{
			IncompatibilityReason.Append(TEXT("D3D12_RESOURCE_STATE_RENDER_TARGET requires D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET.\n"));
			return false;
		}
	}

	// Check for ALLOW_DEPTH_STENCIL compatibility
	if (State & D3D12_RESOURCE_STATE_DEPTH_WRITE || State & D3D12_RESOURCE_STATE_DEPTH_READ)
	{
		if (!(Flags & D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL))
		{
			IncompatibilityReason.Append(TEXT("D3D12_RESOURCE_STATE_DEPTH_WRITE or D3D12_RESOURCE_STATE_DEPTH_READ requires D3D12_RESOURCE_FLAG_ALLOW_DEPTH_STENCIL.\n"));
			return false;
		}
	}

	// Check for ALLOW_UNORDERED_ACCESS compatibility
	if (State & D3D12_RESOURCE_STATE_UNORDERED_ACCESS)
	{
		if (!(Flags & D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS))
		{
			IncompatibilityReason.Append(TEXT("D3D12_RESOURCE_STATE_UNORDERED_ACCESS requires D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS.\n"));
			return false;
		}
	}

	// Check for DENY_SHADER_RESOURCE incompatibility
	if (State & (D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE | D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE))
	{
		if (Flags & D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE)
		{
			IncompatibilityReason.Append(TEXT("D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE or D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE cannot be used with D3D12_RESOURCE_FLAG_DENY_SHADER_RESOURCE.\n"));
			return false;
		}
	}

	return true;
}

inline bool D3D12DoesShaderFrequencySupportUAV(EShaderFrequency ShaderFrequency)
{
	return ShaderFrequency == SF_Pixel
		|| ShaderFrequency == SF_Vertex
		|| ShaderFrequency == SF_Compute
#if PLATFORM_SUPPORTS_MESH_SHADERS
		|| ShaderFrequency == SF_Mesh
		|| ShaderFrequency == SF_Amplification
#endif
		;
}
