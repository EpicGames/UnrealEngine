// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12VertexDeclaration.cpp: D3D vertex declaration RHI implementation.
	=============================================================================*/

#include "D3D12RHIPrivate.h"

/**
 * Key used to look up vertex declarations in the cache.
 */
struct FD3D12VertexDeclarationKey
{
	/** Vertex elements in the declaration. */
	FD3D12VertexElements VertexElements;
	/** Hash of the vertex elements. */
	uint32 Hash;

	/** Hash of the vertex elements, without strides. */
	uint32 HashNoStrides;

	uint16 StreamStrides[MaxVertexElementCount];

	/** Initialization constructor. */
	explicit FD3D12VertexDeclarationKey(const FVertexDeclarationElementList& InElements)
	{
		uint16 UsedStreamsMask = 0;
		FMemory::Memzero(StreamStrides);

		for (int32 ElementIndex = 0; ElementIndex < InElements.Num(); ElementIndex++)
		{
			const FVertexElement& Element = InElements[ElementIndex];
			D3D12_INPUT_ELEMENT_DESC D3DElement = { 0 };
			D3DElement.InputSlot = Element.StreamIndex;
			D3DElement.AlignedByteOffset = Element.Offset;
			switch (Element.Type)
			{
			case VET_Float1:		D3DElement.Format = DXGI_FORMAT_R32_FLOAT; break;
			case VET_Float2:		D3DElement.Format = DXGI_FORMAT_R32G32_FLOAT; break;
			case VET_Float3:		D3DElement.Format = DXGI_FORMAT_R32G32B32_FLOAT; break;
			case VET_Float4:		D3DElement.Format = DXGI_FORMAT_R32G32B32A32_FLOAT; break;
			case VET_PackedNormal:	D3DElement.Format = DXGI_FORMAT_R8G8B8A8_SNORM; break; //TODO: uint32 doesn't work because D3D12  squishes it to 0 in the IA-VS conversion
			case VET_UByte4:		D3DElement.Format = DXGI_FORMAT_R8G8B8A8_UINT; break; //TODO: SINT, blendindices
			case VET_UByte4N:		D3DElement.Format = DXGI_FORMAT_R8G8B8A8_UNORM; break;
			case VET_Color:			D3DElement.Format = DXGI_FORMAT_B8G8R8A8_UNORM; break;
			case VET_Short2:		D3DElement.Format = DXGI_FORMAT_R16G16_SINT; break;
			case VET_Short4:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_SINT; break;
			case VET_Short2N:		D3DElement.Format = DXGI_FORMAT_R16G16_SNORM; break;
			case VET_Half2:			D3DElement.Format = DXGI_FORMAT_R16G16_FLOAT; break;
			case VET_Half4:			D3DElement.Format = DXGI_FORMAT_R16G16B16A16_FLOAT; break;
			case VET_Short4N:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_SNORM; break;
			case VET_UShort2:		D3DElement.Format = DXGI_FORMAT_R16G16_UINT; break;
			case VET_UShort4:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_UINT; break;
			case VET_UShort2N:		D3DElement.Format = DXGI_FORMAT_R16G16_UNORM; break;
			case VET_UShort4N:		D3DElement.Format = DXGI_FORMAT_R16G16B16A16_UNORM; break;
			case VET_URGB10A2N:		D3DElement.Format = DXGI_FORMAT_R10G10B10A2_UNORM; break;
			case VET_UInt:			D3DElement.Format = DXGI_FORMAT_R32_UINT; break;
			default: UE_LOGF(LogD3D12RHI, Fatal, "Unknown RHI vertex element type %u", (uint8)InElements[ElementIndex].Type);
			};

			// We don't assign D3DElement.SemanticName here, because it's a constant string and we don't want to hash pointers. For best debugging experience,
			// we want to get a consistent hash value across sessions. Therefore it's assigned below, after hashing.
			D3DElement.SemanticIndex = Element.AttributeIndex;
			D3DElement.InputSlotClass = Element.bUseInstanceIndex ? D3D12_INPUT_CLASSIFICATION_PER_INSTANCE_DATA : D3D12_INPUT_CLASSIFICATION_PER_VERTEX_DATA;

			// This is a divisor to apply to the instance index used to read from this stream.
			D3DElement.InstanceDataStepRate = Element.bUseInstanceIndex ? 1 : 0;

			if ((UsedStreamsMask & 1 << Element.StreamIndex) != 0)
			{
				ensure(StreamStrides[Element.StreamIndex] == Element.Stride);
			}
			else
			{
				UsedStreamsMask = UsedStreamsMask | (1 << Element.StreamIndex);
				StreamStrides[Element.StreamIndex] = Element.Stride;
			}

			VertexElements.Add(D3DElement);
		}

		// Sort by stream then offset.
		struct FCompareDesc
		{
			FORCEINLINE bool operator()(const D3D12_INPUT_ELEMENT_DESC& A, const D3D12_INPUT_ELEMENT_DESC &B) const
			{
				if (A.InputSlot != B.InputSlot)
				{
					return A.InputSlot < B.InputSlot;
				}
				if (A.AlignedByteOffset != B.AlignedByteOffset)
				{
					return A.AlignedByteOffset < B.AlignedByteOffset;
				}
				return A.SemanticIndex < B.SemanticIndex;
			}
		};
		Algo::Sort(VertexElements, FCompareDesc());

		// Hash once.
		Hash = FCrc::MemCrc_DEPRECATED(VertexElements.GetData(), VertexElements.Num()*sizeof(D3D12_INPUT_ELEMENT_DESC));
		HashNoStrides = Hash;

		Hash = FCrc::MemCrc_DEPRECATED(StreamStrides, sizeof(StreamStrides), Hash);

		// Assign all the SemanticName after hashing. It's a constant string, always the same, so no need to hash the data.
		for (int32 ElementIndex = 0; ElementIndex < VertexElements.Num(); ElementIndex++)
		{
			VertexElements[ElementIndex].SemanticName = "ATTRIBUTE";
		}
	}
};

/** Hashes the array of D3D12 vertex element descriptions. */
uint32 GetTypeHash(const FD3D12VertexDeclarationKey& Key)
{
	return Key.Hash;
}

/** Compare two vertex declaration keys. */
bool operator==(const FD3D12VertexDeclarationKey& A, const FD3D12VertexDeclarationKey& B)
{
	return A.VertexElements == B.VertexElements
		&& !FMemory::Memcmp(A.StreamStrides, B.StreamStrides, sizeof(A.StreamStrides));
}

/** Global cache of vertex declarations. */
struct FVertexDeclarationCache
{
	FORCEINLINE FVertexDeclarationRHIRef* Find(FD3D12VertexDeclarationKey Key)
	{
		FScopeLock RWGuard(&LockGuard);
		return Cache.Find(Key);
	}

	FORCEINLINE FVertexDeclarationRHIRef& Add(const FD3D12VertexDeclarationKey& InKey, const FVertexDeclarationRHIRef& InValue)
	{
		FScopeLock RWGuard(&LockGuard);
		return Cache.Add(InKey, InValue);
	}

	FORCEINLINE FVertexDeclarationRHIRef* FindOrAdd(const FD3D12VertexDeclarationKey& InKey)
	{
		FScopeLock RWGuard(&LockGuard);

		FVertexDeclarationRHIRef* VertexDeclarationRefPtr = Cache.Find(InKey);
		if (VertexDeclarationRefPtr == nullptr)
		{
			VertexDeclarationRefPtr = &Cache.Add(InKey, new FD3D12VertexDeclaration(InKey.VertexElements, InKey.StreamStrides, InKey.Hash, InKey.HashNoStrides));
		}

		return VertexDeclarationRefPtr;
	}

	FCriticalSection LockGuard;
	TMap<FD3D12VertexDeclarationKey, FVertexDeclarationRHIRef> Cache;
};

FVertexDeclarationCache GVertexDeclarationCache;

FVertexDeclarationRHIRef FD3D12DynamicRHI::RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	// Construct a key from the elements.
	FD3D12VertexDeclarationKey Key(Elements);

	// Check for a cached vertex declaration. Add to the cache if it doesn't exist.
	FVertexDeclarationRHIRef* VertexDeclarationRefPtr = GVertexDeclarationCache.FindOrAdd(Key);

	// The cached declaration must match the input declaration!
	check(VertexDeclarationRefPtr);
	check(IsValidRef(*VertexDeclarationRefPtr));
	FD3D12VertexDeclaration* D3D12VertexDeclaration = (FD3D12VertexDeclaration*)VertexDeclarationRefPtr->GetReference();
	checkSlow(D3D12VertexDeclaration->VertexElements == Key.VertexElements);

	return *VertexDeclarationRefPtr;
}

bool FD3D12VertexDeclaration::GetInitializer(FVertexDeclarationElementList& Init)
{
	const int32 NumVertElems = VertexElements.Num();
	Init.Empty(NumVertElems);
	Init.AddUninitialized(NumVertElems);

	for (int32 Idx = 0; Idx < NumVertElems; ++Idx)
	{
		const D3D12_INPUT_ELEMENT_DESC& Elem = VertexElements[Idx];
		FVertexElement& Out = Init[Idx];

		Out.StreamIndex = Elem.InputSlot;
		Out.Offset = Elem.AlignedByteOffset;
		switch (Elem.Format)
		{
		case DXGI_FORMAT_R32_FLOAT:          Out.Type = VET_Float1;       break;
		case DXGI_FORMAT_R32G32_FLOAT:       Out.Type = VET_Float2;       break;
		case DXGI_FORMAT_R32G32B32_FLOAT:    Out.Type = VET_Float3;       break;
		case DXGI_FORMAT_R32G32B32A32_FLOAT: Out.Type = VET_Float4;       break;
		case DXGI_FORMAT_R8G8B8A8_SNORM:     Out.Type = VET_PackedNormal; break;
		case DXGI_FORMAT_R8G8B8A8_UINT:      Out.Type = VET_UByte4;       break;
		case DXGI_FORMAT_R8G8B8A8_UNORM:     Out.Type = VET_UByte4N;      break;
		case DXGI_FORMAT_B8G8R8A8_UNORM:     Out.Type = VET_Color;        break;
		case DXGI_FORMAT_R16G16_SINT:        Out.Type = VET_Short2;       break;
		case DXGI_FORMAT_R16G16B16A16_SINT:  Out.Type = VET_Short4;       break;
		case DXGI_FORMAT_R16G16_SNORM:       Out.Type = VET_Short2N;      break;
		case DXGI_FORMAT_R16G16_FLOAT:       Out.Type = VET_Half2;        break;
		case DXGI_FORMAT_R16G16B16A16_FLOAT: Out.Type = VET_Half4;        break;
		case DXGI_FORMAT_R16G16B16A16_SNORM: Out.Type = VET_Short4N;      break;
		case DXGI_FORMAT_R16G16_UINT:        Out.Type = VET_UShort2;      break;
		case DXGI_FORMAT_R16G16B16A16_UINT:  Out.Type = VET_UShort4;      break;
		case DXGI_FORMAT_R16G16_UNORM:       Out.Type = VET_UShort2N;     break;
		case DXGI_FORMAT_R16G16B16A16_UNORM: Out.Type = VET_UShort4N;     break;
		case DXGI_FORMAT_R10G10B10A2_UNORM:  Out.Type = VET_URGB10A2N;    break;
		case DXGI_FORMAT_R32_UINT:           Out.Type = VET_UInt;         break;
		default:
			UE_LOGF(LogD3D12RHI, Fatal, "Unknown D3D vertex element type %u", Elem.Format);
		}
		Out.AttributeIndex = Elem.SemanticIndex;
		Out.Stride = StreamStrides[Elem.InputSlot];
		Out.bUseInstanceIndex = Elem.InputSlotClass == D3D12_INPUT_CLASSIFICATION_PER_INSTANCE_DATA;
	}
	return true;
}
