// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12View.h"
#include "D3D12RHIPrivate.h"

// -----------------------------------------------------------------------------------------------------
//
//                                           FD3D12ViewRange                                           
//
// -----------------------------------------------------------------------------------------------------

FD3D12ViewRange::FD3D12ViewRange(D3D12_CONSTANT_BUFFER_VIEW_DESC const& ViewDesc)
	: Array(0, 1)
	, Plane(0, 1)
	, Mip  (0, 1)
{}

FD3D12ViewRange::FD3D12ViewRange(D3D12_RENDER_TARGET_VIEW_DESC const& ViewDesc)
{
	uint32 FirstPlane = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(ViewDesc.Format, ViewDesc.Format);
	uint32 NumPlanes  = UE::DXGIUtilities::GetPlaneCount(ViewDesc.Format);
	Plane = { FirstPlane, NumPlanes - FirstPlane };

	switch (ViewDesc.ViewDimension)
	{
	default: checkNoEntry(); return; // not implemented

	case D3D12_RTV_DIMENSION_UNKNOWN:
		Mip   = { 0, 0 };
		Plane = { 0, 0 };
		Array = { 0, 0 };
		break;

	case D3D12_RTV_DIMENSION_TEXTURE2D:
		Mip   = { ViewDesc.Texture2D.MipSlice  , 1 };
		Plane = { ViewDesc.Texture2D.PlaneSlice, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_RTV_DIMENSION_TEXTURE2DARRAY:
		Mip   = { ViewDesc.Texture2DArray.MipSlice  , 1 };
		Plane = { ViewDesc.Texture2DArray.PlaneSlice, 1 };
		Array = { ViewDesc.Texture2DArray.FirstArraySlice, ViewDesc.Texture2DArray.ArraySize };
		break;

	case D3D12_RTV_DIMENSION_TEXTURE2DMS:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_RTV_DIMENSION_TEXTURE2DMSARRAY:
		Mip   = { 0, 1 };
		Array = { ViewDesc.Texture2DMSArray.FirstArraySlice, ViewDesc.Texture2DMSArray.ArraySize };
		break;

	case D3D12_RTV_DIMENSION_TEXTURE3D:
		Mip   = { ViewDesc.Texture3D.MipSlice, 1 };
		Array = { 0, 1 };
		break;
	}
}

FD3D12ViewRange::FD3D12ViewRange(D3D12_SHADER_RESOURCE_VIEW_DESC const& ViewDesc)
{
	uint32 FirstPlane = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(ViewDesc.Format, ViewDesc.Format);
	uint32 NumPlanes  = UE::DXGIUtilities::GetPlaneCount(ViewDesc.Format);
	Plane = { FirstPlane, NumPlanes - FirstPlane };

	switch (ViewDesc.ViewDimension)
	{
	default: checkNoEntry(); return; // not implemented

	case D3D12_SRV_DIMENSION_UNKNOWN:
		Mip   = { 0, 0 };
		Plane = { 0, 0 };
		Array = { 0, 0 };
		break;		

#if D3D12_RHI_RAYTRACING
	case D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;
#endif

	case D3D12_SRV_DIMENSION_BUFFER:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_SRV_DIMENSION_TEXTURE2D:
		Mip   = { ViewDesc.Texture2D.MostDetailedMip, ViewDesc.Texture2D.MipLevels };
		Plane = { ViewDesc.Texture2D.PlaneSlice, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_SRV_DIMENSION_TEXTURE2DARRAY:
		Mip   = { ViewDesc.Texture2DArray.MostDetailedMip, ViewDesc.Texture2DArray.MipLevels };
		Plane = { ViewDesc.Texture2DArray.PlaneSlice, 1 };
		Array = { ViewDesc.Texture2DArray.FirstArraySlice, ViewDesc.Texture2DArray.ArraySize };
		break;

	case D3D12_SRV_DIMENSION_TEXTURE2DMS:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_SRV_DIMENSION_TEXTURE2DMSARRAY:
		Mip   = { 0, 1 };
		Array = { ViewDesc.Texture2DMSArray.FirstArraySlice, ViewDesc.Texture2DMSArray.ArraySize };
		break;

	case D3D12_SRV_DIMENSION_TEXTURE3D:
		Mip   = { ViewDesc.Texture3D.MostDetailedMip, ViewDesc.Texture3D.MipLevels };
		Array = { 0, 1 };
		break;

	case D3D12_SRV_DIMENSION_TEXTURECUBE:
		Mip   = { ViewDesc.TextureCube.MostDetailedMip, ViewDesc.TextureCube.MipLevels };
		Array = { 0, 6 };
		break;

	case D3D12_SRV_DIMENSION_TEXTURECUBEARRAY:
		Mip   = { ViewDesc.TextureCubeArray.MostDetailedMip, ViewDesc.TextureCubeArray.MipLevels };
		Array = { ViewDesc.TextureCubeArray.First2DArrayFace, ViewDesc.TextureCubeArray.NumCubes * 6 };
		break;
	}
}

FD3D12ViewRange::FD3D12ViewRange(D3D12_DEPTH_STENCIL_VIEW_DESC const& ViewDesc)
{
	uint32 FirstPlane = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(ViewDesc.Format, ViewDesc.Format);
	uint32 NumPlanes  = UE::DXGIUtilities::GetPlaneCount(ViewDesc.Format);
	Plane = { FirstPlane, NumPlanes - FirstPlane };

	switch (ViewDesc.ViewDimension)
	{
	default: checkNoEntry(); return; // not implemented

	case D3D12_DSV_DIMENSION_UNKNOWN:
		Mip   = { 0, 0 };
		Plane = { 0, 0 };
		Array = { 0, 0 };
		break;	

	case D3D12_DSV_DIMENSION_TEXTURE2D:
		Mip   = { ViewDesc.Texture2D.MipSlice, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_DSV_DIMENSION_TEXTURE2DARRAY:
		Mip   = { ViewDesc.Texture2DArray.MipSlice, 1 };
		Array = { ViewDesc.Texture2DArray.FirstArraySlice, ViewDesc.Texture2DArray.ArraySize };
		break;

	case D3D12_DSV_DIMENSION_TEXTURE2DMS:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_DSV_DIMENSION_TEXTURE2DMSARRAY:
		Mip   = { 0, 1 };
		Array = { ViewDesc.Texture2DMSArray.FirstArraySlice, ViewDesc.Texture2DMSArray.ArraySize };
		break;
	}
}

FD3D12ViewRange::FD3D12ViewRange(D3D12_UNORDERED_ACCESS_VIEW_DESC const& ViewDesc)
{
	uint32 FirstPlane = UE::DXGIUtilities::GetPlaneSliceFromViewFormat(ViewDesc.Format, ViewDesc.Format);
	uint32 NumPlanes  = UE::DXGIUtilities::GetPlaneCount(ViewDesc.Format);
	Plane = { FirstPlane, NumPlanes - FirstPlane };

	switch (ViewDesc.ViewDimension)
	{
	default: checkNoEntry(); return; // not implemented

	case D3D12_UAV_DIMENSION_UNKNOWN:
		Mip   = { 0, 0 };
		Plane = { 0, 0 };
		Array = { 0, 0 };
		break;	

	case D3D12_UAV_DIMENSION_BUFFER:
		Mip   = { 0, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_UAV_DIMENSION_TEXTURE2D:
		Mip   = { ViewDesc.Texture2D.MipSlice  , 1 };
		Plane = { ViewDesc.Texture2D.PlaneSlice, 1 };
		Array = { 0, 1 };
		break;

	case D3D12_UAV_DIMENSION_TEXTURE2DARRAY:
		Mip   = { ViewDesc.Texture2DArray.MipSlice  , 1 };
		Plane = { ViewDesc.Texture2DArray.PlaneSlice, 1 };
		Array = { ViewDesc.Texture2DArray.FirstArraySlice, ViewDesc.Texture2DArray.ArraySize };
		break;

	case D3D12_UAV_DIMENSION_TEXTURE3D:
		Mip   = { ViewDesc.Texture3D.MipSlice, 1 };
		Array = { 0, 1 };
		break;
	}
}



// -----------------------------------------------------------------------------------------------------
//
//                                              FD3D12View                                              
//
// -----------------------------------------------------------------------------------------------------

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
inline bool ViewTypeSupportsBindless(ED3D12ViewType InViewType)
{
	return (InViewType == ED3D12ViewType::ShaderResource || InViewType == ED3D12ViewType::UnorderedAccess);
}
#endif

FD3D12View::FD3D12View(FD3D12Device* InDevice, ERHIDescriptorType InDescriptorType, ERHIDescriptorHeapType InHeapType, FD3D12View* FirstLinkedObject)
	: FD3D12DeviceChild(InDevice)
	, OfflineCpuHandle(InDevice->GetOfflineDescriptorManager(InHeapType).AllocateHeapSlot())
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	, bFirstLinkedObject(FirstLinkedObject == nullptr)		// If FirstLinkedObject is nullptr, this newly constructed object itself must be the first
#endif
	, HeapType(InHeapType)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (InDescriptorType != ERHIDescriptorType::Invalid)
	{
		// Bindless handle is shared across linked objects, so pull the handle from the first object if available, or allocate if we are the first
		BindlessHandle = FirstLinkedObject ? FirstLinkedObject->BindlessHandle : InDevice->GetBindlessDescriptorAllocator().AllocateDescriptor(InDescriptorType);
	}
#endif
}

FD3D12View::~FD3D12View()
{
	// Unregister this view from the underlying resource
	if (ResourceInfo.BaseResource)
	{
		ResourceInfo.BaseResource->RemoveRenameListener(this);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		// Bindless handle is shared, first linked object handles freeing
		if (bFirstLinkedObject)
		{
			GetParentDevice()->GetBindlessDescriptorManager().DeferredFreeFromDestructor(BindlessHandle);
		}
		BindlessHandle = {};
	}
#endif

	// Free the descriptor heap slot and bindless handle
	GetParentDevice()->GetOfflineDescriptorManager(HeapType).FreeHeapSlot(OfflineCpuHandle);
}

void FD3D12View::InitializeBindlessSlot()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		GetParentDevice()->GetBindlessDescriptorManager().InitializeDescriptor(BindlessHandle, this);
	}
#endif
}

void FD3D12View::UpdateBindlessSlot(FD3D12ContextArray const& Contexts)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		GetParentDevice()->GetBindlessDescriptorManager().UpdateDescriptor(Contexts, BindlessHandle, this);
	}
#endif
}

void FD3D12View::UpdateResourceInfo(FResourceInfo const& InResource, FNullDescPtr NullDescriptor)
{
	if (ResourceInfo.BaseResource != InResource.BaseResource)
	{
		if (ResourceInfo.BaseResource)
			ResourceInfo.BaseResource->RemoveRenameListener(this);

		if (InResource.BaseResource)
			InResource.BaseResource->AddRenameListener(this);
	}

	ResourceInfo = InResource;

	if (ResourceInfo.Resource)
	{
		ViewSubset.Layout = ResourceInfo.Resource->GetDesc();
		UpdateDescriptor();
	}
	else
	{
		ViewSubset.Layout = {};

		// Set the offline descriptor to null by copying the view type's default null descriptor.
		FD3D12Device* Device = GetParentDevice();

		Device->GetDevice()->CopyDescriptorsSimple(
			1,
			OfflineCpuHandle,
			Device->GetDefaultViews().*NullDescriptor,
			Translate(HeapType)
		);

		OfflineCpuHandle.IncrementVersion();
	}
}

void FD3D12View::CreateView(FResourceInfo const& InResource, FNullDescPtr NullDescriptor)
{
	UpdateResourceInfo(InResource, NullDescriptor);
	InitializeBindlessSlot();
}

void FD3D12View::UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, FNullDescPtr NullDescriptor)
{
	UpdateResourceInfo(InResource, NullDescriptor);
	UpdateBindlessSlot(Contexts);
}



// -----------------------------------------------------------------------------------------------------
//
//                                       FD3D12ConstantBufferView                                       
//
// -----------------------------------------------------------------------------------------------------

FD3D12ConstantBufferView::FD3D12ConstantBufferView(FD3D12Device* InDevice, FD3D12ConstantBufferView* FirstLinkedObject, ERHIDescriptorType InDescriptorType)
	: TD3D12View(InDevice, InDescriptorType, ERHIDescriptorHeapType::Standard, FirstLinkedObject)
{
}

void FD3D12ConstantBufferView::CreateView(FResourceInfo const& InResource, uint32 InOffset, uint32 InAlignedSize)
{
	Offset = InOffset;

	D3D12_CONSTANT_BUFFER_VIEW_DESC CBVDesc = {};
	CBVDesc.BufferLocation = InResource.ResourceLocation->GetGPUVirtualAddress() + Offset;
	CBVDesc.SizeInBytes = InAlignedSize;

	TD3D12View::CreateView(InResource, CBVDesc);
}

void FD3D12ConstantBufferView::UpdateView(const FD3D12ContextArray& Contexts, const FResourceInfo& InResource, uint32 InOffset, uint32 InAlignedSize)
{
	Offset = InOffset;

	D3D12_CONSTANT_BUFFER_VIEW_DESC CBVDesc = {};
	CBVDesc.BufferLocation = InResource.ResourceLocation->GetGPUVirtualAddress() + Offset;
	CBVDesc.SizeInBytes = InAlignedSize;

	TD3D12View::UpdateView(Contexts, InResource, CBVDesc);
}

void FD3D12ConstantBufferView::ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation)
{
	D3DViewDesc.BufferLocation = InNewResourceLocation->GetGPUVirtualAddress() + Offset;
	TD3D12View::ResourceRenamed(Contexts, InRenamedResource, InNewResourceLocation);
}

void FD3D12ConstantBufferView::UpdateDescriptor()
{
	GetParentDevice()->GetDevice()->CreateConstantBufferView(&D3DViewDesc, OfflineCpuHandle);

	OfflineCpuHandle.IncrementVersion();
}



// -----------------------------------------------------------------------------------------------------
//
//                                       FD3D12RenderTargetView                                       
//
// -----------------------------------------------------------------------------------------------------

FD3D12RenderTargetView::FD3D12RenderTargetView(FD3D12Device* InDevice, FD3D12RenderTargetView* FirstLinkedObject)
	: TD3D12View(InDevice, ERHIDescriptorType::Invalid, ERHIDescriptorHeapType::RenderTarget, FirstLinkedObject)
{
}

void FD3D12RenderTargetView::UpdateDescriptor()
{
	GetParentDevice()->GetDevice()->CreateRenderTargetView(
		ResourceInfo.Resource->GetResource(),
		&D3DViewDesc,
		OfflineCpuHandle
	);

	OfflineCpuHandle.IncrementVersion();
}



// -----------------------------------------------------------------------------------------------------
//
//                                        FD3D12DepthStencilView                                        
//
// -----------------------------------------------------------------------------------------------------

FD3D12DepthStencilView::FD3D12DepthStencilView(FD3D12Device* InDevice, FD3D12DepthStencilView* FirstLinkedObject)
	: TD3D12View(InDevice, ERHIDescriptorType::Invalid, ERHIDescriptorHeapType::DepthStencil, FirstLinkedObject)
{
}

void FD3D12DepthStencilView::UpdateDescriptor()
{
	GetParentDevice()->GetDevice()->CreateDepthStencilView(
		ResourceInfo.Resource->GetResource(),
		&D3DViewDesc,
		OfflineCpuHandle
	);

	OfflineCpuHandle.IncrementVersion();
}
