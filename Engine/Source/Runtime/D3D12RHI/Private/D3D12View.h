// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "D3D12RHICommon.h"
#include "D3D12Descriptors.h"
#include "D3D12Resources.h"
#include "RHICommandList.h"
#include "RHIResources.h"

class FD3D12Buffer;
class FD3D12Texture;
class FD3D12RayTracingScene;
struct FD3D12ResidencyHandle;

inline ERHIDescriptorType RHIDescriptorTypeFromD3D12ViewDesc(const D3D12_SHADER_RESOURCE_VIEW_DESC& ViewDesc)
{
	if (ViewDesc.ViewDimension == D3D12_SRV_DIMENSION_UNKNOWN)
	{
		return ERHIDescriptorType::Invalid;
	}

	if (ViewDesc.ViewDimension == D3D12_SRV_DIMENSION_BUFFER)
	{
		return ERHIDescriptorType::BufferSRV;
	}

#if D3D12_RHI_RAYTRACING
	if (ViewDesc.ViewDimension == D3D12_SRV_DIMENSION_RAYTRACING_ACCELERATION_STRUCTURE)
	{
		return ERHIDescriptorType::AccelerationStructure;
	}
#endif

	return ERHIDescriptorType::TextureSRV;
}

inline ERHIDescriptorType RHIDescriptorTypeFromD3D12ViewDesc(const D3D12_UNORDERED_ACCESS_VIEW_DESC& ViewDesc)
{
	if (ViewDesc.ViewDimension == D3D12_UAV_DIMENSION_UNKNOWN)
	{
		return ERHIDescriptorType::Invalid;
	}

	if (ViewDesc.ViewDimension == D3D12_UAV_DIMENSION_BUFFER)
	{
		return ERHIDescriptorType::BufferUAV;
	}

	return ERHIDescriptorType::TextureUAV;
}

struct FD3D12DefaultViews
{
	FD3D12OfflineDescriptor NullSRV;
	FD3D12OfflineDescriptor NullRTV;
	FD3D12OfflineDescriptor NullUAV;
	FD3D12OfflineDescriptor NullCBV;
	FD3D12OfflineDescriptor NullDSV;

	TRefCountPtr<class FD3D12SamplerState> DefaultSampler;
};

// Holds the mip, array and plane range for a view, as well as the total number of these for the underlying resource.
struct FD3D12ViewRange
{
	FD3D12ViewRange() = default;
	FD3D12ViewRange(D3D12_CONSTANT_BUFFER_VIEW_DESC  const& ViewDesc);
	FD3D12ViewRange(D3D12_SHADER_RESOURCE_VIEW_DESC  const& ViewDesc);
	FD3D12ViewRange(D3D12_UNORDERED_ACCESS_VIEW_DESC const& ViewDesc);
	FD3D12ViewRange(D3D12_RENDER_TARGET_VIEW_DESC    const& ViewDesc);
	FD3D12ViewRange(D3D12_DEPTH_STENCIL_VIEW_DESC    const& ViewDesc);
	
	// @todo remove this
	bool DoesNotOverlap(FD3D12ViewRange const& Other) const
	{
		return Mip  .ExclusiveLast() <= Other.Mip  .First || Other.Mip  .ExclusiveLast() <= Mip  .First
		    || Array.ExclusiveLast() <= Other.Array.First || Other.Array.ExclusiveLast() <= Array.First
			|| Plane.ExclusiveLast() <= Other.Plane.First || Other.Plane.ExclusiveLast() <= Plane.First;
	}

	// @todo remove this
	uint8 MostDetailedMip() const
	{
		return Mip.First;
	}

	// The subresource range covered by the view.
	FRHIRange16 Array;
	FRHIRange8  Plane;
	FRHIRange8  Mip;
};

struct FD3D12ResourceLayout
{
	FD3D12ResourceLayout() = default;
	FD3D12ResourceLayout(FD3D12ResourceDesc const& ResourceDesc)
		: NumArraySlices(ResourceDesc.DepthOrArraySize)
		, NumPlanes     (UE::DXGIUtilities::GetPlaneCount(ResourceDesc.Format))
		, NumMips       (ResourceDesc.MipLevels)
	{}

	uint16 NumArraySlices = 0;
	uint8  NumPlanes      = 0;
	uint8  NumMips        = 0;
};

struct FD3D12ViewSubset
{
	FD3D12ViewSubset() = default;
	FD3D12ViewSubset(FD3D12ResourceLayout const& Layout, FD3D12ViewRange const& Range)
		: Layout(Layout)
		, Range (Range)
	{}

	bool IsWholeResource() const
	{
		return 
			Range.Mip  .First == 0 && Range.Mip  .Num == Layout.NumMips        &&
			Range.Array.First == 0 && Range.Array.Num == Layout.NumArraySlices &&
			Range.Plane.First == 0 && Range.Plane.Num == Layout.NumPlanes;
	}

	bool HasPlane(uint32 PlaneIndex) const
	{
		return Range.Plane.IsInRange(PlaneIndex);
	}

	FD3D12ViewSubset SelectPlane(uint32 PlaneIndex) const
	{
		check(PlaneIndex >= Range.Plane.First && PlaneIndex < Range.Plane.ExclusiveLast());

		FD3D12ViewSubset Copy { *this };
		Copy.Range.Plane = { PlaneIndex, 1 };

		return Copy;
	}

	//
	// This iterator iterates over the subresources within a view subset. eg:
	//
	//    for (uint32 SubresourceIndex : ViewSubset)
	//    {
	//        // Action for each subresource
	//    }
	//
	class FIterator final
	{
	public:
		FIterator(FD3D12ViewSubset const& ViewSubset, uint8 MipSlice, uint16 ArraySlice, uint8 PlaneSlice)
			: MipMax     (ViewSubset.Range.Mip.Num)
			, ArrayMax   (ViewSubset.Range.Array.Num)
			, ArrayStride(ViewSubset.Layout.NumMips)
			, PlaneStride(ViewSubset.Layout.NumMips * (ViewSubset.Layout.NumArraySlices - ViewSubset.Range.Array.Num))
		{
			MipRangeStart = D3D12CalcSubresource(
				MipSlice,
				ArraySlice,
				PlaneSlice,
				ViewSubset.Layout.NumMips,
				ViewSubset.Layout.NumArraySlices
			);
		}

		FIterator& operator ++ ()
		{
			if (++MipOffset == MipMax)
			{
				// Move to next array slice
				MipOffset = 0;
				MipRangeStart += ArrayStride;

				if (++ArrayOffset == ArrayMax)
				{
					// Move to next plane slice
					ArrayOffset = 0;
					MipRangeStart += PlaneStride;
				}
			}

			return *this;
		}

		uint32 operator * () const { return MipRangeStart + MipOffset; }

		bool operator == (FIterator const& RHS) const { return *(*this) == *RHS; }
		bool operator != (FIterator const& RHS) const { return !(*this == RHS); }

	private:
		// Constants
		uint32 const MipMax;
		uint32 const ArrayMax;
		uint32 const ArrayStride;
		uint32 const PlaneStride;

		// Counters
		uint32 MipRangeStart;
		uint32 MipOffset   = 0;
		uint32 ArrayOffset = 0;
	};

	FIterator begin() const { return FIterator(*this, Range.Mip.First, Range.Array.First, Range.Plane.First          ); }
	FIterator end  () const { return FIterator(*this, Range.Mip.First, Range.Array.First, Range.Plane.ExclusiveLast()); }

	FD3D12ResourceLayout Layout;
	FD3D12ViewRange      Range;
};

// Enumeration of supported view types. Each view type must pass this up to their parent FD3D12View constructor.
enum class ED3D12ViewType : uint8
{
	ShaderResource,
	ConstantBuffer,
	UnorderedAccess,
	RenderTarget,
	DepthStencil,
};

// Manages descriptor allocations and view creation
class FD3D12View : public FD3D12DeviceChild, public FD3D12ShaderResourceRenameListener
{
	typedef FD3D12OfflineDescriptor FD3D12DefaultViews::* FNullDescPtr;

public:
	enum class EReason
	{
		InitialCreate,
		UpdateOrRename,
	};

	struct FResourceInfo
	{
		FD3D12BaseShaderResource*               BaseResource     = nullptr;
		FD3D12ResourceLocation*                 ResourceLocation = nullptr;
		FD3D12Resource*                         Resource         = nullptr;

		FResourceInfo() = default;

		// Constructor for renamable shader resources
		FResourceInfo(FD3D12BaseShaderResource* InBaseResource)
			: BaseResource    (InBaseResource)
			, ResourceLocation(InBaseResource ? &InBaseResource->ResourceLocation : nullptr)
			, Resource        (InBaseResource ? InBaseResource->GetResource()     : nullptr)
		{}

		// Constructor for manual views (does not automatically register for resource renames)
		FResourceInfo(FD3D12ResourceLocation* InResourceLocation)
			: BaseResource    (nullptr)
			, ResourceLocation(InResourceLocation)
			, Resource        (InResourceLocation ? InResourceLocation->GetResource() : nullptr)
		{}
	};

	FD3D12Resource*                         GetResource        () const { check(IsInitialized()); return ResourceInfo.Resource;         }
	FD3D12BaseShaderResource*               GetShaderResource  () const { check(IsInitialized()); return ResourceInfo.BaseResource;     }
	FD3D12ResourceLocation*                 GetResourceLocation() const { check(IsInitialized()); return ResourceInfo.ResourceLocation; }
	TConstArrayView<FD3D12ResidencyHandle*> GetResidencyHandles() const { check(IsInitialized()); return ResourceInfo.Resource ? ResourceInfo.Resource->GetResidencyHandles() : TConstArrayView<FD3D12ResidencyHandle*>(); }
	FD3D12ViewSubset const&                 GetViewSubset      () const { check(IsInitialized()); return ViewSubset;                    }
	FD3D12OfflineDescriptor                 GetOfflineCpuHandle() const { check(IsInitialized()); return OfflineCpuHandle;              }

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FRHIDescriptorHandle        GetBindlessHandle() const { return BindlessHandle;           }
	bool                        IsBindless       () const { return BindlessHandle.IsValid(); }
#else
	FRHIDescriptorHandle        GetBindlessHandle() const { return FRHIDescriptorHandle();   }
	constexpr bool              IsBindless       () const { return false;                    }
#endif

protected:
	FD3D12View() = delete;
	FD3D12View(FD3D12Device* InDevice, ERHIDescriptorType InDescriptorType, ERHIDescriptorHeapType InHeapType, FD3D12View* FirstLinkedObject);
	virtual ~FD3D12View();
		
	virtual void UpdateDescriptor() = 0;

	void UpdateResourceInfo(FResourceInfo const& InResource, FNullDescPtr NullDescriptor);
	void CreateView(FResourceInfo const& InResource, FNullDescPtr NullDescriptor);
	void UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, FNullDescPtr NullDescriptor);

	bool IsInitialized() const { return ResourceInfo.ResourceLocation != nullptr; }

	void InitializeBindlessSlot();
	void UpdateBindlessSlot(FD3D12ContextArray const& Contexts);

	FResourceInfo ResourceInfo;
	FD3D12ViewSubset ViewSubset;

	FD3D12OfflineDescriptor OfflineCpuHandle;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// BindlessHandle is shared across multi-GPU linked objects.  First linked object is responsible for destroying the handle.
	// The D3D12View class doesn't have access to FD3D12LinkedAdapterObject::IsHeadLink(), so this separate bool is used.
	FRHIDescriptorHandle BindlessHandle;
	bool bFirstLinkedObject;
#endif
	ERHIDescriptorHeapType const HeapType;
};

template <typename TParent, typename TDesc>
class TD3D12View : public FD3D12View
{
protected:
	TDesc D3DViewDesc;

	TD3D12View(FD3D12Device* InDevice, ERHIDescriptorType InDescriptorType, ERHIDescriptorHeapType InHeapType, TD3D12View* FirstLinkedObject)
		: FD3D12View(InDevice, InDescriptorType, InHeapType, FirstLinkedObject)
	{}

	void CreateView(FResourceInfo const& InResource, TDesc const& InD3DViewDesc)
	{
		D3DViewDesc = InD3DViewDesc;
		ViewSubset.Range = InD3DViewDesc;
		FD3D12View::CreateView(InResource, TParent::Null);
	}

	void UpdateView(FD3D12ContextArray const& Contexts, FResourceInfo const& InResource, TDesc const& InD3DViewDesc)
	{
		D3DViewDesc = InD3DViewDesc;
		ViewSubset.Range = InD3DViewDesc;
		FD3D12View::UpdateView(Contexts, InResource, TParent::Null);
	}

	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override
	{
		// Can only be called if the base shader resource is not null.
		check(ResourceInfo.BaseResource == InRenamedResource && ResourceInfo.ResourceLocation == InNewResourceLocation);

		UpdateView(Contexts, InRenamedResource, D3DViewDesc);
	}
	
public:
	TDesc const& GetD3DDesc() const { return D3DViewDesc; }
};

class FD3D12ConstantBufferView final : public TD3D12View<FD3D12ConstantBufferView, D3D12_CONSTANT_BUFFER_VIEW_DESC>
{
public:
	static constexpr FD3D12OfflineDescriptor FD3D12DefaultViews::*Null { &FD3D12DefaultViews::NullCBV };

	FD3D12ConstantBufferView(FD3D12Device* InParent, FD3D12ConstantBufferView* FirstLinkedObject, ERHIDescriptorType InDescriptorType = ERHIDescriptorType::Invalid);
	void CreateView(const FResourceInfo& InResource, uint32 InOffset, uint32 InAlignedSize);
	void UpdateView(const FD3D12ContextArray& Contexts, const FResourceInfo& InResource, uint32 InOffset, uint32 InAlignedSize);

private:
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override;
	virtual void UpdateDescriptor() override;

	uint32 Offset;
};

class FD3D12ShaderResourceView : public TD3D12View<FD3D12ShaderResourceView, D3D12_SHADER_RESOURCE_VIEW_DESC>
{
public:
	static constexpr FD3D12OfflineDescriptor FD3D12DefaultViews::*Null { &FD3D12DefaultViews::NullSRV };

	enum class EFlags : uint8
	{
		None = 0,
		SkipFastClearFinalize = 1 << 0,
	};
	FRIEND_ENUM_CLASS_FLAGS(EFlags)

	FD3D12ShaderResourceView(FD3D12Device* InDevice, FD3D12ShaderResourceView* FirstLinkedObject, ERHIDescriptorType InDescriptorType, FD3D12RayTracingScene* InRayTracingScene = nullptr);
	void CreateView(FResourceInfo const& InResource, D3D12_SHADER_RESOURCE_VIEW_DESC const& InD3DViewDesc, EFlags InFlags);
	void UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, const D3D12_SHADER_RESOURCE_VIEW_DESC& InD3DViewDesc, EFlags InFlags);

	bool GetSkipFastClearFinalize() const { return EnumHasAnyFlags(Flags, EFlags::SkipFastClearFinalize); }
	void UpdateMinLODClamp(FD3D12ContextArray const& Contexts, float MinLODClamp);

	FD3D12RayTracingScene* GetRayTracingScene() const { return RayTracingScene; }

protected:
	void UpdateResourceInfo(const FResourceInfo& InResource, const D3D12_SHADER_RESOURCE_VIEW_DESC& InD3DViewDesc, EFlags InFlags);
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override;
	virtual void UpdateDescriptor() override;

	// Required for resource renaming
	uint64 OffsetInBytes = 0;
	uint32 StrideInBytes = 0;

	EFlags Flags = EFlags::None;

	FD3D12RayTracingScene* RayTracingScene = nullptr;
};

ENUM_CLASS_FLAGS(FD3D12ShaderResourceView::EFlags)

class FD3D12UnorderedAccessView : public TD3D12View<FD3D12UnorderedAccessView, D3D12_UNORDERED_ACCESS_VIEW_DESC>
{
public:
	static constexpr FD3D12OfflineDescriptor FD3D12DefaultViews::*Null { &FD3D12DefaultViews::NullUAV };

	enum class EFlags : uint8
	{
		None = 0,
		NeedsCounter = 1 << 0
	};
	FRIEND_ENUM_CLASS_FLAGS(EFlags)

	FD3D12UnorderedAccessView(FD3D12Device* InDevice, FD3D12UnorderedAccessView* FirstLinkedObject, ERHIDescriptorType InDescriptorType);
	void CreateView(FResourceInfo const& InResource, D3D12_UNORDERED_ACCESS_VIEW_DESC const& InD3DViewDesc, EFlags InFlags);
	void UpdateView(FD3D12ContextArray const& Contexts, const FResourceInfo& InResource, const D3D12_UNORDERED_ACCESS_VIEW_DESC& InD3DViewDesc, EFlags InFlags);

	FD3D12Resource* GetCounterResource() const
	{
		return CounterResource;
	}

protected:
	void UpdateResourceInfo(const FResourceInfo& InResource, const D3D12_UNORDERED_ACCESS_VIEW_DESC& InD3DViewDesc, EFlags InFlags);
	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource* InRenamedResource, FD3D12ResourceLocation* InNewResourceLocation) override;
	virtual void UpdateDescriptor() override;

	TRefCountPtr<FD3D12Resource> CounterResource;

	// Required for resource renaming
	uint64 OffsetInBytes = 0;
	uint32 StrideInBytes = 0;
};

ENUM_CLASS_FLAGS(FD3D12UnorderedAccessView::EFlags)

class FD3D12RenderTargetView final : public TD3D12View<FD3D12RenderTargetView, D3D12_RENDER_TARGET_VIEW_DESC>
{
public:
	static constexpr FD3D12OfflineDescriptor FD3D12DefaultViews::*Null { &FD3D12DefaultViews::NullRTV };

	FD3D12RenderTargetView(FD3D12Device* InDevice, FD3D12RenderTargetView* FirstLinkedObject);
	using TD3D12View::CreateView;

private:
	virtual void UpdateDescriptor() override;
};

class FD3D12DepthStencilView final : public TD3D12View<FD3D12DepthStencilView, D3D12_DEPTH_STENCIL_VIEW_DESC>
{
public:
	static constexpr FD3D12OfflineDescriptor FD3D12DefaultViews::*Null { &FD3D12DefaultViews::NullDSV };

	FD3D12DepthStencilView(FD3D12Device* InDevice, FD3D12DepthStencilView* FirstLinkedObject);
	using TD3D12View::CreateView;

	bool HasDepth  () const { return GetViewSubset().HasPlane(0); }
	bool HasStencil() const { return GetViewSubset().HasPlane(1); }

	FD3D12ViewSubset GetDepthOnlySubset  () const { return GetViewSubset().SelectPlane(0); }
	FD3D12ViewSubset GetStencilOnlySubset() const { return GetViewSubset().SelectPlane(1); }

private:
	virtual void UpdateDescriptor() override;
};

template <typename TParent>
struct FD3D12DeferredInitView : public FD3D12LinkedAdapterObject<TParent>
{
	void CreateViews(FRHICommandListBase& RHICmdList)
	{
		auto InitLambda = [this](FRHICommandListBase&)
		{
			for (TParent& LinkedView : *this)
			{
				LinkedView.CreateView();
			}
		};

		if (RHICmdList.IsTopOfPipe())
		{
			// We have to defer the view initialization to the RHI thread if the resource is dynamic (and RHI threading is enabled), since dynamic resources can be renamed.
			// Also insert an RHI thread fence to prevent parallel translate tasks running until this command has completed.
			RHICmdList.EnqueueLambda(MoveTemp(InitLambda));
			RHICmdList.RHIThreadFence(true);
		}
		else
		{
			// Run the command directly if we're bypassing RHI command list recording, or the buffer is not dynamic.
			InitLambda(RHICmdList);
		}
	}
};

// Wrapper classes to expose the internal SRV/UAV types to the render as actual RHI resources.
class FD3D12ShaderResourceView_RHI
	: public FRHIShaderResourceView
	, public FD3D12ShaderResourceView
	, public FD3D12DeferredInitView<FD3D12ShaderResourceView_RHI>
{
public:
	FD3D12ShaderResourceView_RHI(FD3D12Device* InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc, FD3D12ShaderResourceView_RHI* FirstLinkedObject);

	virtual void CreateView();
	virtual void UpdateView(FD3D12ContextArray const& Contexts);

	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource*, FD3D12ResourceLocation*) override
	{
		// Recreate the view from the FRHIViewDesc rather than simply updating the D3D12 descriptor handle from the existing D3D view desc.
		// This is because the streaming system may have replaced the underlying resource with one that has a different layout.
		UpdateView(Contexts);
	}

	virtual FRHIDescriptorHandle GetBindlessHandle() const override { return FD3D12ShaderResourceView::GetBindlessHandle(); }
};

class FD3D12UnorderedAccessView_RHI
	: public FRHIUnorderedAccessView
	, public FD3D12UnorderedAccessView
	, public FD3D12DeferredInitView<FD3D12UnorderedAccessView_RHI>
{
public:
	FD3D12UnorderedAccessView_RHI(FD3D12Device* InDevice, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc, FD3D12UnorderedAccessView_RHI* FirstLinkedObject);

	virtual void CreateView();
	virtual void UpdateView(FD3D12ContextArray const& Contexts);

	virtual void ResourceRenamed(FD3D12ContextArray const& Contexts, FD3D12BaseShaderResource*, FD3D12ResourceLocation*) override
	{
		// Recreate the view from the FRHIViewDesc rather than simply updating the D3D12 descriptor handle from the existing D3D view desc.
		// This is because the streaming system may have replaced the underlying resource with one that has a different layout.
		UpdateView(Contexts);
	}

	virtual FRHIDescriptorHandle GetBindlessHandle() const override { return FD3D12UnorderedAccessView::GetBindlessHandle(); }
};

template<>
struct TD3D12ResourceTraits<FRHIShaderResourceView>
{
	typedef FD3D12ShaderResourceView_RHI TConcreteType;
};
template<>
struct TD3D12ResourceTraits<FRHIUnorderedAccessView>
{
	typedef FD3D12UnorderedAccessView_RHI TConcreteType;
};
