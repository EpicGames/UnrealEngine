// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Viewport.cpp: D3D viewport RHI implementation.
	=============================================================================*/

#include "D3D12Viewport.h"
#include "D3D12RHIPrivate.h"
#include "RenderCore.h"
#include "HDRHelper.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIUtilities.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

namespace D3D12RHI
{
	/**
	 * RHI console variables used by viewports.
	 */
	namespace RHIConsoleVariables
	{
#if !UE_BUILD_SHIPPING
#if LOG_VIEWPORT_EVENTS
		int32 LogViewportEvents = 1;
#else
		int32 LogViewportEvents = 0;
#endif
		static FAutoConsoleVariableRef CVarLogViewportEvents(
			TEXT("D3D12.LogViewportEvents"),
			LogViewportEvents,
			TEXT("Log all the viewport events."),
			ECVF_RenderThreadSafe
		);
#endif
	};

	bool GUseSubmissionThreadPresent = false;
	static FAutoConsoleVariableRef CVarD3D12SubmissionTimeout(
		TEXT("r.D3D12.PresentOnSubmissionThread"),
		GUseSubmissionThreadPresent,
		TEXT("When true, Present is called on the Submission Thread instead of the RHI Thread"),
		ECVF_RenderThreadSafe);


	int32 GD3D12NumViewportBuffers = D3D12RHI_DEFAULT_NUM_BACKBUFFER;
	static FAutoConsoleVariableRef CVarD3D12NumViewportBuffers(
		TEXT("r.D3D12.NumViewportBuffers"),
		GD3D12NumViewportBuffers,
		TEXT("The number of viewport buffers to allocate for the RHI viewport."),
		ECVF_ReadOnly);
}
using namespace D3D12RHI;

FCriticalSection FD3D12Viewport::DXGIBackBufferLock;

/**
 * Creates a FD3D12Surface to represent a swap chain's back buffer.
 */
FD3D12Texture* GetSwapChainSurface(FD3D12Device* Parent, EPixelFormat PixelFormat, uint32 SizeX, uint32 SizeY, IDXGISwapChain* SwapChain, uint32 BackBufferIndex, TRefCountPtr<ID3D12Resource> BackBufferResourceOverride)
{
	verify(D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN || SwapChain == nullptr);

	FD3D12Adapter* Adapter = Parent->GetParentAdapter();

	// Grab the back buffer
	TRefCountPtr<ID3D12Resource> BackBufferResource;
	if (SwapChain)
	{
#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
		VERIFYD3D12RESULT_EX(SwapChain->GetBuffer(BackBufferIndex, IID_PPV_ARGS(BackBufferResource.GetInitReference())), Parent->GetDevice());
#else // #if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
		return nullptr;
#endif // #if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	}
	else if (BackBufferResourceOverride.IsValid())
	{
		BackBufferResource = BackBufferResourceOverride;
	}
	else
	{
		const D3D12_HEAP_PROPERTIES HeapProps = CD3DX12_HEAP_PROPERTIES(D3D12_HEAP_TYPE_DEFAULT, (uint32)Parent->GetGPUIndex(), Parent->GetGPUMask().GetNative());

		// Create custom back buffer texture as no swap chain is created in pixel streaming windowless mode
		D3D12_RESOURCE_DESC TextureDesc;
		TextureDesc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
		TextureDesc.Alignment = 0;
		TextureDesc.Width  = SizeX;
		TextureDesc.Height = SizeY;
		TextureDesc.DepthOrArraySize = 1;
		TextureDesc.MipLevels = 1;
		TextureDesc.Format = UE::DXGIUtilities::GetSwapChainFormat(PixelFormat);
		TextureDesc.SampleDesc.Count = 1;
		TextureDesc.SampleDesc.Quality = 0;
		TextureDesc.Layout = D3D12_TEXTURE_LAYOUT_UNKNOWN;
		TextureDesc.Flags = D3D12_RESOURCE_FLAG_ALLOW_RENDER_TARGET;

		Parent->GetParentAdapter()->CreateCommittedResourceRaw(
			HeapProps,
			D3D12_HEAP_FLAG_NONE,
			TextureDesc,
			ED3D12Access::Present,
			nullptr,
			BackBufferResource);
	}

	FD3D12ResourceDesc BackBufferDesc = BackBufferResource->GetDesc();
	BackBufferDesc.bBackBuffer = true;

	FString Name = FString::Printf(TEXT("BackBuffer%d"), BackBufferIndex);
	
	ETextureCreateFlags SwapchainTextureCreateFlags = ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Presentable | ETextureCreateFlags::ResolveTargetable;

#if D3D12RHI_SUPPORTS_UAV_BACKBUFFER
	SwapchainTextureCreateFlags |= ETextureCreateFlags::UAV;
#endif

	bool const bQuadBufferStereo = FD3D12DynamicRHI::GetD3DRHI()->IsQuadBufferStereoEnabled();

	FRHITextureCreateDesc CreateDesc = bQuadBufferStereo
		? FRHITextureCreateDesc::Create2DArray(*Name)
		: FRHITextureCreateDesc::Create2D(*Name);

	CreateDesc
		.SetExtent(FIntPoint((uint32)BackBufferDesc.Width, BackBufferDesc.Height))
		.SetFormat(PixelFormat)
		.SetFlags(SwapchainTextureCreateFlags)
		.SetInitialState(ERHIAccess::Present);

	if (bQuadBufferStereo)
	{
		CreateDesc.SetArraySize(2);
	}

	FD3D12DynamicRHI* DynamicRHI = FD3D12DynamicRHI::GetD3DRHI();

	FD3D12Texture* SwapChainTexture = Adapter->CreateLinkedObject<FD3D12Texture>(FRHIGPUMask::All(), [&](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
	{
		FD3D12Texture* NewTexture = DynamicRHI->CreateNewD3D12Texture(CreateDesc, Device);

		if (Device->GetGPUIndex() == Parent->GetGPUIndex())
		{
			FD3D12Resource* NewResourceWrapper =
				new FD3D12Resource(
					Device,
					FRHIGPUMask::All(),
					BackBufferResource,
					ED3D12Access::Common,
					BackBufferDesc);
			NewResourceWrapper->AddRef();
			NewTexture->ResourceLocation.AsStandAlone(NewResourceWrapper);
		}
		else // If this is not the GPU which will hold the back buffer, create a compatible texture so that it can still render to the viewport.
		{
			FClearValueBinding ClearValueBinding;
			SafeCreateTexture2D(Device,
				Adapter,
				BackBufferDesc,
				nullptr, // &ClearValueBinding,
				&NewTexture->ResourceLocation,
				NewTexture,
				PixelFormat,
				TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_Presentable,
				ED3D12Access::Common,
				ED3D12Access::Common,
				*Name);
		}

		// active stereoscopy initialization
		if (FD3D12DynamicRHI::GetD3DRHI()->IsQuadBufferStereoEnabled())
		{
			// left
			D3D12_RENDER_TARGET_VIEW_DESC RTVDescLeft = {};
			RTVDescLeft.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DARRAY;
			RTVDescLeft.Format = BackBufferDesc.Format;
			RTVDescLeft.Texture2DArray.MipSlice = 0;
			RTVDescLeft.Texture2DArray.FirstArraySlice = 0;
			RTVDescLeft.Texture2DArray.ArraySize = 1;

			// right
			D3D12_RENDER_TARGET_VIEW_DESC RTVDescRight = {};
			RTVDescRight.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2DARRAY;
			RTVDescRight.Format = BackBufferDesc.Format;
			RTVDescRight.Texture2DArray.MipSlice = 0;
			RTVDescRight.Texture2DArray.FirstArraySlice = 1;
			RTVDescRight.Texture2DArray.ArraySize = 1;

			NewTexture->SetNumRTVs(2);
			NewTexture->EmplaceRTV(RTVDescLeft, 0, FirstLinkedObject);
			NewTexture->EmplaceRTV(RTVDescRight, 1, FirstLinkedObject);
		}
		else
		{
			// create the render target view
			D3D12_RENDER_TARGET_VIEW_DESC RTVDesc = {};
			RTVDesc.Format = BackBufferDesc.Format;
			RTVDesc.ViewDimension = D3D12_RTV_DIMENSION_TEXTURE2D;
			RTVDesc.Texture2D.MipSlice = 0;

			NewTexture->SetNumRTVs(1);
			NewTexture->EmplaceRTV(RTVDesc, 0, FirstLinkedObject);
		}

		// create a shader resource view to allow using the backbuffer as a texture
		D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
		SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
		SRVDesc.Format = BackBufferDesc.Format;
		SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_TEXTURE2D;
		SRVDesc.Texture2D.MostDetailedMip = 0;
		SRVDesc.Texture2D.MipLevels = 1;

		NewTexture->EmplaceSRV(SRVDesc, FirstLinkedObject);

		return NewTexture;
	});

	SetD3D12ResourceName(SwapChainTexture->GetResource(), *Name);

	const D3D12_RESOURCE_ALLOCATION_INFO AllocationInfo = Parent->GetDevice()->GetResourceAllocationInfo(0, 1, &SwapChainTexture->GetResource()->GetDesc());
	SwapChainTexture->ResourceLocation.SetSize(AllocationInfo.SizeInBytes);

	FD3D12TextureStats::D3D12TextureAllocated(*SwapChainTexture);
	return SwapChainTexture;
}

FD3D12Viewport::~FD3D12Viewport()
{
	check(IsInRHIThread() || IsInRenderingThread());

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	// If the swap chain was in fullscreen mode, switch back to windowed before releasing the swap chain.
	// DXGI throws an error otherwise.
	if (SwapChain1)
	{
		SwapChain1->SetFullscreenState(0, nullptr);
	}
#endif // D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN

	//Do not destroy FD3D12Viewport if waiting for a Present
	WaitForLastPresent();	

	GetParentAdapter()->GetViewports().Remove(this);

	FinalDestroyInternal();
}

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
inline DXGI_MODE_DESC SetupDXGI_MODE_DESC(uint32 SizeX, uint32 SizeY, EPixelFormat PixelFormat)
{
	DXGI_MODE_DESC Ret{};
	Ret.Width = SizeX;
	Ret.Height = SizeY;
	Ret.Format = UE::DXGIUtilities::GetSwapChainFormat(PixelFormat);
	return Ret;
}
#endif

void FD3D12Viewport::InitializeBackBufferArrays()
{
#if WITH_MGPU
	// This is a temporary helper to visualize what each GPU is rendering. 
	// Not specifying a value will cycle swap chain through all GPUs.
	BackbufferMultiGPUBinding = 0;
	if (GNumExplicitGPUsForRendering > 1)
	{
		if (FParse::Value(FCommandLine::Get(), TEXT("PresentGPU="), BackbufferMultiGPUBinding))
		{
			BackbufferMultiGPUBinding = FMath::Clamp<int32>(BackbufferMultiGPUBinding, INDEX_NONE, (int32)GNumExplicitGPUsForRendering - 1) ;
		}
	}
#endif // WITH_MGPU

	for (FBackBufferData& Data : BackBuffers)
	{
		Data = FBackBufferData();
	}
}

void FD3D12Viewport::Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat PreferredPixelFormat)
{
	if (bResizeInProgress)
	{
		return;
	}

	bResizeInProgress = true;
	ON_SCOPE_EXIT
	{
		bResizeInProgress = false;
	};

	FD3D12Adapter* Adapter = GetParentAdapter();

#if !UE_BUILD_SHIPPING
	if (RHIConsoleVariables::LogViewportEvents)
	{
		const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
		UE_LOGF(LogD3D12RHI, Log, "Thread %ls: Resize Viewport, %ls", ThreadName.GetCharArray().GetData(), *GetStateString());
	}
#endif

	// Log relevant state changes, makes it easier to track/reproduce crashes
	const bool bLogEvent =
		(bIsFullscreen != bInIsFullscreen) ||
		(bIsFullscreen && (InSizeX != SizeX || InSizeY != SizeY));

	FString OldState;
	if (bLogEvent)
	{
		OldState = GetStateString();
	}

	// Flush the outstanding GPU work and wait for it to complete.
	FlushRenderingCommands();
	Adapter->BlockUntilIdle();

	// Unbind any dangling references to resources.
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		FD3D12Device* Device = Adapter->GetDevice(GPUIndex);
		Device->GetDefaultCommandContext().ClearState(FD3D12ContextCommon::EClearStateMode::TransientOnly);
	}

	if (IsValidRef(CustomPresent))
	{
		CustomPresent->OnBackBufferResize();
	}

	{
		auto ReleaseBackBufferTexture = [](TRefCountPtr<FD3D12Texture>&Texture)
		{
			if (IsValidRef(Texture))
			{
				// Tell the back buffer to delete immediately so that we can call resize.
				for (FD3D12Texture& SubResource : *Texture)
				{
					SubResource.GetResource()->DoNotDeferDelete();
				}
			}

			Texture.SafeRelease();
			check(Texture == nullptr);
		};

		for (uint32 Index = 0; Index < NumBackBuffers; Index++)
		{
			ReleaseBackBufferTexture(BackBuffers[Index].Texture);
#if D3D12RHI_USE_SDR_BACKBUFFER
			ReleaseBackBufferTexture(BackBuffers[Index].TextureSDR);
#endif
		}
	}

	ClearPresentQueue();

	// Flush the outstanding GPU work and wait for it to complete.
	FlushRenderingCommands();
	Adapter->BlockUntilIdle();

	// Keep the current pixel format if one wasn't specified.
	if (PreferredPixelFormat == PF_Unknown)
	{
		PreferredPixelFormat = PixelFormat;
	}

	// Reset the full screen lost because we are resizing and handling fullscreen state change and full recreation of back buffers already
	// We don't want to call resize again, which could happen during ConditionalResetSwapChain otherwise
	bFullscreenLost = false;

	if (SizeX != InSizeX || SizeY != InSizeY || PixelFormat != PreferredPixelFormat)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		PixelFormat = PreferredPixelFormat;

		check(SizeX > 0);
		check(SizeY > 0);
#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
		if (bNeedSwapChain)
		{
			if (bInIsFullscreen)
			{
				const DXGI_MODE_DESC BufferDesc = SetupDXGI_MODE_DESC(SizeX, SizeY, PixelFormat);
				if (FAILED(SwapChain1->ResizeTarget(&BufferDesc)))
				{
					ConditionalResetSwapChain(true);
				}
			}
		}
#endif // D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	}

	if (bIsFullscreen != bInIsFullscreen)
	{
		bIsFullscreen = bInIsFullscreen;
		bIsValid = false;

		if (bNeedSwapChain)
		{
			// Use ConditionalResetSwapChain to call SetFullscreenState, to handle the failure case.
			// Ignore the viewport's focus state; since Resize is called as the result of a user action we assume authority without waiting for Focus.
			ConditionalResetSwapChain(true);

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
			if (!bIsFullscreen)
			{
				// When exiting fullscreen, make sure that the window has the correct size. This is necessary in the following scenario:
				//	* we enter exclusive fullscreen with a resolution lower than the monitor's native resolution, or from windowed with a window size smaller than the screen
				//	* the application loses focus, so Slate asks us to switch to Windowed Fullscreen (see FSlateRenderer::IsViewportFullscreen)
				//	* InSizeX and InSizeY are given to us as the monitor resolution, so we resize the buffers to the correct resolution below (in ResizeInternal)
				//	* however, the target still has the smaller size, because Slate doesn't know it has to resize the window too (as far as it's concerned, it's already the right size)
				//	* therefore, we need to call ResizeTarget, which in windowed mode behaves like SetWindowPos.
				const DXGI_MODE_DESC BufferDesc = SetupDXGI_MODE_DESC(SizeX, SizeY, PixelFormat);
				SwapChain1->ResizeTarget(&BufferDesc);
			}
#endif // D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
		}
	}

	RECT WindowRect = {};
#if PLATFORM_WINDOWS
	GetWindowRect(WindowHandle, &WindowRect);
#endif
	FVector2D WindowTopLeft((float)WindowRect.left, (float)WindowRect.top);
	FVector2D WindowBottomRight((float)WindowRect.right, (float)WindowRect.bottom);
	bool bHDREnabled;
	HDRGetMetaData(DisplayOutputFormat, DisplayColorGamut, bHDREnabled, WindowTopLeft, WindowBottomRight, (void*)WindowHandle);

	ResizeInternal();

	// Enable HDR if desired.
	if (bHDREnabled)
	{
		EnableHDR();
	}
	else
	{
		ShutdownHDR();
	}

	if (bLogEvent)
	{
		UE_LOGF(LogD3D12RHI, Log, "Swapchain Resized:\n\tBefore: %ls\n\tAfter: %ls", *OldState, *GetStateString());
	}
}

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
void* FD3D12Viewport::GetNativeSwapChain() const
{
	return SwapChain1;
}
#endif // #if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN

void* FD3D12Viewport::GetNativeBackBufferTexture() const
{
	return GetBackBuffer_RHIThread()->GetResource();
}

void* FD3D12Viewport::GetNativeBackBufferRT() const
{
	return GetBackBuffer_RHIThread()->GetRenderTargetView(0, 0);
}

void FD3D12Viewport::SetCustomPresent(FRHICustomPresent* InCustomPresent)
{
	CustomPresent = InCustomPresent;
}

FRHICustomPresent* FD3D12Viewport::GetCustomPresent() const
{
	return CustomPresent;
}

/** Update the expected next present GPU back buffer index from RenderThread point of view */
void FD3D12Viewport::AdvanceExpectedBackBufferIndex_RenderThread(bool bPresent)
{
	bool bNeedsNativePresent = IsValidRef(CustomPresent) ? 
		CustomPresent->NeedsNativePresent() || CustomPresent->NeedsAdvanceBackbuffer() : bPresent;

	if (bNeedsNativePresent && IsPresentAllowed())
	{
#if WITH_MGPU
		FScopeLock Lock(&ExpectedBackBufferIndexLock);
#endif

		SetBackBufferIndex_RenderThread(ExpectedBackBufferIndex_RenderThread + 1);

#if !UE_BUILD_SHIPPING
		if (RHIConsoleVariables::LogViewportEvents)
		{
			const FString ThreadName(FThreadManager::Get().GetThreadName(FPlatformTLS::GetCurrentThreadId()));
			UE_LOGF(LogD3D12RHI, Log, "Thread %ls: Incrementing Expected RenderThread back buffer index of viewport: %#016llx to value: %u", ThreadName.GetCharArray().GetData(), this, ExpectedBackBufferIndex_RenderThread);
		}
#endif
	}
}

FString FD3D12Viewport::GetStateString()
{
	return FString::Printf(
		TEXT("Viewport=0x%p, Num=%d, Size=(%d,%d), PF=%d, DXGIFormat=0x%x, Fullscreen=%d, AllowTearing=%d"),
		this,
		NumBackBuffers, SizeX, SizeY, static_cast<int32>(PixelFormat),
		static_cast<int32>(UE::DXGIUtilities::GetSwapChainFormat(PixelFormat)),
		static_cast<int32>(bIsFullscreen),
		static_cast<int32>(bAllowTearing)
	);
}

static bool IsTransientPresentationError(HRESULT Result)
{
	return Result == E_INVALIDARG ||
		   Result == DXGI_ERROR_INVALID_CALL;
}

/** Presents the swap chain checking the return result. */
void FD3D12Viewport::PresentChecked(int32 SyncInterval)
{
	static constexpr uint32 MaxPresentFailures = 5;

	if (PresentBeginLatencyCallback)
	{
		PresentBeginLatencyCallback();
	}

	// Present the back buffer to the viewport window.
	// In case presentation failures are transient, don't fault on the first one.
	HRESULT Result = PresentInternal(SyncInterval);
	if (SUCCEEDED(Result))
	{
		CheckedPresentFailureCounter = 0;
	}
	else if (!IsTransientPresentationError(Result) || ++CheckedPresentFailureCounter >= MaxPresentFailures)
	{			
		VERIFYD3D12RESULT_LAMBDA(Result, GetParentAdapter()->GetD3DDevice(), [this]
		{
			return GetStateString();
		});
	}
	else
	{
		UE_LOGF(
			LogD3D12RHI, Error,
			"Swapchain presentation try %u/%u failed with HR(0x%x): %ls",
			CheckedPresentFailureCounter, MaxPresentFailures,
			Result, *GetStateString()
		);
	}

	if (PresentEndLatencyCallback)
	{
		PresentEndLatencyCallback();
	}

#if LOG_PRESENT
	const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
	UE_LOGF(LogD3D12RHI, Log, "*** PRESENT: Thread %ls: Viewport %#016llx: BackBuffer %#016llx (SyncInterval %u) ***", ThreadName.GetCharArray().GetData(), this, GetBackBuffer_RHIThread(), SyncInterval);
#endif
}

UE_TRACE_EVENT_BEGIN(Cpu, GetBackBufferOnRhiThread, NoSync)
UE_TRACE_EVENT_FIELD(uint8, BackBuffer)
UE_TRACE_EVENT_END()

FD3D12Texture* FD3D12Viewport::GetBackBuffer_RHIThread(bool bWaitForLastPresent) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12Viewport::GetBackBuffer_RHIThread);

	if (bWaitForLastPresent)
	{
		WaitForLastPresent();
	}
	
	{
#if CPUPROFILERTRACE_ENABLED
		UE_TRACE_LOG_SCOPED_T(Cpu, GetBackBufferOnRhiThread, CpuChannel)
		<< GetBackBufferOnRhiThread.BackBuffer(CurrentBackBufferIndex_RHIThread);
#endif // CPUPROFILERTRACE_ENABLED
	}

	return CurrentBackBuffer_RHIThread->Texture;
}


UE_TRACE_EVENT_BEGIN(Cpu, SchedulePresent, NoSync)
UE_TRACE_EVENT_FIELD(uint8, BackBuffer)
UE_TRACE_EVENT_FIELD(uint64, PresentCounter)
UE_TRACE_EVENT_FIELD(uint64, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, Present_SubmissionThread, NoSync)
UE_TRACE_EVENT_FIELD(uint8, BackBuffer)
UE_TRACE_EVENT_FIELD(uint64, PresentCounter)
UE_TRACE_EVENT_FIELD(uint64, FrameCounter)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, End_Rendering, NoSync)
UE_TRACE_EVENT_FIELD(uint8, BackBuffer)
UE_TRACE_EVENT_FIELD(uint64, PresentCounter)
UE_TRACE_EVENT_FIELD(uint64, FrameCounter)
UE_TRACE_EVENT_END()


bool FD3D12Viewport::SchedulePresent(FD3D12CommandContextBase* BaseContext, const FRHIPresentArgs& InPresentArgs)
{	
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, SchedulePresent, CpuChannel)
		<< SchedulePresent.BackBuffer(CurrentBackBufferIndex_RHIThread)
		<< SchedulePresent.PresentCounter(GRHIPresentCounter)
		<< SchedulePresent.FrameCounter(InPresentArgs.FrameCounter)
		;
#endif // CPUPROFILERTRACE_ENABLED

	check(BaseContext->GetParentAdapter() == GetParentAdapter());
	check(InPresentArgs.bPresent);

	WaitForLastPresent();

	FD3D12PresentArgs PresentArgs;
	PresentEvent = FGraphEvent::CreateGraphEvent();
	PresentArgs.DrawingViewport = this;
	PresentArgs.bLockToVsync = InPresentArgs.bLockToVsync;
	PresentArgs.FrameCounter = InPresentArgs.FrameCounter;
	PresentArgs.PresentEvent = PresentEvent;

	// MPGU Present only on first GPU context
	FD3D12CommandContext& Context = *BaseContext->GetSingleDeviceContext(0);
	Context.SetPresent(PresentArgs);

	return true;
}

void FD3D12Viewport::PresentOnSubmissionThread(FD3D12PresentArgs& PresentArgs)
{
	//Idle scopes needs to know that this is a RHI related thread
	FTaskTagScope TaskTagScope(ETaskTag::EParallelRhiThread);

#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, Present_SubmissionThread, CpuChannel)
		<< Present_SubmissionThread.BackBuffer(CurrentBackBufferIndex_RHIThread)
		<< Present_SubmissionThread.PresentCounter(GRHIPresentCounter)
		<< Present_SubmissionThread.FrameCounter(PresentArgs.FrameCounter)
		;
#endif // CPUPROFILERTRACE_ENABLED
	check(PresentArgs.IsValid());


	int32 SyncInterval = PresentArgs.bLockToVsync ? RHIGetSyncInterval() : 0;

	bool bNeedNativePresent = true;

	if (IsValidRef(CustomPresent))
	{
		SCOPE_CYCLE_COUNTER(STAT_D3D12CustomPresentTime);
		// CustomPresent::Present can decide to change the SyncInterval to toggle the Vsync
		bNeedNativePresent = CustomPresent->PresentOnSubmissionThread(SyncInterval);
	}
	
	if (bNeedNativePresent)
	{
		PresentChecked(SyncInterval);
	}

	if (IsValidRef(CustomPresent))
	{
		CustomPresent->PostPresent();
	}

	if (bNeedNativePresent || (IsValidRef(CustomPresent) && CustomPresent->NeedsAdvanceBackbuffer()))
	{
		// Increment back buffer
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 3
		if (SwapChain3)
		{
			SetBackBufferIndex_RHIThread(SwapChain3->GetCurrentBackBufferIndex());
		}
		else
#endif
		{
			SetBackBufferIndex_RHIThread(CurrentBackBufferIndex_RHIThread + 1);
		}

#if !UE_BUILD_SHIPPING
		if (RHIConsoleVariables::LogViewportEvents)
		{
			const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
			UE_LOGF(LogD3D12RHI, Log, "Thread %ls: Incrementing RHIThread back buffer index of viewport: %#016llx to value: %u BackBuffer %#016llx", ThreadName.GetCharArray().GetData(), this, CurrentBackBufferIndex_RHIThread, CurrentBackBuffer_RHIThread->Texture.GetReference());
		}
#endif
	}
	
	PresentArgs.PresentEvent->DispatchSubsequents();
}

void FD3D12Viewport::WaitForLastPresent() const
{
	FRenderThreadIdleScope Scope(ERenderThreadIdleTypes::WaitingForGPUPresent);

	if ( (!PresentEvent.IsValid()) || (!PresentEvent->IsAwaitable()) || (PresentEvent->IsComplete()) ) 
	{
		return;
	}
		
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TEXT("PresentEvent_Wait Frame"));
		PresentEvent->Wait();
	}
	
}

bool FD3D12Viewport::EndRendering(FD3D12CommandContextBase& ContextBase, const FRHIPresentArgs& InPresentArgs)
{
	if (!IsPresentAllowed() || !InPresentArgs.bPresent )
	{
		bLastFramePresented = false;
		return false;
	}
	
	bLastFramePresented = true;

#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, End_Rendering, CpuChannel)
		<< End_Rendering.BackBuffer(CurrentBackBufferIndex_RHIThread)
		<< End_Rendering.PresentCounter(GRHIPresentCounter)
		<< End_Rendering.FrameCounter(InPresentArgs.FrameCounter);
#endif // CPUPROFILERTRACE_ENABLED

	check(ContextBase.GetParentAdapter() == GetParentAdapter());

	const bool bCustomPresent = IsValidRef(CustomPresent);
	bool bNeedNativePresent = true;
	bool bNativelyPresented = false;
	FRHIPresentArgs LocalArgs = InPresentArgs;

	if (CheckSwapChain())
	{
		if (GUseSubmissionThreadPresent)
		{
			SchedulePresent(&ContextBase, InPresentArgs);

			// The native Present call has only been scheduled
			bNativelyPresented = false;
		}
		else
		{
			SyncToSubmissionThread(ContextBase);

			int32 SyncInterval = InPresentArgs.bLockToVsync ? RHIGetSyncInterval() : 0;

			if (bCustomPresent)
			{
				SCOPE_CYCLE_COUNTER(STAT_D3D12CustomPresentTime);
				// Give to the CustomPresent::Present the opportunity to use the RHIContext from the RHIThread
				// CustomPresent::Present can decide to change the SyncInterval to toggle the Vsync
				bNeedNativePresent = CustomPresent->Present(this, ContextBase, SyncInterval);
			}

			if (bNeedNativePresent)
			{
				PresentChecked(SyncInterval);
				bNativelyPresented = true;
			}

			if (bCustomPresent)
			{
				CustomPresent->PostPresent();
			}

			if (bNativelyPresented || (bCustomPresent && CustomPresent->NeedsAdvanceBackbuffer()))
			{
				// Increment back buffer
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 3
				if (bNativelyPresented && SwapChain3)
				{
					SetBackBufferIndex_RHIThread(SwapChain3->GetCurrentBackBufferIndex());
				}
				else
#endif
				{
					SetBackBufferIndex_RHIThread(CurrentBackBufferIndex_RHIThread + 1);
				}

#if !UE_BUILD_SHIPPING
				if (RHIConsoleVariables::LogViewportEvents)
				{
					const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
					UE_LOGF(LogD3D12RHI, Log, "Thread %ls: Incrementing RHIThread back buffer index of viewport: %#016llx to value: %u BackBuffer %#016llx", ThreadName.GetCharArray().GetData(), this, CurrentBackBufferIndex_RHIThread, CurrentBackBuffer_RHIThread->Texture.GetReference());
				}
#endif
			}
		}
	} 

	return bNativelyPresented;
}

void FD3D12Viewport::WaitForFrameEventCompletion()
{
	if (FrameSyncPoints.Num())
	{
		for (auto& SyncPoint : FrameSyncPoints)
		{
			if (SyncPoint)
			{
				SyncPoint->Wait();
			}
		}

		FrameSyncPoints.Reset();
	}
}

void FD3D12Viewport::IssueFrameEvent()
{
	TArray<FD3D12Payload*> Payloads;
	for (FD3D12Device* Device : ParentAdapter->GetDevices())
	{
		FD3D12CommandContext& Context = Device->GetDefaultCommandContext();

		FD3D12SyncPointRef SyncPoint = FD3D12SyncPoint::Create(ED3D12SyncPointType::GPUAndCPU, UE_D3D12_STATIC_FNAME("WaitPreviousFrameEvent"));

		Context.SignalSyncPoint(SyncPoint);
		Context.Finalize(Payloads);

		FrameSyncPoints.Emplace(MoveTemp(SyncPoint));
	}

	FD3D12DynamicRHI::GetD3DRHI()->SubmitPayloads(MoveTemp(Payloads));
}

bool FD3D12Viewport::CheckHDRSupport()
{
	return IsHDREnabled();
}

/*==============================================================================
 *	The following RHI functions must be called from the main thread.
 *=============================================================================*/
FViewportRHIRef FD3D12DynamicRHI::RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());

	const uint32 InNumBackBuffers = GD3D12NumViewportBuffers;

	FD3D12Viewport* RenderingViewport = new FD3D12Viewport(&GetAdapter(), (HWND)WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat, InNumBackBuffers);
	RenderingViewport->Init();
	return RenderingViewport;
}

void FD3D12DynamicRHI::RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	ResourceCast(ViewportRHI)->Resize(SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FD3D12DynamicRHI::RHITick(float DeltaTime)
{
	check(IsInGameThread());

	// Check if any swap chains have been invalidated.
	auto& Viewports = GetAdapter().GetViewports();
	for (int32 ViewportIndex = 0; ViewportIndex < Viewports.Num(); ViewportIndex++)
	{
		Viewports[ViewportIndex]->ConditionalResetSwapChain(false);
	}
}

/*=============================================================================
 *	Viewport functions.
 *=============================================================================*/

void FD3D12DynamicRHI::RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* ViewportRHI, const FRHIPresentArgs& PresentArgs)
{
	//
	// Make sure all prior graphics and async compute work has been submitted. This is necessary because D3D12 RHI handles the present from inside the below lambda.
	// Since the present is not yet pipelined via payloads etc, we must ensure prior work has been handed to RHISubmitCommandLists before we translate the lambda.
	//
	// In future, Present() itself should be an enqueued command, and platform RHIs should never implicitly send commands to the GPU queues during RHI translation.
	// All GPU work should be submitted via RHISubmitCommandLists.
	//
	// This is also why there is a task dependency between RHI translate and RHI submission tasks in RHICommandList.cpp.
	// See GRHIGlobals.SupportsConcurrentTranslateAndSubmit
	//
	RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

	{
		FRHICommandListScopedPipeline PipelineScope(RHICmdList, ERHIPipeline::Graphics);
		RHICmdList.EnqueueLambda(TEXT("RHIEndDrawingViewport"), [this, ViewportRHI, PresentArgs](FRHICommandListImmediate& ExecutingCmdList)
		{
			FD3D12CommandContextBase& Context = FD3D12CommandContextBase::Get(ExecutingCmdList);

			// Not all platforms have control on the Present Counter, correlate the information FrameCounter -> PresentCounter before Presenting the frame
			RHIFrameInfoSetPresentCounter(PresentArgs.FrameCounter, GRHIPresentCounter);

			FD3D12Viewport* Viewport = FD3D12DynamicRHI::ResourceCast(ViewportRHI);
			SCOPE_CYCLE_COUNTER(STAT_D3D12PresentTime);

		#if !UE_BUILD_SHIPPING
			if (RHIConsoleVariables::LogViewportEvents)
			{
				const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
				UE_LOGF(LogD3D12RHI, Log, "Thread %ls: RHIEndDrawingViewport (Viewport %#016llx: BackBuffer %#016llx: CmdList: %016llx)",
					ThreadName.GetCharArray().GetData(),
					Viewport,
					Viewport->GetBackBuffer_RHIThread(),
					Context.GetSingleDeviceContext(0)->BaseCommandList().GetNoRefCount()
				);
			}
		#endif

			const bool bNativelyPresented = Viewport->EndRendering(Context, PresentArgs);

			if ((bNativelyPresented || GUseSubmissionThreadPresent) || (PresentArgs.bPresent == false))
			{
				FRenderThreadIdleScope Scope(ERenderThreadIdleTypes::WaitingForAllOtherSleep);
				static const auto CFinishFrameVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FinishCurrentFrame"));

				int32 FinishCurrentFrame = CFinishFrameVar->GetValueOnRenderThread();

				if (FinishCurrentFrame == 0)
				{
					// Wait for the GPU to finish rendering the previous frame before finishing this frame.
					Viewport->WaitForFrameEventCompletion();
					Viewport->IssueFrameEvent();
				}
				else if (FinishCurrentFrame > 0)
				{
					// Finish current frame immediately to reduce latency
					Viewport->IssueFrameEvent();
					Viewport->WaitForFrameEventCompletion();
				}
				else if (FinishCurrentFrame < 0)
				{
					//No frame wait
				}
			}
		});
	}

#if !UE_BUILD_SHIPPING
	if (RHIConsoleVariables::LogViewportEvents)
	{
		const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
		UE_LOGF(LogD3D12RHI, Log, "Thread %ls: RHIEndDrawingViewport", ThreadName.GetCharArray().GetData());
	}
#endif

	// Don't need to do anything on the back because dummy back buffer texture is used to make sure the correct back
	// buffer index is always used on RHI thread

	// But advance the expected present GPU index so the next call to RHIGetViewportNextPresentGPUIndex returns the expected GPU index for the next present.
	// Warning: when present fails or is not called on the RHIThread then this might not be in sync but RHI thread will fix up the correct state
	//          Present doesn't happen so shouldn't matter that the index was wrong then
	ResourceCast(ViewportRHI)->AdvanceExpectedBackBufferIndex_RenderThread(PresentArgs.bPresent);
}

uint32 FD3D12DynamicRHI::RHIGetViewportNextPresentGPUIndex(FRHIViewport* ViewportRHI)
{
	check(IsInRenderingThread());
	
#if WITH_MGPU	
	const FD3D12Viewport* Viewport = FD3D12DynamicRHI::ResourceCast(ViewportRHI);
	if (Viewport)
	{
		return Viewport->GetNextPresentGPUIndex();
	}
#endif // WITH_MGPU

	return 0;
}

FTextureRHIRef FD3D12DynamicRHI::RHIGetViewportBackBuffer(FRHIViewport* ViewportRHI)
{
	check(IsInRenderingThread());

	const FD3D12Viewport* const Viewport = FD3D12DynamicRHI::ResourceCast(ViewportRHI);

	FRHITexture* SelectedBackBuffer = Viewport->GetBackBuffer_RenderThread();
#if !UE_BUILD_SHIPPING
	if (RHIConsoleVariables::LogViewportEvents)
	{
		const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
		UE_LOGF(LogD3D12RHI, Log, "Thread %ls: RHIGetViewportBackBuffer (Viewport %#016llx: BackBuffer %#016llx)", ThreadName.GetCharArray().GetData(), Viewport, SelectedBackBuffer);
	}
#endif

	return SelectedBackBuffer;
}

void FD3D12Viewport::SyncToSubmissionThread(FD3D12CommandContextBase& ContextBase)
{
	// Sync with the submission thread 
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		FD3D12CommandContext& Context = *ContextBase.GetSingleDeviceContext(GPUIndex);

		Context.FlushResourceBarriers();

		// Currently, the swap chain Present() is called directly by the RHI thread.
		// We need to submit the above commands and wait for the submission thread to process everything before we can continue.
		Context.FlushCommands(ED3D12FlushFlags::WaitForSubmission);
	}
}

bool FD3D12Viewport::CheckSwapChain()
{
#if PLATFORM_WINDOWS
	// We can't call Present if !bIsValid, as it waits a window message to be processed, but the main thread may not be pumping the message handler.
	if (bIsValid && SwapChain1.IsValid())
	{
		// Check if the viewport's swap chain has been invalidated by DXGI.
		BOOL bSwapChainFullscreenState;
		TRefCountPtr<IDXGIOutput> SwapChainOutput;
		SwapChain1->GetFullscreenState(&bSwapChainFullscreenState, SwapChainOutput.GetInitReference());
		// Can't compare BOOL with bool...
		if ((!!bSwapChainFullscreenState) != bIsFullscreen)
		{
			bFullscreenLost = true;
			bIsValid = false;
		}
	}

	if (!bIsValid)
	{
#if WITH_MGPU
		// Present failed so current expected GPU index will not match anymore, so patch up expected back buffer index
		// Warning: Present is skipped for this frame but could cause a black screen for the next frame as well
		FScopeLock Lock(&ExpectedBackBufferIndexLock);
		SetBackBufferIndex_RenderThread((ExpectedBackBufferIndex_RenderThread == 0) ? NumBackBuffers - 1 : ExpectedBackBufferIndex_RenderThread - 1);
#endif // WITH_MGPU
		return false;
	}
#endif

	return true;
}