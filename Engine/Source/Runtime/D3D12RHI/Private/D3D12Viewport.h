// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Viewport.h: D3D viewport RHI definitions.
=============================================================================*/

#pragma once

#include "D3D12RHICommon.h"
#include "D3D12Texture.h"
#include "D3D12Submission.h"
#include "HAL/Runnable.h"
#include "MultiGPU.h"
#include "RHIResources.h"
#include "Templates/RefCounting.h"
#include "DXGIUtilities.h"

class FD3D12Texture;
class FD3D12UnorderedAccessView_RHI;

class FD3D12SyncPoint;
using FD3D12SyncPointRef = TRefCountPtr<FD3D12SyncPoint>;

class FD3D12Viewport : public FRHIViewport, public FD3D12AdapterChild
{
public:

	// Lock viewport windows association and back buffer destruction because of possible crash inside DXGI factory during a call to MakeWindowAssociation
	// Backbuffer release will wait on the call to MakeWindowAssociation while this will fail internally with 'The requested operation is not implemented.' in KernelBase.dll
	// Reported & known problem in DXGI and will be fixed with future release but DXGI is not part of the Agility SDK so a code side fix is needed for now.
	static FCriticalSection DXGIBackBufferLock;

	FD3D12Viewport(class FD3D12Adapter* InParent, HWND InWindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InPixelFormat, uint32 InNumBackBuffers);

	void Init();

	~FD3D12Viewport();

	void Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat PreferredPixelFormat);

	/**
	 * If the swap chain has been invalidated by DXGI, resets the swap chain to the expected state; otherwise, does nothing.
	 * Called once/frame by the game thread on all viewports.
	 * @param bIgnoreFocus - Whether the reset should happen regardless of whether the window is focused.
	 */
	void ConditionalResetSwapChain(bool bIgnoreFocus);

	/** Schedule Present from the RHI Thread to the Submission Thread
	* Returns true if Present was correctly setup.
	*/
	bool SchedulePresent(class FD3D12CommandContextBase* BaseContext, const FRHIPresentArgs& InPresentArgs);
	
	/** Presents the swap chain on the Submission Thread
	*/
	void PresentOnSubmissionThread(FD3D12PresentArgs& PresentArgs);

	/** Wait for the last Present to be called from the Submission Thread
	*/
	void WaitForLastPresent() const;

	/** Signal that the Presents on the device has been completed and the backbuffer index has been update
	*/
	FGraphEventRef			PresentEvent;

	/** Presents the swap chain.
	 * Returns true if Present was done by Engine.
	 */
	bool EndRendering(class FD3D12CommandContextBase& Context, const FRHIPresentArgs& InPresentArgs);

	// Accessors.
	FIntPoint GetSizeXY() const
	{
		return FIntPoint(SizeX, SizeY);
	}

	FD3D12Texture* GetBackBuffer_RenderThread() const
	{
		check(IsInRenderingThread());
#if D3D12RHI_USE_DUMMY_BACKBUFFER
		return DummyBackBuffer_RenderThread;
#else
		checkSlow(CurrentBackBuffer_RenderThread);
		return CurrentBackBuffer_RenderThread->Texture;
#endif
	}

	FD3D12Texture* GetBackBuffer_RHIThread(bool bWaitForLastPresent = true) const;

	FD3D12Texture* GetSDRBackBuffer_RHIThread(bool bWaitForLastPresent = true) const
	{
		if (bWaitForLastPresent)
		{
			WaitForLastPresent();
		}

#if D3D12RHI_USE_SDR_BACKBUFFER
		if (PixelFormat != SDRPixelFormat)
		{
			return CurrentBackBuffer_RHIThread->TextureSDR;
		}
#endif

		return CurrentBackBuffer_RHIThread->Texture;
	}

#if D3D12RHI_USE_SDR_BACKBUFFER
	FRHITexture* GetOptionalSDRBackBuffer(FRHITexture* BackBufferTex) const override
	{
		for (const FBackBufferData& CurBackBuffer : BackBuffers)
		{
			if ( (FRHITexture*)CurBackBuffer.Texture.GetReference() == BackBufferTex)
			{
				return (FRHITexture*)CurBackBuffer.TextureSDR.GetReference();
			}
		}

		return nullptr;
	}
#endif


#if WITH_MGPU
	uint32 GetNextPresentGPUIndex() const
	{
		FScopeLock Lock(&ExpectedBackBufferIndexLock);
		return BackBuffers[ExpectedBackBufferIndex_RenderThread].GPUIndex;
	}
#endif // WITH_MGPU

	virtual void WaitForFrameEventCompletion() override;
	virtual void IssueFrameEvent() override;

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	virtual void* GetNativeSwapChain() const override;
#endif // #if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN

	virtual void* GetNativeBackBufferTexture() const override;
	virtual void* GetNativeBackBufferRT() const override;

	virtual void SetCustomPresent(FRHICustomPresent* InCustomPresent) override;
	virtual FRHICustomPresent* GetCustomPresent() const override;

	virtual void* GetNativeWindow(void** AddParam = nullptr) const override { return (void*)WindowHandle; }

	uint32 GetNumBackBuffers() const { return NumBackBuffers; }

	inline const bool IsFullscreen() const { return bIsFullscreen; }

	/** Query the swap chain's current connected output for HDR support. */
	bool CurrentOutputSupportsHDR() const;

	/** Advance and get the next present GPU index */
	void AdvanceExpectedBackBufferIndex_RenderThread(bool bPresent);

	void OnResumeRendering();
	void OnSuspendRendering();

	bool PresentedLastFrame(){ return bLastFramePresented; }

private:
	bool IsPresentAllowed();
	bool CheckSwapChain();
	void SyncToSubmissionThread(FD3D12CommandContextBase& ContextBase);

	FString GetStateString();

#if D3D12RHI_USE_DUMMY_BACKBUFFER
	FD3D12Texture* CreateDummyBackBufferTextures(FD3D12Adapter* InAdapter, EPixelFormat InPixelFormat, uint32 InSizeX, uint32 InSizeY);
#endif

	/**
	 * Presents the swap chain checking the return result.
	 */
	void PresentChecked(int32 SyncInterval);

	/**
	 * Presents the backbuffer to the viewport window.
	 * Returns the HRESULT for the call.
	 */
	HRESULT PresentInternal(int32 SyncInterval);

	void ResizeInternal();
	void FinalDestroyInternal();
	void ClearPresentQueue();

	// Determine how deep the swapchain should be
	void InitializeBackBufferArrays();

	/** See if HDR can be enabled or not based on RHI support and current engine settings. */
	bool CheckHDRSupport();

	/** Enable HDR meta data transmission and set the necessary color space. */
	void EnableHDR();

	/** Disable HDR meta data transmission and set the necessary color space. */
	void ShutdownHDR();

#if D3D12RHI_USE_DXGI_COLOR_SPACE
	/** Ensure the correct color space is set on the swap chain */
	void EnsureColorSpace(DXGI_COLOR_SPACE_TYPE NewColorSpace);

	/** Ensure the correct color space is set on the swap chain */
	void EnsureColorSpace(EDisplayColorGamut DisplayGamut, EDisplayOutputFormat OutputDevice);
#endif

	void SetBackBufferIndex_RHIThread(uint32 Index)
	{
		CurrentBackBufferIndex_RHIThread = Index % NumBackBuffers;
		CurrentBackBuffer_RHIThread = &BackBuffers[CurrentBackBufferIndex_RHIThread];
	}

	void SetBackBufferIndex_RenderThread(uint32 Index)
	{
		ExpectedBackBufferIndex_RenderThread = Index % NumBackBuffers;
		CurrentBackBuffer_RenderThread = &BackBuffers[ExpectedBackBufferIndex_RenderThread];
	}

private:
	const HWND WindowHandle;
	uint32 SizeX = 0;
	uint32 SizeY = 0;
	EPixelFormat PixelFormat;
#if D3D12RHI_USE_SDR_BACKBUFFER
	static constexpr EPixelFormat SDRPixelFormat = PF_B8G8R8A8;
#endif

	bool bIsFullscreen = false;
	bool bFullscreenLost = false;
	bool bIsValid = true;
	bool bAllowTearing = true;
	bool bNeedSwapChain = false;
	bool bLastFramePresented = true;
	bool bResizeInProgress = false;

	uint32 CheckedPresentFailureCounter = 0;

#if D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN
	TRefCountPtr<IDXGISwapChain1> SwapChain1;
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 2
	TRefCountPtr<IDXGISwapChain2> SwapChain2;
#endif
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 3
	TRefCountPtr<IDXGISwapChain3> SwapChain3;
#endif
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 4
	TRefCountPtr<IDXGISwapChain4> SwapChain4;
#endif

#if D3D12RHI_USE_DXGI_COLOR_SPACE
	DXGI_COLOR_SPACE_TYPE ColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709;
#endif
#endif // D3D12_VIEWPORT_EXPOSES_SWAP_CHAIN

	struct FBackBufferData
	{
		TRefCountPtr<FD3D12Texture> Texture;
#if D3D12RHI_USE_SDR_BACKBUFFER
		// When HDR is enabled, SDR backbuffers may be required on some architectures for game DVR or broadcasting
		TRefCountPtr<FD3D12Texture> TextureSDR;
#endif
#if WITH_MGPU
		uint32                      GPUIndex = 0;
#endif
	};

	uint32 NumBackBuffers = 0;
	TArray<FBackBufferData, TInlineAllocator<D3D12RHI_DEFAULT_NUM_BACKBUFFER>> BackBuffers;

#if D3D12RHI_USE_DUMMY_BACKBUFFER
	// Dummy back buffer texture which always references the current back buffer on the RHI thread
	TRefCountPtr<FD3D12Texture> DummyBackBuffer_RenderThread;
#endif

	FBackBufferData* CurrentBackBuffer_RHIThread = nullptr;
	FBackBufferData* CurrentBackBuffer_RenderThread = nullptr;

	uint32 CurrentBackBufferIndex_RHIThread = 0;
	uint32 ExpectedBackBufferIndex_RenderThread = 0;
	EDisplayColorGamut DisplayColorGamut = EDisplayColorGamut::sRGB_D65;
	EDisplayOutputFormat DisplayOutputFormat = EDisplayOutputFormat::SDR_sRGB;

	/** A fence value used to track the GPU's progress. */
	TArray<FD3D12SyncPointRef> FrameSyncPoints;

	FCustomPresentRHIRef CustomPresent;

#if WITH_MGPU
	// Where INDEX_NONE cycles through the GPU, otherwise the GPU index.
	int32 BackbufferMultiGPUBinding = 0;

	// Can very rarely be modified on the RHI thread as well if present is skipped
	mutable FCriticalSection ExpectedBackBufferIndexLock;
#endif

};

template<>
struct TD3D12ResourceTraits<FRHIViewport>
{
	typedef FD3D12Viewport TConcreteType;
};
