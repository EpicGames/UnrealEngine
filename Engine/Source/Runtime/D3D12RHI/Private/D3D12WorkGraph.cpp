// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12WorkGraph.h"

#include "Async/ParallelFor.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "D3D12ExplicitDescriptorCache.h"
#include "D3D12RHIPrivate.h"
#include "D3D12ResourceCollection.h"
#include "D3D12Shader.h"
#include "D3D12TextureReference.h"
#include "PipelineStateCache.h"
#include "ShaderBundles.h"
#include "RHIUniformBufferUtilities.h"

static bool GShaderBundleSkipDispatch = false;
static FAutoConsoleVariableRef CVarShaderBundleSkipDispatch(
	TEXT("wg.ShaderBundle.SkipDispatch"),
	GShaderBundleSkipDispatch,
	TEXT("Whether to dispatch the built shader bundle pipeline (for debugging)"),
	ECVF_RenderThreadSafe
);

FD3D12WorkGraphPipelineState::FD3D12WorkGraphPipelineState(FD3D12Device* Device, const FWorkGraphPipelineStateInitializer& Initializer)
	: Device(Device)
{
#if D3D12_RHI_WORKGRAPHS
	
	RootArgStrideInBytes = 0;

	// Use global root signature from specified node, or if non-specified use a fixed root signature.
	ID3D12RootSignature* GlobalRootSignature = nullptr;
	const int32 RootShaderIndex = Initializer.GetRootShaderIndex();
	if (RootShaderIndex != -1)
	{
		if (Initializer.GetShaderTable().IsValidIndex(RootShaderIndex))
		{
			FRHIWorkGraphShader* Shader = Initializer.GetShaderTable()[RootShaderIndex];
			if (Shader != nullptr && Shader->GetFrequency() == SF_WorkGraphRoot)
			{
				GlobalRootSignature = FD3D12DynamicRHI::ResourceCast(Shader)->RootSignature->GetRootSignature();
			}
		}
	}
	if (GlobalRootSignature == nullptr)
	{
		FD3D12Adapter* Adapter = Device->GetParentAdapter();
		FRHIShaderBindingLayout ShaderBindingLayout; // todo pass ShaderBindingLayout down
		GlobalRootSignature = Adapter->GetGlobalWorkGraphRootSignature(ShaderBindingLayout)->GetRootSignature();
	}
	RootSignature = GlobalRootSignature;

	CD3DX12_STATE_OBJECT_DESC StateObjectDesc(D3D12_STATE_OBJECT_TYPE_EXECUTABLE);
	
	CD3DX12_GLOBAL_ROOT_SIGNATURE_SUBOBJECT* GlobalRootSignatureSubobject = StateObjectDesc.CreateSubobject<CD3DX12_GLOBAL_ROOT_SIGNATURE_SUBOBJECT>();
	GlobalRootSignatureSubobject->SetRootSignature(RootSignature);

	CD3DX12_WORK_GRAPH_SUBOBJECT* WorkGraphSubobject = StateObjectDesc.CreateSubobject<CD3DX12_WORK_GRAPH_SUBOBJECT>();
	TCHAR const* ProgramName = Initializer.GetProgramName().IsEmpty() ? TEXT("WorkGraphProgram") : *Initializer.GetProgramName();
	WorkGraphSubobject->SetProgramName(ProgramName);

	D3D12_NODE_ID EntryPoint;
	{
		FD3D12WorkGraphShader* EntryShader = FD3D12DynamicRHI::ResourceCast(Initializer.GetShaderTable()[0]);
		EntryPoint.Name = *EntryShader->EntryPoint;
		EntryPoint.ArrayIndex = 0;
	}
	WorkGraphSubobject->AddEntrypoint(EntryPoint);

	// Compute Shader Table
	const int32 ShaderTableNum = Initializer.GetShaderTable().Num();
	for (int32 Index = 0; Index < ShaderTableNum; ++Index) 
	{
		FD3D12WorkGraphShader* NodeShader = FD3D12DynamicRHI::ResourceCast(Initializer.GetShaderTable()[Index]);
		static const FString EmptyString;
		FString const& ExportName = NodeShader == nullptr ? EmptyString : NodeShader->EntryPoint;
		const int32 NodeNameIndex = Initializer.GetNameTable().IndexOfByPredicate([ExportName](FWorkGraphPipelineStateInitializer::FNameMap const& NameMap)
		{
			return NameMap.ExportName == ExportName;
		});

		if (NodeNameIndex != INDEX_NONE)
		{
			FString const& NodeName = Initializer.GetNameTable()[NodeNameIndex].NodeName;
			const uint32 NodeArrayIndex = (NodeCountPerName.FindOrAdd(NodeName))++;

			if (NodeShader != nullptr)
			{
				const FString NodePathName = FString::Printf(TEXT("%s_%d"), *NodeName, NodeArrayIndex);

				CD3DX12_DXIL_LIBRARY_SUBOBJECT* Lib = StateObjectDesc.CreateSubobject<CD3DX12_DXIL_LIBRARY_SUBOBJECT>();
				D3D12_SHADER_BYTECODE LibCode = NodeShader->GetShaderBytecode();

				Lib->SetDXILLibrary(&LibCode);
				Lib->DefineExport(*NodePathName, *NodeShader->EntryPoint);

				CD3DX12_LOCAL_ROOT_SIGNATURE_SUBOBJECT* LocalRootSignature = StateObjectDesc.CreateSubobject<CD3DX12_LOCAL_ROOT_SIGNATURE_SUBOBJECT>();
				LocalRootSignature->SetRootSignature(NodeShader->RootSignature->GetRootSignature());
				CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT* AssociationSubobject = StateObjectDesc.CreateSubobject<CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT>();
				AssociationSubobject->SetSubobjectToAssociate(*LocalRootSignature);
				AssociationSubobject->AddExport(*NodePathName);

				CD3DX12_COMMON_COMPUTE_NODE_OVERRIDES* Override = WorkGraphSubobject->CreateCommonComputeNodeOverrides(*NodePathName);
				Override->NewName(D3D12_NODE_ID{ *NodeName, NodeArrayIndex });
				
				const int32 LocalRootArgumentsTableIndex = RootArgOffsets.Num();
				RootArgOffsets.Add(LocalRootArgumentsTableIndex);
				Override->LocalRootArgumentsTableIndex(LocalRootArgumentsTableIndex);
		
				RootArgStrideInBytes = FMath::Max(RootArgStrideInBytes, NodeShader->RootSignature->GetTotalRootSignatureSizeInBytes());
			}
		}
	}
	
#if D3D12_RHI_WORKGRAPHS_GRAPHICS

	// Graphics Shader Table
	const int32 PSOTableNum = Initializer.GetGraphicsPSOTable().Num();
	if (PSOTableNum > 0)
	{
		CD3DX12_STATE_OBJECT_CONFIG_SUBOBJECT* ConfigSubobject = StateObjectDesc.CreateSubobject<CD3DX12_STATE_OBJECT_CONFIG_SUBOBJECT>();
		ConfigSubobject->SetFlags(D3D12_STATE_OBJECT_FLAG_WORK_GRAPHS_USE_GRAPHICS_STATE_FOR_GLOBAL_ROOT_SIGNATURE);
	}

	for (int32 Index = 0; Index < PSOTableNum; ++Index)
	{
		FGraphicsPipelineStateInitializer const* NodePSO = Initializer.GetGraphicsPSOTable()[Index];
		FRHIWorkGraphShader* MeshShader = NodePSO == nullptr ? nullptr : NodePSO->BoundShaderState.GetWorkGraphShader();
		FRHIPixelShader* PixelShader = NodePSO == nullptr ? nullptr : NodePSO->BoundShaderState.GetPixelShader();

		static const FString EmptyString;
		FString const& ExportName = MeshShader == nullptr ? EmptyString : FD3D12DynamicRHI::ResourceCast(MeshShader)->EntryPoint;
		const int32 NodeNameIndex = Initializer.GetNameTable().IndexOfByPredicate([ExportName](FWorkGraphPipelineStateInitializer::FNameMap const& NameMap)
		{
			return NameMap.ExportName == ExportName;
		});

		if (NodeNameIndex == INDEX_NONE)
		{
			continue;
		}

		FString const& NodeName = Initializer.GetNameTable()[NodeNameIndex].NodeName;
		const uint32 NodeArrayIndex = (NodeCountPerName.FindOrAdd(NodeName))++;

		if (MeshShader == nullptr || PixelShader == nullptr)
		{
			continue;
		}
		
		const FString NodePathName = FString::Printf(TEXT("%s_%d"), *NodeName, NodeArrayIndex);
		FString MeshShaderName = FString::Printf(TEXT("MeshShader_%d"), NodeArrayIndex);
		FString PixelShaderName = FString::Printf(TEXT("PixelShader_%d"), NodeArrayIndex);
			
		FD3D12RootSignature const* LocalRootSignature = Device->GetParentAdapter()->GetWorkGraphGraphicsRootSignature(NodePSO->BoundShaderState);
		CD3DX12_LOCAL_ROOT_SIGNATURE_SUBOBJECT* LocalRootSignatureSubobject = StateObjectDesc.CreateSubobject<CD3DX12_LOCAL_ROOT_SIGNATURE_SUBOBJECT>();
		LocalRootSignatureSubobject->SetRootSignature(LocalRootSignature->GetRootSignature());
		RootArgStrideInBytes = FMath::Max(RootArgStrideInBytes, LocalRootSignature->GetTotalRootSignatureSizeInBytes());

		{
			FD3D12WorkGraphShader* D3D12Shader = FD3D12DynamicRHI::ResourceCast(MeshShader);

			CD3DX12_DXIL_LIBRARY_SUBOBJECT* LibrarySubobject = StateObjectDesc.CreateSubobject<CD3DX12_DXIL_LIBRARY_SUBOBJECT>();
			CD3DX12_SHADER_BYTECODE ByteCode(D3D12Shader->Code.GetData(), D3D12Shader->Code.Num());
			LibrarySubobject->SetDXILLibrary(&ByteCode);
			LibrarySubobject->DefineExport(*MeshShaderName, *D3D12Shader->EntryPoint);

			CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT* ExportAssociationSubobject = StateObjectDesc.CreateSubobject<CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT>();
			ExportAssociationSubobject->SetSubobjectToAssociate(*LocalRootSignatureSubobject);
			ExportAssociationSubobject->AddExport(*MeshShaderName);
		}
		{
			FD3D12PixelShader* D3D12Shader = FD3D12DynamicRHI::ResourceCast(PixelShader);

			CD3DX12_DXIL_LIBRARY_SUBOBJECT* LibrarySubobject = StateObjectDesc.CreateSubobject<CD3DX12_DXIL_LIBRARY_SUBOBJECT>();
			CD3DX12_SHADER_BYTECODE ByteCode(D3D12Shader->Code.GetData(), D3D12Shader->Code.Num());
			LibrarySubobject->SetDXILLibrary(&ByteCode);
			LibrarySubobject->DefineExport(*PixelShaderName, *D3D12Shader->EntryPoint);

			CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT* ExportAssociationSubobject = StateObjectDesc.CreateSubobject<CD3DX12_SUBOBJECT_TO_EXPORTS_ASSOCIATION_SUBOBJECT>();
			ExportAssociationSubobject->SetSubobjectToAssociate(*LocalRootSignatureSubobject);
			ExportAssociationSubobject->AddExport(*PixelShaderName);
		}

		TArray<CD3DX12_STATE_OBJECT_DESC::SUBOBJECT_HELPER_BASE*, TInlineAllocator<8>> StateSubobjects;

		CD3DX12_PRIMITIVE_TOPOLOGY_SUBOBJECT* PrimitiveTopologyState = StateObjectDesc.CreateSubobject<CD3DX12_PRIMITIVE_TOPOLOGY_SUBOBJECT>();
		const EPrimitiveType PrimitiveType = NodePSO->PrimitiveType;
		PrimitiveTopologyState->SetPrimitiveTopologyType(D3D12PrimitiveTypeToTopologyType(TranslatePrimitiveType(PrimitiveType)));

		FD3D12RasterizerState* D3D12RasterizerState = FD3D12DynamicRHI::ResourceCast(NodePSO->RasterizerState);
		CD3DX12_RASTERIZER_SUBOBJECT* RasterizerSubobject = StateObjectDesc.CreateSubobject<CD3DX12_RASTERIZER_SUBOBJECT>();
		RasterizerSubobject->SetFrontCounterClockwise(D3D12RasterizerState->Desc.FrontCounterClockwise);
		RasterizerSubobject->SetFillMode(D3D12RasterizerState->Desc.FillMode);
		RasterizerSubobject->SetCullMode(D3D12RasterizerState->Desc.CullMode);
		StateSubobjects.Add(RasterizerSubobject);

		FD3D12DepthStencilState* D3D12DepthStencilState = FD3D12DynamicRHI::ResourceCast(NodePSO->DepthStencilState);
		CD3DX12_DEPTH_STENCIL_SUBOBJECT* DepthStencilSubobject = StateObjectDesc.CreateSubobject<CD3DX12_DEPTH_STENCIL_SUBOBJECT>();
		DepthStencilSubobject->SetDepthEnable(D3D12DepthStencilState->Desc.DepthEnable);
		DepthStencilSubobject->SetDepthFunc(D3D12DepthStencilState->Desc.DepthFunc);
		DepthStencilSubobject->SetDepthWriteMask(D3D12DepthStencilState->Desc.DepthWriteMask);
		DepthStencilSubobject->SetStencilEnable(D3D12DepthStencilState->Desc.StencilEnable);
		DepthStencilSubobject->SetStencilReadMask(D3D12DepthStencilState->Desc.StencilReadMask);
		DepthStencilSubobject->SetStencilWriteMask(D3D12DepthStencilState->Desc.StencilWriteMask);
		StateSubobjects.Add(DepthStencilSubobject);

		if (NodePSO->DepthStencilTargetFormat != PF_Unknown)
		{
			CD3DX12_DEPTH_STENCIL_FORMAT_SUBOBJECT* DepthStencilFormatSubobject = StateObjectDesc.CreateSubobject<CD3DX12_DEPTH_STENCIL_FORMAT_SUBOBJECT>();
			DepthStencilFormatSubobject->SetDepthStencilFormat((DXGI_FORMAT)GPixelFormats[NodePSO->DepthStencilTargetFormat].PlatformFormat);
			StateSubobjects.Add(DepthStencilFormatSubobject);
		}

		CD3DX12_RENDER_TARGET_FORMATS_SUBOBJECT* RenderTargetFormatSubobject = StateObjectDesc.CreateSubobject<CD3DX12_RENDER_TARGET_FORMATS_SUBOBJECT>();
		const int32 NumRenderTargets = NodePSO->ComputeNumValidRenderTargets();
		RenderTargetFormatSubobject->SetNumRenderTargets(NumRenderTargets);
		if (NumRenderTargets > 0)
		{
			for (int32 RenderTargetIndex = 0; RenderTargetIndex < NumRenderTargets; ++RenderTargetIndex)
			{
				RenderTargetFormatSubobject->SetRenderTargetFormat(RenderTargetIndex, (DXGI_FORMAT)GPixelFormats[NodePSO->RenderTargetFormats[RenderTargetIndex]].PlatformFormat);
			}
		}
		StateSubobjects.Add(RenderTargetFormatSubobject);

		CD3DX12_GENERIC_PROGRAM_SUBOBJECT* ProgramSubobject = StateObjectDesc.CreateSubobject<CD3DX12_GENERIC_PROGRAM_SUBOBJECT>();
		ProgramSubobject->SetProgramName(*NodePathName);
		ProgramSubobject->AddExport(*MeshShaderName);
		ProgramSubobject->AddExport(*PixelShaderName);
		for (CD3DX12_STATE_OBJECT_DESC::SUBOBJECT_HELPER_BASE* StateSubobject : StateSubobjects)
		{
			ProgramSubobject->AddSubobject(*StateSubobject);
		}
		ProgramSubobject->Finalize();

		CD3DX12_MESH_LAUNCH_NODE_OVERRIDES* NodeOverrides = WorkGraphSubobject->CreateMeshLaunchNodeOverrides(*NodePathName);
		NodeOverrides->NewName(D3D12_NODE_ID{ *NodeName, NodeArrayIndex });
		NodeOverrides->MaxInputRecordsPerGraphEntryRecord(1, false);

		const int32 LocalRootArgumentsTableIndex = RootArgOffsets.Num();
		RootArgOffsets.Add(LocalRootArgumentsTableIndex);
		NodeOverrides->LocalRootArgumentsTableIndex(LocalRootArgumentsTableIndex);
	}
	
#endif // D3D12_RHI_WORKGRAPHS_GRAPHICS

	RootArgStrideInBytes = ((RootArgStrideInBytes + 15) & ~15);
	MaxRootArgOffset = RootArgOffsets.Last();

#if UE_D3D12_SDK_VERSION < 616
	WorkGraphSubobject->Finalize();
#endif

	ID3D12Device9* Device9 = (ID3D12Device9*)Device->GetDevice();
	HRESULT HResult = Device9->CreateStateObject(StateObjectDesc, IID_PPV_ARGS(StateObject.GetInitReference()));
	checkf(SUCCEEDED(HResult), TEXT("Failed to create work graph state object. Result=%08x"), uint32(HResult));

	TRefCountPtr<ID3D12StateObjectProperties1> PipelineProperties;
	HResult = StateObject->QueryInterface(IID_PPV_ARGS(PipelineProperties.GetInitReference()));
	checkf(SUCCEEDED(HResult), TEXT("Failed to query pipeline properties from the work graph pipeline state object. Result=%08x"), uint32(HResult));

	ProgramIdentifier = PipelineProperties->GetProgramIdentifier(ProgramName);

#if D3D12_RHI_WORKGRAPHS_GRAPHICS
	TRefCountPtr<ID3D12WorkGraphProperties1> WorkGraphProperties;
#else
	TRefCountPtr<ID3D12WorkGraphProperties> WorkGraphProperties;
#endif

	HResult = StateObject->QueryInterface(IID_PPV_ARGS(WorkGraphProperties.GetInitReference()));
	checkf(SUCCEEDED(HResult), TEXT("Failed to query work graph properties from the work graph pipeline state object. Result=%08x"), uint32(HResult));

	UINT WorkGraphIndex = WorkGraphProperties->GetWorkGraphIndex(ProgramName);
#if D3D12_RHI_WORKGRAPHS_GRAPHICS
	WorkGraphProperties->SetMaximumInputRecords(WorkGraphIndex, 1, 1);
#endif
	D3D12_WORK_GRAPH_MEMORY_REQUIREMENTS MemoryRequirements = {};
	WorkGraphProperties->GetWorkGraphMemoryRequirements(WorkGraphIndex, &MemoryRequirements);

	ID3D12Resource* BackingMemoryBufferResource = nullptr;
	{
		CD3DX12_RESOURCE_DESC BufferDesc = CD3DX12_RESOURCE_DESC::Buffer(MemoryRequirements.MaxSizeInBytes, D3D12_RESOURCE_FLAG_ALLOW_UNORDERED_ACCESS, 65536ull);
		CD3DX12_HEAP_PROPERTIES HeapProperties(D3D12_HEAP_TYPE_DEFAULT);

		HResult = Device->GetDevice()->CreateCommittedResource(	//TODO: don't use raw device?
			&HeapProperties,
			D3D12_HEAP_FLAG_NONE,
			&BufferDesc,
			D3D12_RESOURCE_STATE_UNORDERED_ACCESS,
			NULL,
			__uuidof(ID3D12Resource),
			(void**)&BackingMemoryBufferResource);
		checkf(SUCCEEDED(HResult), TEXT("Failed to allocate backing memory for work graph. Result=%08x"), uint32(HResult));
	}
	BackingMemoryAddressRange.StartAddress = BackingMemoryBufferResource->GetGPUVirtualAddress();
	BackingMemoryAddressRange.SizeInBytes = MemoryRequirements.MaxSizeInBytes;

#if NV_AFTERMATH
	if (UE::RHICore::Nvidia::Aftermath::IsShaderRegistrationEnabled())
	{
		// Copy shader table for late association
		for (FRHIWorkGraphShader* Shader : Initializer.GetShaderTable())
		{
			Shaders.Add(TRefCountPtr<FRHIShader>(Shader));
		}
	}
#endif // NV_AFTERMATH
#endif // D3D12_RHI_WORKGRAPHS
}

FWorkGraphPipelineStateRHIRef FD3D12DynamicRHI::RHICreateWorkGraphPipelineState(const FWorkGraphPipelineStateInitializer& Initializer)
{
	FD3D12Device* Device = GetAdapter().GetDevice(0); // All pipelines are created on the first node, as they may be used on any other linked GPU.
	return new FD3D12WorkGraphPipelineState(Device, Initializer);
}

#if D3D12_RHI_WORKGRAPHS

/** Struct to collect transitions for all shader bundle dispatches. */
struct FShaderBundleBinderOps
{
	Experimental::TSherwoodSet<FD3D12View*> TransitionViewSet;
	Experimental::TSherwoodSet<FD3D12View*> TransitionClearSet;

	TArray<FD3D12ShaderResourceView*>  TransitionSRVs;
	TArray<FD3D12UnorderedAccessView*> TransitionUAVs;
	TArray<FD3D12UnorderedAccessView*> ClearUAVs;

	inline void AddResourceTransition(FD3D12ShaderResourceView* SRV)
	{
		if (SRV->GetResource()->RequiresResourceStateTracking())
		{
			bool bAlreadyInSet = false;
			TransitionViewSet.Add(SRV, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				TransitionSRVs.Add(SRV);
			}
		}
	}

	inline void AddResourceTransition(FD3D12UnorderedAccessView* UAV)
	{
		if (UAV->GetResource()->RequiresResourceStateTracking())
		{
			bool bAlreadyInSet = false;
			TransitionViewSet.Add(UAV, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				TransitionUAVs.Add(UAV);
			}
		}
	}

	inline void AddResourceClear(FD3D12UnorderedAccessView* UAV)
	{
		bool bAlreadyInSet = false;
		TransitionClearSet.Add(UAV, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			ClearUAVs.Add(UAV);
		}
	}
};

/** Struct to collect shader bundle bindings. */
struct FWorkGraphShaderBundleBinder
{
	FD3D12CommandContext& Context;
	FShaderBundleBinderOps& BinderOps;
	const uint32 GPUIndex;
	const EShaderFrequency Frequency;
	const bool bBindlessResources;
	const bool bBindlessSamplers;

	uint32 CBVVersions[MAX_CBS];
	uint32 SRVVersions[MAX_SRVS];
	uint32 UAVVersions[MAX_UAVS];
	uint32 SamplerVersions[MAX_SAMPLERS];

	uint64 BoundCBVMask = 0;
	uint64 BoundSRVMask = 0;
	uint64 BoundUAVMask = 0;
	uint64 BoundSamplerMask = 0;

	D3D12_CPU_DESCRIPTOR_HANDLE LocalCBVs[MAX_CBS];
	D3D12_CPU_DESCRIPTOR_HANDLE LocalSRVs[MAX_SRVS];
	D3D12_CPU_DESCRIPTOR_HANDLE LocalUAVs[MAX_UAVS];
	D3D12_CPU_DESCRIPTOR_HANDLE LocalSamplers[MAX_SAMPLERS];

	FWorkGraphShaderBundleBinder(FD3D12CommandContext& InContext, EShaderFrequency InFrequency, FShaderBundleBinderOps& InBinderOps, FD3D12ShaderData const* ShaderData)
		: Context(InContext)
		, BinderOps(InBinderOps)
		, GPUIndex(InContext.GetGPUIndex())
		, Frequency(InFrequency)
		, bBindlessResources(ShaderData->UsesBindlessResources())
		, bBindlessSamplers(ShaderData->UsesBindlessSamplers())
	{
	}

	void SetUAV(FRHIUnorderedAccessView* InUnorderedAccessView, uint32 Index, bool bClearResources = false)
	{
		FD3D12UnorderedAccessView* UAV = FD3D12CommandContext::RetrieveObject<FD3D12UnorderedAccessView_RHI>(InUnorderedAccessView, GPUIndex);
		check(UAV != nullptr);

		if (bClearResources)
		{
			BinderOps.AddResourceClear(UAV);
		}

		if (!bBindlessResources)
		{
			FD3D12OfflineDescriptor Descriptor = UAV->GetOfflineCpuHandle();
			LocalUAVs[Index] = Descriptor;
			UAVVersions[Index] = Descriptor.GetVersion();
			BoundUAVMask |= 1ull << Index;
		}

		BinderOps.AddResourceTransition(UAV);
	}

	void SetSRV(FRHIShaderResourceView* InShaderResourceView, uint32 Index)
	{
		FD3D12ShaderResourceView_RHI* SRV = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(InShaderResourceView, GPUIndex);
		check(SRV != nullptr);

		if (!bBindlessResources)
		{
			FD3D12OfflineDescriptor Descriptor = SRV->GetOfflineCpuHandle();
			LocalSRVs[Index] = Descriptor;
			SRVVersions[Index] = Descriptor.GetVersion();
			BoundSRVMask |= 1ull << Index;
		}
		
		BinderOps.AddResourceTransition(SRV);
	}

	void SetTexture(FRHITexture* InTexture, uint32 Index)
	{
		FD3D12ShaderResourceView* SRV = FD3D12CommandContext::RetrieveTexture(InTexture, GPUIndex)->GetShaderResourceView();
		check(SRV != nullptr);

		if (!bBindlessResources)
		{
			FD3D12OfflineDescriptor Descriptor = SRV->GetOfflineCpuHandle();
			LocalSRVs[Index] = Descriptor;
			SRVVersions[Index] = Descriptor.GetVersion();
			BoundSRVMask |= 1ull << Index;
		}

		BinderOps.AddResourceTransition(SRV);
	}

	void SetSampler(FRHISamplerState* InSampler, uint32 Index)
	{
		FD3D12SamplerState* Sampler = FD3D12CommandContext::RetrieveObject<FD3D12SamplerState>(InSampler, GPUIndex);
		check(Sampler != nullptr);

		if (!bBindlessSamplers)
		{
			FD3D12OfflineDescriptor Descriptor = Sampler->OfflineDescriptor;
			LocalSamplers[Index] = Descriptor;
			SamplerVersions[Index] = Descriptor.GetVersion();
			BoundSamplerMask |= 1ull << Index;
		}
	}

	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		FD3D12ResourceCollection* D3D12ResourceCollection = FD3D12CommandContext::RetrieveObject<FD3D12ResourceCollection>(ResourceCollection, GPUIndex);
		FD3D12ShaderResourceView* SRV = D3D12ResourceCollection ? D3D12ResourceCollection->GetShaderResourceView() : nullptr;

		check(bBindlessResources);
		if (SRV && bBindlessResources)
		{
			Context.StateCache.QueueBindlessSRV(Frequency, SRV);
			Context.StateCache.QueueBindlessSRVs(Frequency, D3D12ResourceCollection->AllSrvs);

			// Resolve texture references to their current textures and ensure residency.
			for (FD3D12RHITextureReference* TextureReference : D3D12ResourceCollection->AllTextureReferences)
			{
				if (FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureReference))
				{
					Context.StateCache.QueueBindlessSRV(Frequency, Texture->GetShaderResourceView());
				}
			}
		}
	}
};

/** Wrapper for constant buffer and it's underlying resource allocation. */
struct FAllocatedConstantBuffer
{
	FAllocatedConstantBuffer(FD3D12CommandContext& Context)
		: ResourceLocation(Context.GetParentDevice())
	{
	}

	FD3D12ConstantBuffer* ConstantBuffer = nullptr;
	FD3D12ResourceLocation ResourceLocation;
};

// Record bindings from shader bundle parameters.
static void RecordBindings(
	FD3D12CommandContext& Context,
	EShaderFrequency Frequency,
	FD3D12ExplicitDescriptorCache& TransientDescriptorCache,
	FShaderBundleBinderOps& BinderOps,
	uint32 WorkerIndex,
	FRHIShader* ShaderRHI,
	FD3D12ShaderData const* D3D12ShaderData,
	FRHIBatchedShaderParameters const& Parameters,
	FD3D12RootSignature const* LocalRootSignature,
	FAllocatedConstantBuffer const& SharedConstantBuffer,
	FUint32Vector4 const& Constants,
	TArrayView<uint32> RootArgs
)
{
	const uint32 NumSMPs = D3D12ShaderData->ResourceCounts.NumSamplers;
	const uint32 NumSRVs = D3D12ShaderData->ResourceCounts.NumSRVs;
	const uint32 NumCBVs = D3D12ShaderData->ResourceCounts.NumCBs;
	const uint32 NumUAVs = D3D12ShaderData->ResourceCounts.NumUAVs;

	// With shader root constants, we should never hit this expensive path!
	// If we hit this, check if the shaders in the bundle had loose
	// uniform parameters added to it recently, falling into this path.
	check(!D3D12ShaderData->UsesGlobalUniformBuffer() || SharedConstantBuffer.ConstantBuffer != nullptr);

	FWorkGraphShaderBundleBinder BundleBinder(Context, Frequency, BinderOps, D3D12ShaderData);

	FD3D12UniformBuffer* BundleUniformBuffers[MAX_CBS] = { nullptr };

	uint32 UniformBufferMask = 0u;

	const bool bClearUAVResources = false;

	for (const FRHIShaderParameterResource& Parameter : Parameters.ResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			BundleBinder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			BundleBinder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			BundleBinder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index, bClearUAVResources);
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			BundleBinder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BundleUniformBuffers[Parameter.Index] = FD3D12CommandContext::RetrieveObject<FD3D12UniformBuffer>(Parameter.Resource, 0 /*GpuIndex*/);
			UniformBufferMask |= (1 << Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceCollection:
			BundleBinder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Parameter.Resource), Parameter.Index);
			break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}

	UE::RHICore::ApplyStaticUniformBuffers(ShaderRHI, Context.GetStaticUniformBuffers(),
		[&](int32 BufferIndex, FRHIUniformBuffer* Buffer)
		{
			BundleUniformBuffers[BufferIndex] = Context.RetrieveObject<FD3D12UniformBuffer>(Buffer);
		});


	uint32 FakeDirtyUniformBuffers = ~(0u);
	UE::RHI::Private::SetUniformBufferResourcesFromTables(BundleBinder, *ShaderRHI, FakeDirtyUniformBuffers, BundleUniformBuffers
#if ENABLE_RHI_VALIDATION
		, Context.Tracker
#endif
	);

	if (SharedConstantBuffer.ConstantBuffer != nullptr)
	{
		check(BundleUniformBuffers[0] == nullptr);
		BundleBinder.BoundCBVMask |= 1ull << 0;
	}

	for (uint32 CBVIndex = 0; CBVIndex < MAX_CBS; ++CBVIndex)
	{
		FD3D12UniformBuffer* UniformBuffer = BundleUniformBuffers[CBVIndex];
		if (UniformBuffer)
		{
			BundleBinder.BoundCBVMask |= 1ull << CBVIndex;
		}
	}

	// Validate that all resources required by the shader are set
	auto IsCompleteBinding = [](uint32 ExpectedCount, uint64 BoundMask)
	{
		if (ExpectedCount > 64) return false; // Bound resource mask can't be represented by uint64

		// All bits of the mask [0..ExpectedCount) are expected to be set
		uint64 ExpectedMask = ExpectedCount == 64 ? ~0ull : ((1ull << ExpectedCount) - 1);
		return (ExpectedMask & BoundMask) == ExpectedMask;
	};

	check(IsCompleteBinding(D3D12ShaderData->ResourceCounts.NumSRVs, BundleBinder.BoundSRVMask));
	check(IsCompleteBinding(D3D12ShaderData->ResourceCounts.NumUAVs, BundleBinder.BoundUAVMask));
	check(IsCompleteBinding(D3D12ShaderData->ResourceCounts.NumCBs, BundleBinder.BoundCBVMask));
	check(IsCompleteBinding(D3D12ShaderData->ResourceCounts.NumSamplers, BundleBinder.BoundSamplerMask));

	if (NumSRVs > 0)
	{
		const int32 DescriptorTableBaseIndex = TransientDescriptorCache.Allocate(BundleBinder.LocalSRVs, NumSRVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
		check(DescriptorTableBaseIndex != INDEX_NONE);

		const uint32 BindSlot = LocalRootSignature->SRVRDTBindSlot(Frequency);
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = TransientDescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		FMemory::Memcpy(&RootArgs[BindSlotOffset], &ResourceDescriptorTableBaseGPU, sizeof(ResourceDescriptorTableBaseGPU));
	}

	if (NumSMPs > 0)
	{
		const int32 DescriptorTableBaseIndex = TransientDescriptorCache.AllocateDeduplicated(BundleBinder.SamplerVersions, BundleBinder.LocalSamplers, NumSMPs, D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER, WorkerIndex);
		check(DescriptorTableBaseIndex != INDEX_NONE);

		const uint32 BindSlot = LocalRootSignature->SamplerRDTBindSlot(Frequency);
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = TransientDescriptorCache.SamplerHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		FMemory::Memcpy(&RootArgs[BindSlotOffset], &ResourceDescriptorTableBaseGPU, sizeof(ResourceDescriptorTableBaseGPU));
	}

	if (NumUAVs > 0)
	{
		const int32 DescriptorTableBaseIndex = TransientDescriptorCache.AllocateDeduplicated(BundleBinder.UAVVersions, BundleBinder.LocalUAVs, NumUAVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
		//const int32 DescriptorTableBaseIndex = TransientDescriptorCache.Allocate(BundleBinder.LocalUAVs, NumUAVs, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, WorkerIndex);
		check(DescriptorTableBaseIndex != INDEX_NONE);

		const uint32 BindSlot = LocalRootSignature->UAVRDTBindSlot(Frequency);
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = TransientDescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);
		FMemory::Memcpy(&RootArgs[BindSlotOffset], &ResourceDescriptorTableBaseGPU, sizeof(ResourceDescriptorTableBaseGPU));
	}

	if (SharedConstantBuffer.ConstantBuffer != nullptr)
	{
		const uint32 BindSlot = LocalRootSignature->CBVRDBindSlot(Frequency, 0);
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		D3D12_GPU_VIRTUAL_ADDRESS Address = SharedConstantBuffer.ResourceLocation.GetGPUVirtualAddress();
		FMemory::Memcpy(&RootArgs[BindSlotOffset], &Address, sizeof(Address));
	}

	for (uint32 CBVIndex = 0; CBVIndex < NumCBVs; ++CBVIndex)
	{
		const uint32 BindSlot = LocalRootSignature->CBVRDBindSlot(Frequency, CBVIndex);
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		FD3D12UniformBuffer* UniformBuffer = BundleUniformBuffers[CBVIndex];
		if (UniformBuffer)
		{
			D3D12_GPU_VIRTUAL_ADDRESS Address = UniformBuffer->ResourceLocation.GetGPUVirtualAddress();
			FMemory::Memcpy(&RootArgs[BindSlotOffset], &Address, sizeof(Address));
		}
	}

	const int8 BindSlot = LocalRootSignature->GetRootConstantsSlot();
	if (BindSlot != -1)
	{
		const uint32 BindSlotOffset = LocalRootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		RootArgs[BindSlotOffset] = Constants.X;
		RootArgs[BindSlotOffset + 1] = Constants.Y;
		RootArgs[BindSlotOffset + 2] = Constants.Z;
		RootArgs[BindSlotOffset + 3] = Constants.W;
	}
}

struct FD3D12BindlessConstantSetter
{
	FD3D12CommandContext& Context;
	FD3D12ConstantBuffer& ConstantBuffer;
	const uint32 GpuIndex;
	const EShaderFrequency Frequency;

	FD3D12BindlessConstantSetter(FD3D12CommandContext& InContext, EShaderFrequency InFrequency)
		: Context(InContext)
		, ConstantBuffer(InContext.StageConstantBuffers[SF_Compute])
		, GpuIndex(InContext.GetGPUIndex())
		, Frequency(InFrequency)
	{
	}

	void SetBindlessHandle(const FRHIDescriptorHandle& Handle, uint32 Offset)
	{
		if (Handle.IsValid())
		{
			const uint32 BindlessIndex = Handle.GetIndex();
			ConstantBuffer.UpdateConstant(reinterpret_cast<const uint8*>(&BindlessIndex), Offset, 4);
		}
	}

	void SetUAV(FD3D12UnorderedAccessView* D3D12UnorderedAccessView, uint32 Offset)
	{
		SetBindlessHandle(D3D12UnorderedAccessView->GetBindlessHandle(), Offset);
		Context.StateCache.QueueBindlessUAV(Frequency, D3D12UnorderedAccessView);
	}

	void SetUAV(FRHIUnorderedAccessView* InUnorderedAccessView, uint32 Offset)
	{
		FD3D12UnorderedAccessView_RHI* D3D12UnorderedAccessView = FD3D12CommandContext::RetrieveObject<FD3D12UnorderedAccessView_RHI>(InUnorderedAccessView, GpuIndex);
		SetUAV(static_cast<FD3D12UnorderedAccessView*>(D3D12UnorderedAccessView), Offset);
	}

	void SetSRV(FD3D12ShaderResourceView* D3D12ShaderResourceView, uint32 Offset)
	{
		SetBindlessHandle(D3D12ShaderResourceView->GetBindlessHandle(), Offset);
		Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
	}

	void SetSRV(FRHIShaderResourceView* InShaderResourceView, uint32 Offset)
	{
		FD3D12ShaderResourceView_RHI* D3D12ShaderResourceView = FD3D12CommandContext::RetrieveObject<FD3D12ShaderResourceView_RHI>(InShaderResourceView, GpuIndex);
		SetSRV(static_cast<FD3D12ShaderResourceView*>(D3D12ShaderResourceView), Offset);
	}

	void SetTexture(FRHITexture* InTexture, uint32 Offset)
	{
		FD3D12Texture* D3D12Texture = FD3D12CommandContext::RetrieveTexture(InTexture, GpuIndex);
		FD3D12ShaderResourceView* D3D12ShaderResourceView = D3D12Texture ? D3D12Texture->GetShaderResourceView() : nullptr;

		SetBindlessHandle(InTexture->GetDefaultBindlessHandle(), Offset);
		Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
	}

	void SetSampler(FRHISamplerState* InSampler, uint32 Offset)
	{
		FD3D12SamplerState* D3D12SamplerState = FD3D12CommandContext::RetrieveObject<FD3D12SamplerState>(InSampler, GpuIndex);

		SetBindlessHandle(D3D12SamplerState->GetBindlessHandle(), Offset);
	}

	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Offset)
	{
		if (FD3D12ResourceCollection* D3D12ResourceCollection = FD3D12CommandContext::RetrieveObject<FD3D12ResourceCollection>(ResourceCollection, GpuIndex))
		{
			FD3D12ShaderResourceView* D3D12ShaderResourceView = D3D12ResourceCollection->GetShaderResourceView();
			Context.StateCache.QueueBindlessSRV(Frequency, D3D12ShaderResourceView);
			Context.StateCache.QueueBindlessSRVs(Frequency, D3D12ResourceCollection->AllSrvs);

			// We have to go through each TextureReference to get the most recent version.
			for (FD3D12RHITextureReference* TextureReference : D3D12ResourceCollection->AllTextureReferences)
			{
				if (FD3D12Texture* Texture = GetD3D12TextureFromRHITexture(TextureReference))
				{
					Context.StateCache.QueueBindlessSRV(Frequency, Texture->GetShaderResourceView());
				}
			}
		}
	}

	void Finalize(FAllocatedConstantBuffer& OutConstantBuffer)
	{
		OutConstantBuffer.ConstantBuffer = &ConstantBuffer;
		ConstantBuffer.Version(OutConstantBuffer.ResourceLocation, false);
	}
};

void SetShaderBundleSharedBindlessConstants(FD3D12CommandContext& Context, TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters, FAllocatedConstantBuffer& OutConstantBuffer)
{
	if (SharedBindlessParameters.Num())
	{
		FD3D12BindlessConstantSetter Setter(Context, SF_Compute);

		for (const FRHIShaderParameterResource& Parameter : SharedBindlessParameters)
		{
			if (FRHIResource* Resource = Parameter.Resource)
			{
				switch (Parameter.Type)
				{
				case FRHIShaderParameterResource::EType::Texture:
					Setter.SetTexture(static_cast<FRHITexture*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceView:
					Setter.SetSRV(static_cast<FRHIShaderResourceView*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::UnorderedAccessView:
					Setter.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::Sampler:
					break;
				case FRHIShaderParameterResource::EType::ResourceCollection:
					Setter.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource), Parameter.Index);
					break;
				}
			}
		}

		Setter.Finalize(OutConstantBuffer);
	}
}

#endif // D3D12_RHI_WORKGRAPHS

void FD3D12CommandContext::DispatchWorkGraphShaderBundle(
	FRHIShaderBundle* ShaderBundle, 
	FRHIBuffer* RecordArgBuffer, 
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters, 
	TConstArrayView<FRHIShaderBundleComputeDispatch> Dispatches)
{
#if D3D12_RHI_WORKGRAPHS

	TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);

	FD3D12ShaderBundle* D3D12ShaderBundle = static_cast<FD3D12ShaderBundle*>(FD3D12DynamicRHI::ResourceCast(ShaderBundle));

	TShaderRef<FDispatchShaderBundleWorkGraph> WorkGraphGlobalShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FDispatchShaderBundleWorkGraph>();
	FD3D12WorkGraphShader* D3D12EntryShader = FD3D12DynamicRHI::ResourceCast(WorkGraphGlobalShader.GetWorkGraphShader());
	const bool bBindlessResources = D3D12EntryShader->UsesBindlessResources();

	uint32 ViewDescriptorCount = D3D12EntryShader->ResourceCounts.NumSRVs + D3D12EntryShader->ResourceCounts.NumCBs + D3D12EntryShader->ResourceCounts.NumUAVs;
	uint32 SamplerDescriptorCount = D3D12EntryShader->ResourceCounts.NumSamplers;

	const int32 NumRecords = Dispatches.Num();
	checkf(NumRecords <= FDispatchShaderBundleWorkGraph::GetMaxShaderBundleSize(), TEXT("Too many entries in a shader bundle (%d). Try increasing 'r.ShaderBundle.MaxSize'"), NumRecords);

	TArray<uint32> ValidRecords;
	ValidRecords.Reserve(NumRecords);
	TArray<FRHIWorkGraphShader*> LocalNodeShaders;
	LocalNodeShaders.Reserve(NumRecords + 1);
	LocalNodeShaders.Add(D3D12EntryShader);

	for (int32 DispatchIndex = 0; DispatchIndex < NumRecords; ++DispatchIndex)
	{
		const FRHIShaderBundleComputeDispatch& Dispatch = Dispatches[DispatchIndex];
		FRHIWorkGraphShader* Shader = Dispatch.IsValid() ? Dispatch.WorkGraphShader : nullptr;
		LocalNodeShaders.Add(Shader);

		if (Shader != nullptr)
		{
			ValidRecords.Add(uint32(DispatchIndex));

			if (FD3D12WorkGraphShader* D3D12Shader = FD3D12DynamicRHI::ResourceCast(Shader))
			{
				ViewDescriptorCount += D3D12Shader->ResourceCounts.NumSRVs + D3D12Shader->ResourceCounts.NumCBs + D3D12Shader->ResourceCounts.NumUAVs;
				SamplerDescriptorCount += D3D12Shader->ResourceCounts.NumSamplers;
			}
		}
	}
	const int32 NumValidRecords = ValidRecords.Num();

	FWorkGraphPipelineStateInitializer Initializer;
	Initializer.SetProgramName(TEXT("ShaderBundle"));
	TArray<FWorkGraphPipelineStateInitializer::FNameMap> NameTable;
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT("WorkGraphMainCS"), TEXT("WorkGraphMainCS"))); // Entry node.
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT(""), TEXT("ShaderBundleNode"))); // Empty shader slots still increment bundle node index.
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT("MainCS"), TEXT("ShaderBundleNode"))); // Nanite compute materials.
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT("MicropolyRasterize"), TEXT("ShaderBundleNode"))); // Nanite software rasterize.
	Initializer.SetNameTable(NameTable);
	Initializer.SetShaderTable(LocalNodeShaders);

	FWorkGraphPipelineState* WorkGraphPipelineState = PipelineStateCache::GetAndOrCreateWorkGraphPipelineState(RHICmdList, Initializer);
	FD3D12WorkGraphPipelineState* Pipeline = static_cast<FD3D12WorkGraphPipelineState*>(GetRHIWorkGraphPipelineState(WorkGraphPipelineState));

	const uint32 NumViewDescriptors = ViewDescriptorCount;
	const uint32 NumSamplerDescriptors = SamplerDescriptorCount;

	const uint32 MaxWorkers = 4u;
	const uint32 NumWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
	const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() ? FMath::Min<uint32>(NumWorkerThreads, MaxWorkers) : 1u;

	struct FTaskContext
	{
		uint32 WorkerIndex = 0;
	};

	TArray<FTaskContext, TInlineAllocator<MaxWorkers>> TaskContexts;
	for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
	{
		TaskContexts.Add(FTaskContext{ WorkerIndex });
	}

	FD3D12ExplicitDescriptorCache TransientDescriptorCache(GetParentDevice(), MaxTasks /* Worker Count */);
	TransientDescriptorCache.Init(0, NumViewDescriptors, NumSamplerDescriptors, ERHIBindlessConfiguration::Minimal);

	TArray<FShaderBundleBinderOps, TInlineAllocator<MaxWorkers>> BinderOps;
	BinderOps.SetNum(MaxTasks);

	TResourceArray<uint32> LocalRootArgs;
	uint32 MinRootArgBufferSizeInDWords = (Pipeline->RootArgStrideInBytes / 4) * (Pipeline->MaxRootArgOffset + 1);
	LocalRootArgs.AddZeroed(MinRootArgBufferSizeInDWords);

	FAllocatedConstantBuffer SharedConstantBuffer(*this);
	SetShaderBundleSharedBindlessConstants(*this, SharedBindlessParameters, SharedConstantBuffer);

	auto RecordTask = [this, &LocalRootArgs, Pipeline, &TransientDescriptorCache, &ValidRecords, &Dispatches, &BinderOps, &SharedConstantBuffer](FTaskContext& Context, int32 RecordIndex)
	{
		const uint32 DispatchIndex = ValidRecords[RecordIndex];
		const FRHIShaderBundleComputeDispatch& Dispatch = Dispatches[DispatchIndex];
		check(Dispatch.IsValid());

		const uint32 ShaderTableIndex = RecordIndex + 1;
		check(Pipeline->RootArgOffsets.IsValidIndex(ShaderTableIndex));
		uint32 RootArgOffset = Pipeline->RootArgOffsets[ShaderTableIndex];
		check((Pipeline->RootArgStrideInBytes / 4) * (RootArgOffset + 1) <= (uint32)LocalRootArgs.Num());

		FD3D12WorkGraphShader* D3D12WorkGraphShader = FD3D12DynamicRHI::ResourceCast(Dispatch.WorkGraphShader);
		FD3D12RootSignature const* LocalRootSignature = D3D12WorkGraphShader->RootSignature;

		RecordBindings(
			*this,
			SF_Compute,
			TransientDescriptorCache,
			BinderOps[Context.WorkerIndex],
			Context.WorkerIndex,
			Dispatch.WorkGraphShader,
			D3D12WorkGraphShader,
			*Dispatch.Parameters,
			LocalRootSignature,
			SharedConstantBuffer,
			Dispatch.Constants,
			MakeArrayView(&LocalRootArgs[RootArgOffset * Pipeline->RootArgStrideInBytes / 4], Pipeline->RootArgStrideInBytes / 4)
		);
	};

	// One helper worker task will be created at most per this many work items, plus one worker for current thread (unless running on a task thread),
	// up to a hard maximum of MaxWorkers.
	// Internally, parallel for tasks still subdivide the work into smaller chunks and perform fine-grained load-balancing.
	const int32 ItemsPerTask = 1024;

	ParallelForWithExistingTaskContext(TEXT("DispatchShaderBundle"), MakeArrayView(TaskContexts), ValidRecords.Num(), ItemsPerTask, RecordTask);

	// Bind RecordArgBuffer
	FD3D12Buffer* RecordArgBufferPtr = FD3D12DynamicRHI::ResourceCast(RecordArgBuffer);

	D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
	SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
	SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
	SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
	SRVDesc.Buffer.FirstElement = 0;
	SRVDesc.Buffer.NumElements = RecordArgBufferPtr->GetSize() >> 2u;
	SRVDesc.Buffer.StructureByteStride = 0;
	SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;

	TSharedPtr<FD3D12ShaderResourceView> RecordArgBufferSRV;

	// Always single GPU object, so FirstLinkedObject is nullptr
	RecordArgBufferSRV = MakeShared<FD3D12ShaderResourceView>(GetParentDevice(), nullptr, ERHIDescriptorType::BufferSRV);
	RecordArgBufferSRV->CreateView(RecordArgBufferPtr, SRVDesc, FD3D12ShaderResourceView::EFlags::None);

	uint32 RecordArgBufferBindlessHandle = 0;

	if (bBindlessResources)
	{
		RecordArgBufferBindlessHandle = RecordArgBufferSRV->GetBindlessHandle().GetIndex();
		check(RecordArgBufferBindlessHandle != INDEX_NONE);
	}
	else
	{
		D3D12_CPU_DESCRIPTOR_HANDLE LocalSRVs[MAX_SRVS];
		LocalSRVs[WorkGraphGlobalShader->RecordArgBufferParam.GetBaseIndex()] = RecordArgBufferSRV->GetOfflineCpuHandle();

		const int32 DescriptorTableBaseIndex = TransientDescriptorCache.Allocate(LocalSRVs, 1, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, 0);
		const D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = TransientDescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);

		const uint32 BindSlot = D3D12EntryShader->RootSignature->SRVRDTBindSlot(SF_Compute);
		const uint32 BindSlotOffset = D3D12EntryShader->RootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		TArrayView<uint32> RootArgSlice = MakeArrayView(&LocalRootArgs[Pipeline->RootArgOffsets[0] * Pipeline->RootArgStrideInBytes / 4], Pipeline->RootArgStrideInBytes / 4);
		FMemory::Memcpy(&RootArgSlice[BindSlotOffset], &ResourceDescriptorTableBaseGPU, sizeof(ResourceDescriptorTableBaseGPU));
	}

	// Upload local root arguments table.
	D3D12_GPU_VIRTUAL_ADDRESS_RANGE_AND_STRIDE NodeLocalRootArgumentsTable{ 0, 0, 0 };
	if (ValidRecords.Num() && LocalRootArgs.Num())
	{
		const uint32 DataSize = LocalRootArgs.GetResourceDataSize();

		// todo: Check if copy queue is the optimal way to upload the root args.
		// todo: Use a single buffer owned by the shader bundle RHI object (needs a copy operation that doesn't complain about multiple uploads).
		const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(DataSize, D3D12_RESOURCE_FLAG_NONE);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("BundleRecordBuffer"), DataSize, 0, EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::CopyDest)
			.SetGPUMask(FRHIGPUMask::FromIndex(GetParentDevice()->GetGPUIndex()));

		FD3D12Buffer* RootArgBuffer = GetParentDevice()->GetParentAdapter()->CreateRHIBuffer(
			ResourceDesc,
			16,
			CreateDesc,
			ED3D12ResourceStateMode::MultiState,
			ED3D12Access::CopyDest,
			true
		);

		BatchedSyncPoints.ToWait.Emplace(RootArgBuffer->UploadResourceDataViaCopyQueue(*this, &LocalRootArgs));
		AddBarrier(RootArgBuffer->GetResource(), ED3D12Access::CopyDest, ED3D12Access::Common, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);

		NodeLocalRootArgumentsTable.StartAddress = RootArgBuffer->ResourceLocation.GetGPUVirtualAddress();
		NodeLocalRootArgumentsTable.SizeInBytes = RootArgBuffer->ResourceLocation.GetSize();
		NodeLocalRootArgumentsTable.StrideInBytes = Pipeline->RootArgStrideInBytes;
	}

	// Apply Binder Ops
	{
		for (int32 WorkerIndex = 1; WorkerIndex < BinderOps.Num(); ++WorkerIndex)
		{
			for (FD3D12ShaderResourceView* SRV : BinderOps[WorkerIndex].TransitionSRVs)
			{
				BinderOps[0].AddResourceTransition(SRV);
			}

			for (FD3D12UnorderedAccessView* UAV : BinderOps[WorkerIndex].TransitionUAVs)
			{
				BinderOps[0].AddResourceTransition(UAV);
			}

			BinderOps[WorkerIndex].TransitionSRVs.Empty();
			BinderOps[WorkerIndex].TransitionUAVs.Empty();
			BinderOps[WorkerIndex].TransitionViewSet.Empty();

			BinderOps[WorkerIndex].ClearUAVs.Empty();
			BinderOps[WorkerIndex].TransitionClearSet.Empty();
		}

		for (FD3D12UnorderedAccessView* UAV : BinderOps[0].ClearUAVs)
		{
			ClearShaderResources(UAV, EShaderParameterTypeMask::SRVMask);
		}
//TODO: check resource view
// 		for (FD3D12ShaderResourceView* SRV : BinderOps[0].TransitionSRVs)
// 		{
// 			TransitionResource(SRV, D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE);
// 		}
// 
// 		for (FD3D12UnorderedAccessView* UAV : BinderOps[0].TransitionUAVs)
// 		{
// 			TransitionResource(UAV, D3D12_RESOURCE_STATE_UNORDERED_ACCESS);
// 		}
	}

	FlushResourceBarriers();

	// Apply the transient descriptor heaps.
	// Note this only uses transient heap for non-bindless.
	SetExplicitDescriptorCache(TransientDescriptorCache);

	if (bBindlessResources)
	{
		StateCache.ApplyBindlessResources(nullptr, SF_Compute, SF_NumStandardFrequencies);
	}

	GraphicsCommandList()->SetComputeRootSignature(Pipeline->RootSignature);

	// Kick off the work graph
	D3D12_SET_PROGRAM_DESC SetProgramDesc = {};
	SetProgramDesc.Type = D3D12_PROGRAM_TYPE_WORK_GRAPH;
	SetProgramDesc.WorkGraph.ProgramIdentifier = Pipeline->ProgramIdentifier;
	SetProgramDesc.WorkGraph.Flags = !Pipeline->bInitialized ? D3D12_SET_WORK_GRAPH_FLAG_INITIALIZE : D3D12_SET_WORK_GRAPH_FLAG_NONE;
	SetProgramDesc.WorkGraph.BackingMemory = Pipeline->BackingMemoryAddressRange;
	SetProgramDesc.WorkGraph.NodeLocalRootArgumentsTable = NodeLocalRootArgumentsTable;
	GraphicsCommandList10()->SetProgram(&SetProgramDesc);

	FDispatchShaderBundleWorkGraph::FEntryNodeRecord InputRecord = FDispatchShaderBundleWorkGraph::MakeInputRecord(NumRecords, ShaderBundle->ArgOffset, ShaderBundle->ArgStride, RecordArgBufferBindlessHandle);

	if (!GShaderBundleSkipDispatch)
	{
		D3D12_DISPATCH_GRAPH_DESC DispatchGraphDesc = {};
		DispatchGraphDesc.Mode = D3D12_DISPATCH_MODE_NODE_CPU_INPUT;
		DispatchGraphDesc.NodeCPUInput.EntrypointIndex = 0;
		DispatchGraphDesc.NodeCPUInput.NumRecords = 1;
		DispatchGraphDesc.NodeCPUInput.RecordStrideInBytes = sizeof(InputRecord);
		DispatchGraphDesc.NodeCPUInput.pRecords = &InputRecord;
		GraphicsCommandList10()->DispatchGraph(&DispatchGraphDesc);
	}

	// Pipeline state memory should now be initialized.
	Pipeline->bInitialized = true;

	// Restore global descriptor heaps if necessary.
	UnsetExplicitDescriptorCache();

	// We did not write through the state cache, so we need to invalidate it so subsequent workloads correctly re-bind state
	StateCache.DirtyState();

	ConditionalSplitCommandList();

#endif // D3D12_RHI_WORKGRAPHS
}

void FD3D12CommandContext::DispatchWorkGraphShaderBundle(
	FRHIShaderBundle* ShaderBundle, 
	FRHIBuffer* RecordArgBuffer, 
	const FRHIShaderBundleGraphicsState& BundleState, 
	TConstArrayView<FRHIShaderParameterResource> SharedBindlessParameters, 
	TConstArrayView<FRHIShaderBundleGraphicsDispatch> Dispatches)
{
#if D3D12_RHI_WORKGRAPHS

	TRHICommandList_RecursiveHazardous<FD3D12CommandContext> RHICmdList(this);

	FD3D12ShaderBundle* D3D12ShaderBundle = static_cast<FD3D12ShaderBundle*>(FD3D12DynamicRHI::ResourceCast(ShaderBundle));

	TShaderRef<FDispatchShaderBundleWorkGraph> WorkGraphGlobalShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FDispatchShaderBundleWorkGraph>();
	FD3D12WorkGraphShader* D3D12EntryShader = FD3D12DynamicRHI::ResourceCast(WorkGraphGlobalShader.GetWorkGraphShader());
	const bool bBindlessResources = D3D12EntryShader->UsesBindlessResources();

	uint32 ViewDescriptorCount = D3D12EntryShader->ResourceCounts.NumSRVs + D3D12EntryShader->ResourceCounts.NumCBs + D3D12EntryShader->ResourceCounts.NumUAVs;
	uint32 SamplerDescriptorCount = D3D12EntryShader->ResourceCounts.NumSamplers;

	TArray<FRHIWorkGraphShader*> LocalNodeShaders;
	LocalNodeShaders.Add(D3D12EntryShader);

	const int32 NumRecords = Dispatches.Num();
	checkf(NumRecords <= FDispatchShaderBundleWorkGraph::GetMaxShaderBundleSize(), TEXT("Too many entries in a shader bundle (%d). Try increasing 'r.ShaderBundle.MaxSize'"), NumRecords);

	TArray<uint32> ValidRecords;
	ValidRecords.Reserve(NumRecords);
	TArray<FGraphicsPipelineStateInitializer const*> LocalPSOs;
	LocalPSOs.Reserve(NumRecords);

	for (int32 DispatchIndex = 0; DispatchIndex < NumRecords; ++DispatchIndex)
	{
		const FRHIShaderBundleGraphicsDispatch& Dispatch = Dispatches[DispatchIndex];
		FGraphicsPipelineStateInitializer const* PSO = Dispatch.IsValid() ? &Dispatch.PipelineInitializer : nullptr;
		LocalPSOs.Add(PSO);

		if (PSO != nullptr)
		{
			FRHIWorkGraphShader* MeshShader = PSO->BoundShaderState.GetWorkGraphShader();
			FRHIPixelShader* PixelShader = PSO->BoundShaderState.GetPixelShader();
			if (MeshShader != nullptr && PixelShader != nullptr)
			{
				ValidRecords.Add(uint32(DispatchIndex));

				if (FD3D12WorkGraphShader* D3D12Shader = FD3D12DynamicRHI::ResourceCast(MeshShader))
				{
					ViewDescriptorCount += D3D12Shader->ResourceCounts.NumSRVs + D3D12Shader->ResourceCounts.NumCBs + D3D12Shader->ResourceCounts.NumUAVs;
					SamplerDescriptorCount += D3D12Shader->ResourceCounts.NumSamplers;
				}
				if (FD3D12PixelShader* D3D12Shader = FD3D12DynamicRHI::ResourceCast(PixelShader))
				{
					ViewDescriptorCount += D3D12Shader->ResourceCounts.NumSRVs + D3D12Shader->ResourceCounts.NumCBs + D3D12Shader->ResourceCounts.NumUAVs;
					SamplerDescriptorCount += D3D12Shader->ResourceCounts.NumSamplers;
				}
			}
		}
	}
	const int32 NumValidRecords = ValidRecords.Num();

	FWorkGraphPipelineStateInitializer Initializer;
	Initializer.SetProgramName(TEXT("ShaderBundle"));
	TArray<FWorkGraphPipelineStateInitializer::FNameMap> NameTable;
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT("WorkGraphMainCS"), TEXT("WorkGraphMainCS"))); // Entry node.
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT(""), TEXT("ShaderBundleNode"))); // Empty shader slots still increment bundle node index.
	NameTable.Add(FWorkGraphPipelineStateInitializer::FNameMap(TEXT("HWRasterizeMS"), TEXT("ShaderBundleNode"))); // Nanite software rasterize.
	Initializer.SetNameTable(NameTable);
	Initializer.SetShaderTable(LocalNodeShaders);
	Initializer.SetGraphicsPSOTable(LocalPSOs);

	FWorkGraphPipelineState* WorkGraphPipelineState = PipelineStateCache::GetAndOrCreateWorkGraphPipelineState(RHICmdList, Initializer);
	FD3D12WorkGraphPipelineState* Pipeline = static_cast<FD3D12WorkGraphPipelineState*>(GetRHIWorkGraphPipelineState(WorkGraphPipelineState));

	const uint32 NumViewDescriptors = ViewDescriptorCount;
	const uint32 NumSamplerDescriptors = SamplerDescriptorCount;

	const uint32 MaxWorkers = 4u;
	const uint32 NumWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
	const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() ? FMath::Min<uint32>(NumWorkerThreads, MaxWorkers) : 1u;

	struct FTaskContext
	{
		uint32 WorkerIndex = 0;
	};

	TArray<FTaskContext, TInlineAllocator<MaxWorkers>> TaskContexts;
	for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
	{
		TaskContexts.Add(FTaskContext{ WorkerIndex });
	}

	FD3D12ExplicitDescriptorCache TransientDescriptorCache(GetParentDevice(), MaxTasks /* Worker Count */);
	TransientDescriptorCache.Init(0, NumViewDescriptors, NumSamplerDescriptors, ERHIBindlessConfiguration::Minimal);

	TArray<FShaderBundleBinderOps, TInlineAllocator<MaxWorkers>> BinderOps;
	BinderOps.SetNum(MaxTasks);

	TResourceArray<uint32> LocalRootArgs;
	uint32 MinRootArgBufferSizeInDWords = (Pipeline->RootArgStrideInBytes / 4) * (Pipeline->MaxRootArgOffset + 1);
	LocalRootArgs.AddZeroed(MinRootArgBufferSizeInDWords);

	FAllocatedConstantBuffer SharedConstantBuffer(*this);
	SetShaderBundleSharedBindlessConstants(*this, SharedBindlessParameters, SharedConstantBuffer);

	auto RecordTask = [this, &LocalRootArgs, Pipeline, &TransientDescriptorCache, &ValidRecords, &Dispatches, &BinderOps, &SharedConstantBuffer](FTaskContext& Context, int32 RecordIndex)
	{
		uint32 DispatchIndex = ValidRecords[RecordIndex];
		const FRHIShaderBundleGraphicsDispatch& Dispatch = Dispatches[DispatchIndex];
		check(Dispatch.IsValid());

		const uint32 ShaderTableIndex = RecordIndex + 1;
		check(Pipeline->RootArgOffsets.IsValidIndex(ShaderTableIndex));
		const uint32 RootArgOffset = Pipeline->RootArgOffsets[ShaderTableIndex];
		check((Pipeline->RootArgStrideInBytes / 4) * (RootArgOffset + 1) <= (uint32)LocalRootArgs.Num());

		FRHIWorkGraphShader* MeshShader = Dispatch.PipelineInitializer.BoundShaderState.GetWorkGraphShader();
		FD3D12WorkGraphShader* D3D12MeshShader = FD3D12DynamicRHI::ResourceCast(MeshShader);
		FRHIPixelShader* PixelShader = Dispatch.PipelineInitializer.BoundShaderState.GetPixelShader();
		FD3D12PixelShader* D3D12PixelShader = FD3D12DynamicRHI::ResourceCast(PixelShader);
		FD3D12RootSignature const* LocalRootSignature = GetParentAdapter()->GetWorkGraphGraphicsRootSignature(Dispatch.PipelineInitializer.BoundShaderState);

		RecordBindings(
			*this,
			SF_Pixel,
			TransientDescriptorCache,
			BinderOps[Context.WorkerIndex],
			Context.WorkerIndex,
			PixelShader,
			D3D12PixelShader,
			*Dispatch.Parameters_PS,
			LocalRootSignature,
			SharedConstantBuffer,
			Dispatch.Constants,
			MakeArrayView(&LocalRootArgs[RootArgOffset * Pipeline->RootArgStrideInBytes / 4], Pipeline->RootArgStrideInBytes / 4)
		);

		RecordBindings(
			*this,
			SF_Mesh,
			TransientDescriptorCache,
			BinderOps[Context.WorkerIndex],
			Context.WorkerIndex,
			MeshShader,
			D3D12MeshShader,
			*Dispatch.Parameters_MSVS,
			LocalRootSignature,
			SharedConstantBuffer,
			Dispatch.Constants,
			MakeArrayView(&LocalRootArgs[RootArgOffset * Pipeline->RootArgStrideInBytes / 4], Pipeline->RootArgStrideInBytes / 4)
		);
	};

	const int32 ItemsPerTask = 1024;
	ParallelForWithExistingTaskContext(TEXT("DispatchShaderBundle"), MakeArrayView(TaskContexts), ValidRecords.Num(), ItemsPerTask, RecordTask);

	// Apply Binder Ops
	{
		for (int32 WorkerIndex = 1; WorkerIndex < BinderOps.Num(); ++WorkerIndex)
		{
			for (FD3D12ShaderResourceView* SRV : BinderOps[WorkerIndex].TransitionSRVs)
			{
				BinderOps[0].AddResourceTransition(SRV);
			}

			for (FD3D12UnorderedAccessView* UAV : BinderOps[WorkerIndex].TransitionUAVs)
			{
				BinderOps[0].AddResourceTransition(UAV);
			}

			BinderOps[WorkerIndex].TransitionSRVs.Empty();
			BinderOps[WorkerIndex].TransitionUAVs.Empty();
			BinderOps[WorkerIndex].TransitionViewSet.Empty();

			BinderOps[WorkerIndex].ClearUAVs.Empty();
			BinderOps[WorkerIndex].TransitionClearSet.Empty();
		}

		for (FD3D12UnorderedAccessView* UAV : BinderOps[0].ClearUAVs)
		{
			ClearShaderResources(UAV, EShaderParameterTypeMask::SRVMask);
		}

//TODO: check resource view
// 		for (FD3D12ShaderResourceView* SRV : BinderOps[0].TransitionSRVs)
// 		{
// 			TransitionResource(SRV, D3D12_RESOURCE_STATE_NON_PIXEL_SHADER_RESOURCE);
// 		}
// 
// 		for (FD3D12UnorderedAccessView* UAV : BinderOps[0].TransitionUAVs)
// 		{
// 			TransitionResource(UAV, D3D12_RESOURCE_STATE_UNORDERED_ACCESS);
//		}
	}

	FlushResourceBarriers();

	// Create SRV for RecordArgsBuffer
	FD3D12Buffer* RecordArgBufferPtr = FD3D12DynamicRHI::ResourceCast(RecordArgBuffer);

	D3D12_SHADER_RESOURCE_VIEW_DESC SRVDesc = {};
	SRVDesc.ViewDimension = D3D12_SRV_DIMENSION_BUFFER;
	SRVDesc.Format = DXGI_FORMAT_R32_TYPELESS;
	SRVDesc.Shader4ComponentMapping = D3D12_DEFAULT_SHADER_4_COMPONENT_MAPPING;
	SRVDesc.Buffer.FirstElement = 0;
	SRVDesc.Buffer.NumElements = RecordArgBufferPtr->GetSize() >> 2u;
	SRVDesc.Buffer.StructureByteStride = 0;
	SRVDesc.Buffer.Flags = D3D12_BUFFER_SRV_FLAG_RAW;

	TSharedPtr<FD3D12ShaderResourceView> RecordArgBufferSRV;
	// Always single GPU object, so FirstLinkedObject is nullptr
	RecordArgBufferSRV = MakeShared<FD3D12ShaderResourceView>(GetParentDevice(), nullptr, ERHIDescriptorType::BufferSRV);
	RecordArgBufferSRV->CreateView(RecordArgBufferPtr, SRVDesc, FD3D12ShaderResourceView::EFlags::None);

	// Gather root arguments for shader bundle entry node.
	uint32 RecordArgBufferBindlessHandle = 0;
	uint32 BindSlot = 0;
	D3D12_GPU_DESCRIPTOR_HANDLE ResourceDescriptorTableBaseGPU = {};

	if (bBindlessResources)
	{
		RecordArgBufferBindlessHandle = RecordArgBufferSRV->GetBindlessHandle().GetIndex();
		check(RecordArgBufferBindlessHandle != INDEX_NONE);

		StateCache.ApplyBindlessResources(nullptr, SF_Compute, SF_NumStandardFrequencies);
	}
	else
	{
		D3D12_CPU_DESCRIPTOR_HANDLE LocalSRVs[MAX_SRVS];
		LocalSRVs[WorkGraphGlobalShader->RecordArgBufferParam.GetBaseIndex()] = RecordArgBufferSRV->GetOfflineCpuHandle();

		const int32 DescriptorTableBaseIndex = TransientDescriptorCache.Allocate(LocalSRVs, 1, D3D12_DESCRIPTOR_HEAP_TYPE_CBV_SRV_UAV, 0);
		ResourceDescriptorTableBaseGPU = TransientDescriptorCache.ViewHeap.GetDescriptorGPU(DescriptorTableBaseIndex);

		BindSlot = D3D12EntryShader->RootSignature->SRVRDTBindSlot(SF_Compute);
		const uint32 BindSlotOffset = D3D12EntryShader->RootSignature->GetBindSlotOffsetInBytes(BindSlot) / 4;

		TArrayView<uint32> RootArgSlice = MakeArrayView(&LocalRootArgs[Pipeline->RootArgOffsets[0] * Pipeline->RootArgStrideInBytes / 4], Pipeline->RootArgStrideInBytes / 4);
		FMemory::Memcpy(&RootArgSlice[BindSlotOffset], &ResourceDescriptorTableBaseGPU, sizeof(ResourceDescriptorTableBaseGPU));
	}

	// Upload local root arguments table.
	D3D12_GPU_VIRTUAL_ADDRESS_RANGE_AND_STRIDE NodeLocalRootArgumentsTable{ 0, 0, 0 };
	if (LocalRootArgs.Num())
	{
		const uint32 DataSize = LocalRootArgs.GetResourceDataSize();

		// todo: Check if copy queue is the optimal way to upload the root args.
		// todo: Use a single buffer owned by the shader bundle RHI object (needs a copy operation that doesn't complain about multiple uploads).
		const D3D12_RESOURCE_DESC ResourceDesc = CD3DX12_RESOURCE_DESC::Buffer(DataSize, D3D12_RESOURCE_FLAG_NONE);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("BundleRecordBuffer"), DataSize, 0, EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::CopyDest)
			.SetGPUMask(FRHIGPUMask::FromIndex(GetParentDevice()->GetGPUIndex()));

		FD3D12Buffer* RootArgBuffer = GetParentDevice()->GetParentAdapter()->CreateRHIBuffer(
			ResourceDesc,
			16,
			CreateDesc,
			ED3D12ResourceStateMode::MultiState,
			ED3D12Access::CopyDest,
			true
		);

		BatchedSyncPoints.ToWait.Emplace(RootArgBuffer->UploadResourceDataViaCopyQueue(*this, &LocalRootArgs));
		AddBarrier(RootArgBuffer->GetResource(), ED3D12Access::CopyDest, ED3D12Access::Common, D3D12_RESOURCE_BARRIER_ALL_SUBRESOURCES);

		NodeLocalRootArgumentsTable.StartAddress = RootArgBuffer->ResourceLocation.GetGPUVirtualAddress();
		NodeLocalRootArgumentsTable.SizeInBytes = RootArgBuffer->ResourceLocation.GetSize();
		NodeLocalRootArgumentsTable.StrideInBytes = Pipeline->RootArgStrideInBytes;
	}

	// Apply the transient descriptor heaps.
	// Note this only uses transient heap for non-bindless.
	SetExplicitDescriptorCache(TransientDescriptorCache);

	// Set graphics state
	GraphicsCommandList()->SetGraphicsRootSignature(Pipeline->RootSignature);

	if (!bBindlessResources)
	{
		GraphicsCommandList()->SetGraphicsRootDescriptorTable(BindSlot, ResourceDescriptorTableBaseGPU);
	}

	GraphicsCommandList()->IASetVertexBuffers(0, D3D12_IA_VERTEX_INPUT_RESOURCE_SLOT_COUNT, nullptr);
	GraphicsCommandList()->IASetIndexBuffer(nullptr);
	GraphicsCommandList()->OMSetRenderTargets(0, nullptr, 0, nullptr);

	D3D12_VIEWPORT Viewport{};
	{
		Viewport.TopLeftX = BundleState.ViewRect.Min.X;
		Viewport.TopLeftY = BundleState.ViewRect.Min.Y;
		Viewport.Width = BundleState.ViewRect.Width();
		Viewport.Height = BundleState.ViewRect.Height();
		Viewport.MinDepth = BundleState.DepthMin;
		Viewport.MaxDepth = BundleState.DepthMax;
	}
	GraphicsCommandList()->RSSetViewports(1, &Viewport);

	D3D12_RECT Rect{};
	{
		Rect.left = BundleState.ViewRect.Min.X;
		Rect.top = BundleState.ViewRect.Min.Y;
		Rect.right = BundleState.ViewRect.Max.X;
		Rect.bottom = BundleState.ViewRect.Max.Y;
	}
	GraphicsCommandList()->RSSetScissorRects(1, &Rect);

	const D3D_PRIMITIVE_TOPOLOGY PrimitiveTopology = TranslatePrimitiveType(BundleState.PrimitiveType);
	GraphicsCommandList()->IASetPrimitiveTopology(PrimitiveTopology);

	GraphicsCommandList()->OMSetStencilRef(BundleState.StencilRef);
	GraphicsCommandList()->OMSetBlendFactor(BundleState.BlendFactor);
//	GraphicsCommandList()->OMSetDepthBounds(BundleState.DepthMin, BundleState.DepthMax);

	// Kick off the work graph	
	D3D12_SET_PROGRAM_DESC SetProgramDesc = {};
	SetProgramDesc.Type = D3D12_PROGRAM_TYPE_WORK_GRAPH;
	SetProgramDesc.WorkGraph.ProgramIdentifier = Pipeline->ProgramIdentifier;
	SetProgramDesc.WorkGraph.Flags = D3D12_SET_WORK_GRAPH_FLAG_INITIALIZE;
	SetProgramDesc.WorkGraph.BackingMemory = Pipeline->BackingMemoryAddressRange;
	SetProgramDesc.WorkGraph.NodeLocalRootArgumentsTable = NodeLocalRootArgumentsTable;
	GraphicsCommandList10()->SetProgram(&SetProgramDesc);

	Pipeline->FrameCounter.Set(GetFrameFenceCounter());

	const uint32 RecordCount = Dispatches.Num();
	FDispatchShaderBundleWorkGraph::FEntryNodeRecord InputRecord = FDispatchShaderBundleWorkGraph::MakeInputRecord(RecordCount, ShaderBundle->ArgOffset, ShaderBundle->ArgStride, RecordArgBufferBindlessHandle);

	if (!GShaderBundleSkipDispatch)
	{
		D3D12_DISPATCH_GRAPH_DESC DispatchGraphDesc = {};
		DispatchGraphDesc.Mode = D3D12_DISPATCH_MODE_NODE_CPU_INPUT;
		DispatchGraphDesc.NodeCPUInput.EntrypointIndex = 0;
		DispatchGraphDesc.NodeCPUInput.NumRecords = 1;
		DispatchGraphDesc.NodeCPUInput.RecordStrideInBytes = sizeof(InputRecord);
		DispatchGraphDesc.NodeCPUInput.pRecords = &InputRecord;
		GraphicsCommandList10()->DispatchGraph(&DispatchGraphDesc);
	}

	// Restore global descriptor heaps if necessary.
	UnsetExplicitDescriptorCache();

	// We did not write through the state cache, so we need to invalidate it so subsequent workloads correctly re-bind state
	StateCache.DirtyState();

	ConditionalSplitCommandList();

#endif // D3D12_RHI_WORKGRAPHS
}
