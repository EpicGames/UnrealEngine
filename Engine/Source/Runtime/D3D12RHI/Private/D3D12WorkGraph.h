// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHICommon.h"
#include "RHIResources.h"
#include "D3D12PipelineState.h"

class FD3D12Device;
class FWorkGraphPipelineStateInitializer;

struct FD3D12WorkGraphPipelineState : public FRHIWorkGraphPipelineState
{
public:
	UE_NONCOPYABLE(FD3D12WorkGraphPipelineState)

	FD3D12WorkGraphPipelineState(FD3D12Device* Device, const FWorkGraphPipelineStateInitializer& Initializer);

	FD3D12Device* Device;

#if D3D12_RHI_WORKGRAPHS
	TRefCountPtr<ID3D12RootSignature> RootSignature;
	TRefCountPtr<ID3D12StateObject>	StateObject;
	
	D3D12_PROGRAM_IDENTIFIER ProgramIdentifier = {};
	D3D12_GPU_VIRTUAL_ADDRESS_RANGE BackingMemoryAddressRange = {};

	uint32 RootArgStrideInBytes = 0;
	uint32 MaxRootArgOffset = 0;
	TArray<uint32> RootArgOffsets;

	TMap<FString, uint32> NodeCountPerName;

	bool bInitialized = false;

	D3D12ResourceFrameCounter FrameCounter;
	
#if NV_AFTERMATH
	TArray<TRefCountPtr<FRHIShader>> Shaders;
#endif // NV_AFTERMATH
#endif
};
