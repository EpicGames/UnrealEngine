// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHIDefinitions.h"

// For the CVars
#include "HAL/IConsoleManager.h"

// Can't forward declare D3D12_HEAP_FLAGS
#include "D3D12ThirdParty.h"

template <typename>
class TRefCountPtr;

struct FRHITransition;
struct FRHITransitionCreateInfo;
enum class ED3D12Access : uint32;

class FD3D12Resource;
struct FD3D12ResourceDesc;
class FD3D12CommandContext;
class FD3D12CommandList;
class FD3D12QueryAllocator;
class FD3D12ContextCommon;
class FD3D12Adapter;
enum class ED3D12ResourceStateMode;

class ID3D12BarriersForContext;

inline int32 GD3D12AllowDiscardResources = 1;
inline FAutoConsoleVariableRef CVarD3D12AllowDiscardResources(
	TEXT("d3d12.AllowDiscardResources"),
	GD3D12AllowDiscardResources,
	TEXT("Whether to call DiscardResources after transient aliasing acquire. This is not needed on some platforms if newly acquired resources are cleared before use."),
	ECVF_RenderThreadSafe);

inline int32 GD3D12DisableDiscardOfDepthResources = 0;
inline FAutoConsoleVariableRef CVarDisableDiscardOfDepthResources(
	TEXT("d3d12.DisableDiscardOfDepthResources"),
	GD3D12DisableDiscardOfDepthResources,
	TEXT("Whether to skip discarding depth resources after transient aliasing acquire. This is not needed on some platforms if the whole (sub)resource is written before it's read."));

inline int32 GD3D12BatchResourceBarriers = 1;
inline FAutoConsoleVariableRef CVarD3D12BatchResourceBarriers(
	TEXT("d3d12.BatchResourceBarriers"),
	GD3D12BatchResourceBarriers,
	TEXT("Whether to allow batching resource barriers"));

#if D3D12RHI_BARRIER_LOGGING
inline int32 GD3D12LogBarriers = 0;
inline FAutoConsoleVariableRef CVarD3D12LogBarriers(
	TEXT("d3d12.LogBarriers"),
	GD3D12LogBarriers,
	TEXT("Whether to log barriers. Use d3d12.LogBarriersResourceFilter to filter by resource name.\n")
	TEXT("0 - No logging\n")
	TEXT("1 - Log when batched\n")
	TEXT("2 - Log when flushed\n")
	TEXT("3 - Log when batched and when flushed"));

enum ED3D12LogBarriersFlags : decltype(GD3D12LogBarriers)
{
	D3D12LogBarriersNever       = 0,
	D3D12LogBarriersWhenBatched = 1u << 0,
	D3D12LogBarriersWhenFlushed = 1u << 1,
};

// Do not access the CVar directly as that will create a race.
// Instead use this variable for which updates are sync'd via the callback
inline FString GD3D12LogBarriersResourceFilter = "";
inline FAutoConsoleVariable CVarD3D12LogBarriersResourceFilter(
	TEXT("d3d12.LogBarriersResourceFilter"),
	TEXT(""),
	TEXT("Which resources to log barriers for. Set to empty to log everything."),
	FConsoleVariableDelegate::CreateStatic(
		[](IConsoleVariable* CVarFilter)
		{
			GD3D12LogBarriersResourceFilter = CVarFilter->GetString();
		}));
#endif // D3D12RHI_BARRIER_LOGGING

class ID3D12BarriersForAdapter
{
public:
	virtual ~ID3D12BarriersForAdapter() = default;

	virtual void ConfigureDevice(
		ID3D12Device* Device,
		bool InWithD3DDebug) const = 0;

	virtual uint64 GetTransitionDataSizeBytes() const = 0;
	virtual uint64 GetTransitionDataAlignmentBytes() const = 0;

	virtual void CreateTransition(
		FRHITransition* Transition,
		const FRHITransitionCreateInfo& CreateInfo) const = 0;

	virtual void ReleaseTransition(
		FRHITransition* Transition) const = 0;

	virtual HRESULT CreateCommittedResource(
		FD3D12Adapter& Adapter,
		const D3D12_HEAP_PROPERTIES& InHeapProps,
		D3D12_HEAP_FLAGS InHeapFlags,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const = 0;

	virtual HRESULT CreateReservedResource(
		FD3D12Adapter& Adapter,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const = 0;

	virtual HRESULT CreatePlacedResource(
		FD3D12Adapter& Adapter,
		ID3D12Heap* Heap,
		uint64 InHeapOffset,
		const FD3D12ResourceDesc& InDesc,
		ED3D12Access InInitialD3D12Access,
		const D3D12_CLEAR_VALUE* InClearValue,
		TRefCountPtr<ID3D12Resource>& OutResource) const = 0;

	virtual const TCHAR* GetImplementationName() const = 0;
};

class ID3D12BarriersForContext
{
public:
	virtual ~ID3D12BarriersForContext() = default;

	virtual void BeginTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) = 0;

	virtual void EndTransitions(
		FD3D12CommandContext& Context,
		TArrayView<const FRHITransition*> Transitions) = 0;

	virtual void AddGlobalBarrier(
		FD3D12ContextCommon& Context,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter) = 0;

	virtual void AddBarrier(
		FD3D12ContextCommon& Context,
		const FD3D12Resource* pResource,
		ED3D12Access D3D12AccessBefore,
		ED3D12Access D3D12AccessAfter,
		uint32 Subresource) = 0;

	virtual void FlushIntoCommandList(
		FD3D12CommandList& CommandList,
		FD3D12QueryAllocator& TimestampAllocator) = 0;

	virtual int32 GetNumPendingBarriers() const = 0;
};