// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12Adapter.h"

class FWindowsD3D12Adapter : public FD3D12Adapter
{
public:

	FWindowsD3D12Adapter(FD3D12AdapterDesc& InDesc)
		: FD3D12Adapter(InDesc)
	{}

protected:

	virtual void CreateCommandSignatures() final override;

};

