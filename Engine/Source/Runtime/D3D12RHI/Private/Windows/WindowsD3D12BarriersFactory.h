// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12BarriersFactory.h"
#include "D3D12EnhancedBarriers.h"
#include "D3D12LegacyBarriers.h"


using FD3D12BarriersFactory = TD3D12BarriersFactory<
#if D3D12RHI_SUPPORTS_ENHANCED_BARRIERS
	TD3D12BarriersFactoryEntry<
		ED3D12BarrierImplementationType::Enhanced,
		FD3D12EnhancedBarriersForAdapter,
		FD3D12EnhancedBarriersForContext>,
#endif
#if D3D12RHI_SUPPORTS_LEGACY_BARRIERS
	TD3D12BarriersFactoryEntry<
		ED3D12BarrierImplementationType::Legacy,
		FD3D12LegacyBarriersForAdapter,
		FD3D12LegacyBarriersForContext>,
#endif
	FNullD3D12BarriersFactoryEntry
>;