// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define D3D12RHI_CUSTOM_BINDLESS_RESOURCE_MANAGER 0
#define D3D12RHI_BINDLESS_RESOURCE_MANAGER_SUPPORTS_RESIZING 1
