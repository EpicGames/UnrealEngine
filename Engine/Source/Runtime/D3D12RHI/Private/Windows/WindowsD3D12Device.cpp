// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WindowsD3D12Device.cpp: Windows D3D device RHI implementation.
=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "D3D12AmdExtensions.h"
#include "D3D12IntelExtensions.h"
#include "D3D12NvidiaExtensions.h"
#include "WindowsD3D12Adapter.h"
#include "Modules/ModuleManager.h"
#include "Windows/WindowsPlatformCrashContext.h"
#include "HAL/FileManager.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#if WITH_ENGINE
#include "HardwareInfo.h"
#include "IHeadMountedDisplayModule.h"
#endif // WITH_ENGINE
#include "GenericPlatform/GenericPlatformDriver.h"			// FGPUDriverInfo
#include "RHIValidation.h"
#include "RHIUtilities.h"

#include "Windows/WindowsD3D.h"

IMPLEMENT_MODULE(FD3D12DynamicRHIModule, D3D12RHI);

extern bool D3D12RHI_ShouldCreateWithWarp();
extern bool D3D12RHI_AllowSoftwareFallback();
extern bool D3D12RHI_ShouldAllowAsyncResourceCreation();
extern bool D3D12RHI_ShouldForceCompatibility();

#if WITH_AMD_AGS
// Filled in during InitializeDeviceAgnosticVendorAPI if IsRHIDeviceAMD
struct AmdAgsInfo
{
	AGSContext* AmdAgsContext;
	AGSGPUInfo AmdGpuInfo;
};
static AmdAgsInfo AmdInfo;
#endif

#if INTEL_EXTENSIONS
bool GDX12INTCAtomicUInt64Emulation = false;
#endif

FD3D12DynamicRHI* GD3D12RHI = nullptr;

#if WITH_NVAPI
int32 GEnableNVAPIRaytracingValidation = 0;
static FAutoConsoleVariableRef CVarNVAPIRaytracingValidation(
	TEXT("r.D3D12.DXR.RaytracingValidation"),
	GEnableNVAPIRaytracingValidation,
	TEXT("Enables NVAPI Raytracing validation."),
	ECVF_ReadOnly
);
static TAutoConsoleVariable<FString> CVarNVAPIRaytracingFilterMessages(
	TEXT("r.D3D12.DXR.RaytracingValidation.IgnoreList"),
	"",
	TEXT("List of warnings or errors to ignore to prevent spam from NVAPI Raytracing validation. For instance: warning1;warning2 etc"),
	ECVF_ReadOnly);
#endif

int32 GMinimumWindowsBuildVersionForRayTracing = 0;
static FAutoConsoleVariableRef CVarMinBuildVersionForRayTracing(
	TEXT("r.D3D12.DXR.MinimumWindowsBuildVersion"),
	GMinimumWindowsBuildVersionForRayTracing,
	TEXT("Sets the minimum Windows build version required to enable ray tracing."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

FString GMinimumDriverVersionForRayTracingNVIDIA;
static FAutoConsoleVariableRef CVarMinDriverVersionForRayTracingNVIDIA(
	TEXT("r.D3D12.DXR.MinimumDriverVersionNVIDIA"),
	GMinimumDriverVersionForRayTracingNVIDIA,
	TEXT("Sets the minimum driver version required to enable ray tracing on NVIDIA GPUs."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

FString GMinimumDriverVersionForRayTracingAMD;
static FAutoConsoleVariableRef CVarMinDriverVersionForRayTracingAMD(
	TEXT("r.D3D12.DXR.MinimumDriverVersionAMD"),
	GMinimumDriverVersionForRayTracingAMD,
	TEXT("Sets the minimum driver version required to enable ray tracing on AMD GPUs."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

FString GMinimumDriverVersionForRayTracingIntel;
static FAutoConsoleVariableRef CVarMinDriverVersionForRayTracingIntel(
	TEXT("r.D3D12.DXR.MinimumDriverVersionIntel"),
	GMinimumDriverVersionForRayTracingIntel,
	TEXT("Sets the minimum driver version required to enable ray tracing on Intel GPUs."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarExperimentalShaderModels(
	TEXT("r.D3D12.ExperimentalShaderModels"),
	0,
	TEXT("Controls whether D3D12 experimental shader models should be allowed. Not available in shipping builds. (default = 0)."),
	ECVF_ReadOnly
);
#endif // !UE_BUILD_SHIPPING

// See https://microsoft.github.io/DirectX-Specs/d3d/BackgroundProcessing.html.
int32 GDevDisableD3DRuntimeBackgroundThreads = 0;
static FAutoConsoleVariableRef CVarDevDisableD3DRuntimeBackgroundThreads(
	TEXT("r.D3D12.DevDisableD3DRuntimeBackgroundThreads"),
	GDevDisableD3DRuntimeBackgroundThreads,
	TEXT("If > 0, disables the background threads created by the D3D runtime for background shader optimization. Only available when Windows developer mode is enabled. (default = 0)."),
	ECVF_ReadOnly
);

#if D3D12RHI_SUPPORTS_WIN_PIX
int32 GAutoAttachPIX = 0;
static FAutoConsoleVariableRef CVarAutoAttachPIX(
	TEXT("r.D3D12.AutoAttachPIX"),
	GAutoAttachPIX,
	TEXT("Automatically attach PIX on startup"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);
#endif // D3D12RHI_SUPPORTS_WIN_PIX

using namespace D3D12RHI;

static bool bIsQuadBufferStereoEnabled = false;

static D3D_FEATURE_LEVEL FindHighestFeatureLevel(ID3D12Device* Device, D3D_FEATURE_LEVEL MinFeatureLevel)
{
	const D3D_FEATURE_LEVEL FeatureLevels[] =
	{
		// Add new feature levels that the app supports here.
#if D3D12_CORE_ENABLED
		D3D_FEATURE_LEVEL_12_2,
#endif
		D3D_FEATURE_LEVEL_12_1,
		D3D_FEATURE_LEVEL_12_0,
		D3D_FEATURE_LEVEL_11_1,
		D3D_FEATURE_LEVEL_11_0
	};

	// Determine the max feature level supported by the driver and hardware.
	D3D12_FEATURE_DATA_FEATURE_LEVELS FeatureLevelCaps{};
	FeatureLevelCaps.pFeatureLevelsRequested = FeatureLevels;
	FeatureLevelCaps.NumFeatureLevels = UE_ARRAY_COUNT(FeatureLevels);

	if (SUCCEEDED(Device->CheckFeatureSupport(D3D12_FEATURE_FEATURE_LEVELS, &FeatureLevelCaps, sizeof(FeatureLevelCaps))))
	{
		return FeatureLevelCaps.MaxSupportedFeatureLevel;
	}

	return MinFeatureLevel;
}

static D3D_SHADER_MODEL FindHighestShaderModel(ID3D12Device* Device)
{
	// Because we can't guarantee older Windows versions will know about newer shader models, we need to check them all
	// in descending order and return the first result that succeeds.
	const D3D_SHADER_MODEL ShaderModelsToCheck[] =
	{
#if D3D12_CORE_ENABLED
		D3D_SHADER_MODEL_6_7,
		D3D_SHADER_MODEL_6_6,
#endif
		D3D_SHADER_MODEL_6_5,
		D3D_SHADER_MODEL_6_4,
		D3D_SHADER_MODEL_6_3,
		D3D_SHADER_MODEL_6_2,
		D3D_SHADER_MODEL_6_1,
		D3D_SHADER_MODEL_6_0,
	};

	D3D12_FEATURE_DATA_SHADER_MODEL FeatureShaderModel{};
	for (const D3D_SHADER_MODEL ShaderModelToCheck : ShaderModelsToCheck)
	{
		FeatureShaderModel.HighestShaderModel = ShaderModelToCheck;

		if (SUCCEEDED(Device->CheckFeatureSupport(D3D12_FEATURE_SHADER_MODEL, &FeatureShaderModel, sizeof(FeatureShaderModel))))
		{
			return FeatureShaderModel.HighestShaderModel;
		}
	}

	// Last ditch effort, the minimum requirement for DX12 is 5.1
	return D3D_SHADER_MODEL_5_1;
}

#if INTEL_EXTENSIONS
INTCExtensionContext* CreateIntelExtensionsContext(ID3D12Device* Device, INTCExtensionInfo& INTCExtensionInfo, uint32 DeviceId)
{
	if (FAILED(INTC_LoadExtensionsLibrary(false, (uint32)EGpuVendorId::Intel, DeviceId)))
	{
		UE_LOGF(LogD3D12RHI, Log, "Failed to load Intel Extensions Library!");
		return nullptr;
	}

	INTCExtensionVersion* SupportedExtensionsVersions = nullptr;
	uint32_t SupportedExtensionsVersionCount = 0;
	if( SUCCEEDED( INTC_D3D12_GetSupportedVersions( Device, nullptr, &SupportedExtensionsVersionCount ) ) )
	{
		SupportedExtensionsVersions = new INTCExtensionVersion[ SupportedExtensionsVersionCount ]{};

		if( SUCCEEDED( INTC_D3D12_GetSupportedVersions( Device, SupportedExtensionsVersions, &SupportedExtensionsVersionCount ) ) && SupportedExtensionsVersions != nullptr )
		{
			// We have the list of supported versions, now find the lowest common version that supports the required features	
			for( uint32_t i = 0; i < SupportedExtensionsVersionCount; i++ )
			{
				CA_SUPPRESS( 6385 );
				if( MatchIntelExtensionVersion( SupportedExtensionsVersions[ i ] ) )
				{
					INTCExtensionInfo.RequestedExtensionVersion = SupportedExtensionsVersions[ i ];
				}
			}
		}
	}

	// No need for this table anymore
	if (SupportedExtensionsVersions != nullptr)
	{
		delete[] SupportedExtensionsVersions;
	}

	// No version matched means we cannot use the Intel Extensions
	if (INTCExtensionInfo.RequestedExtensionVersion.HWFeatureLevel == 0 &&
		INTCExtensionInfo.RequestedExtensionVersion.APIVersion == 0 )
	{
		UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework not supported by driver. Please check if a driver update is available.");
		return nullptr;
	}

	INTCExtensionContext* IntelExtensionContext = nullptr;
	
	// Fill in registration information for this workload (App name and Engine name)
	INTCExtensionAppInfo1 AppInfo = GetIntelApplicationInfo();

	const HRESULT hr = INTC_D3D12_CreateDeviceExtensionContext1(Device, &IntelExtensionContext, &INTCExtensionInfo, &AppInfo);

	if (SUCCEEDED(hr))
	{
		UE_LOGF( LogD3D12RHI, Log, "Intel Extensions Framework enabled (version: %u.%u.%u).",
			INTCExtensionInfo.RequestedExtensionVersion.HWFeatureLevel,
			INTCExtensionInfo.RequestedExtensionVersion.APIVersion,
			INTCExtensionInfo.RequestedExtensionVersion.Revision );
	}
	else
	{
		if (hr == E_NOINTERFACE)
		{
			UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework not supported by driver. Please check if a driver update is available.");
		}
		else if (hr == E_INVALIDARG)
		{
			UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework passed invalid creation arguments.");
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework failed to initialize. Error code: 0x%08x. Please check if a driver update is available.", hr);
		}

		if (IntelExtensionContext)
		{
			DestroyIntelExtensionsContext(IntelExtensionContext);
			IntelExtensionContext = nullptr;
		}
	}

	return IntelExtensionContext;
}

void DestroyIntelExtensionsContext(INTCExtensionContext* IntelExtensionContext)
{
	if (IntelExtensionContext)
	{
		const HRESULT hr = INTC_DestroyDeviceExtensionContext(&IntelExtensionContext);
		if (SUCCEEDED(hr))
		{
			UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework unloaded");
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Log, "Intel Extensions Framework error when unloading: 0x%08x", hr);
		}
	}
}

bool EnableIntelAtomic64Support(INTCExtensionContext* IntelExtensionContext, INTCExtensionInfo& INTCExtensionInfo)
{
	if (IntelExtensionContext)
	{
		INTC_D3D12_FEATURE_DATA_D3D12_OPTIONS1 INTCFeatureSupportData;
		const HRESULT hrCheck = INTC_D3D12_CheckFeatureSupport(IntelExtensionContext, INTC_D3D12_FEATURE_D3D12_OPTIONS1, &INTCFeatureSupportData, sizeof(INTCFeatureSupportData));

		if (SUCCEEDED(hrCheck))
		{
			if (INTCFeatureSupportData.EmulatedTyped64bitAtomics)
			{
				INTC_D3D12_FEATURE INTCFeature;
				INTCFeature.EmulatedTyped64bitAtomics = true;

				const HRESULT hrSet = INTC_D3D12_SetFeatureSupport(IntelExtensionContext, &INTCFeature);
				if (SUCCEEDED(hrSet))
				{
					GDX12INTCAtomicUInt64Emulation = true;
					UE_LOGF(LogD3D12RHI, Log, "Intel Extensions 64-bit Typed Atomics emulation enabled.");
				}
				else
				{
					UE_LOGF(LogD3D12RHI, Log, "Failed to enable Intel Extensions 64-bit Typed Atomics emulation, error code: 0x%08x.", hrSet);
				}
			}
			else
			{
				UE_LOGF(LogD3D12RHI, Log, "Intel Extensions 64-bit Typed Atomics emulation not needed.");
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Log, "Failed to check for Intel Extensions 64-bit Typed Atomics emulation, error code: 0x%08x.", hrCheck);
		}
	}

	return GDX12INTCAtomicUInt64Emulation;
}

bool TryCheckIntelAsyncCompute(ID3D12Device* Device, uint32 DeviceId, bool& bOutSupported)
{
	bool bSuccess = false;

	if (UE::RHICore::AllowVendorDevice())
	{
		INTCExtensionInfo INTCExtensionInfo{};
		if (INTCExtensionContext* IntelExtensionContext = CreateIntelExtensionsContext(Device, INTCExtensionInfo, DeviceId))
		{
			INTC_D3D12_FEATURE_DATA_D3D12_OPTIONS3 INTCFeatureSupportData;
			const HRESULT hr = INTC_D3D12_CheckFeatureSupport(IntelExtensionContext, INTC_D3D12_FEATURE_D3D12_OPTIONS3, &INTCFeatureSupportData, sizeof(INTCFeatureSupportData));
			bSuccess = SUCCEEDED(hr);
			if (bSuccess)
			{
				UE_LOGF(LogD3D12RHI, Log, "Intel Extensions AsyncCompute=%d.", INTCFeatureSupportData.AsyncCompute);
				bOutSupported = INTCFeatureSupportData.AsyncCompute != 0;
			}
			else
			{
				UE_LOGF(LogD3D12RHI, Log, "Failed to check for Intel Extensions AsyncCompute: 0x%08X", hr);
			}
			DestroyIntelExtensionsContext(IntelExtensionContext);
		}
	}

	return bSuccess;
}
#endif // INTEL_EXTENSIONS

static bool CheckDeviceForEmulatedAtomic64Support(IDXGIAdapter* Adapter, ID3D12Device* Device)
{
	bool bEmulatedAtomic64Support = false;

#if INTEL_EXTENSIONS
	DXGI_ADAPTER_DESC AdapterDesc{};
	Adapter->GetDesc(&AdapterDesc);

	if ((RHIConvertToGpuVendorId(AdapterDesc.VendorId) == EGpuVendorId::Intel) && UE::RHICore::AllowVendorDevice())
	{
		INTCExtensionInfo INTCExtensionInfo{};
		if (INTCExtensionContext* IntelExtensionContext = CreateIntelExtensionsContext(Device, INTCExtensionInfo, AdapterDesc.DeviceId))
		{
			INTC_D3D12_FEATURE_DATA_D3D12_OPTIONS1 INTCFeatureSupportData;
			const HRESULT hr = INTC_D3D12_CheckFeatureSupport(IntelExtensionContext, INTC_D3D12_FEATURE_D3D12_OPTIONS1, &INTCFeatureSupportData, sizeof(INTCFeatureSupportData));
			if (SUCCEEDED(hr))
			{
				bEmulatedAtomic64Support = INTCFeatureSupportData.EmulatedTyped64bitAtomics;
			}
			else
			{
				UE_LOGF(LogD3D12RHI, Log, "Failed to check for Intel Extensions 64-bit Typed Atomics emulation.");
			}
			DestroyIntelExtensionsContext(IntelExtensionContext);
		}
	}
#endif

	return bEmulatedAtomic64Support;
}

inline bool ShouldCheckBindlessSupport(EShaderPlatform ShaderPlatform)
{
	// Note: only checking against All allows the RayTracingOnly configuration to not raise the requirements for projects that aren't using RayTracing.
	return UE::RHICore::GetBindlessConfigurationOnStartup(ShaderPlatform) > ERHIBindlessConfiguration::RayTracing;
}

inline ERHIFeatureLevel::Type FindMaxRHIFeatureLevel(D3D_FEATURE_LEVEL InMaxFeatureLevel, D3D_SHADER_MODEL InMaxShaderModel, D3D12_RESOURCE_BINDING_TIER ResourceBindingTier, bool bSupportsWaveOps, bool bSupportsAtomic64)
{
	ERHIFeatureLevel::Type MaxRHIFeatureLevel = ERHIFeatureLevel::Num;

	if (InMaxFeatureLevel >= D3D_FEATURE_LEVEL_12_0 && InMaxShaderModel >= D3D_SHADER_MODEL_6_6)
	{
		bool bHighEnoughBindingTier = true;
		if (ShouldCheckBindlessSupport(SP_PCD3D_SM6))
		{
			bHighEnoughBindingTier = ResourceBindingTier >= D3D12_RESOURCE_BINDING_TIER_3;
		}
		else
		{
			bHighEnoughBindingTier = ResourceBindingTier >= D3D12_RESOURCE_BINDING_TIER_2;
		}

		if (bSupportsWaveOps && bHighEnoughBindingTier && bSupportsAtomic64)
		{
			if (FParse::Param(FCommandLine::Get(), TEXT("ForceDisableSM6")))
			{
				UE_LOGF(LogD3D12RHI, Log, "ERHIFeatureLevel::SM6 disabled via -ForceDisableSM6");
			}
			else
			{
				MaxRHIFeatureLevel = ERHIFeatureLevel::SM6;
			}
		}
	}

	if (MaxRHIFeatureLevel == ERHIFeatureLevel::Num && InMaxFeatureLevel >= D3D_FEATURE_LEVEL_11_0)
	{
		MaxRHIFeatureLevel = ERHIFeatureLevel::SM5;
	}

	return MaxRHIFeatureLevel;
}

inline void GetResourceTiers(ID3D12Device* Device, D3D12_RESOURCE_BINDING_TIER& OutResourceBindingTier, D3D12_RESOURCE_HEAP_TIER& OutResourceHeapTier)
{
	D3D12_FEATURE_DATA_D3D12_OPTIONS D3D12Caps{};
	Device->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS, &D3D12Caps, sizeof(D3D12Caps));

	OutResourceBindingTier = D3D12Caps.ResourceBindingTier;
	OutResourceHeapTier = D3D12Caps.ResourceHeapTier;
}

inline bool GetSupportsWaveOps(ID3D12Device* Device)
{
	D3D12_FEATURE_DATA_D3D12_OPTIONS1 D3D12Caps1{};
	Device->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS1, &D3D12Caps1, sizeof(D3D12Caps1));

	return D3D12Caps1.WaveOps;
}

inline bool GetSupportsAtomic64(IDXGIAdapter* Adapter, ID3D12Device* Device)
{
	D3D12_FEATURE_DATA_D3D12_OPTIONS9 D3D12Caps9{};
	Device->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS9, &D3D12Caps9, sizeof(D3D12Caps9));

	return (D3D12Caps9.AtomicInt64OnTypedResourceSupported || CheckDeviceForEmulatedAtomic64Support(Adapter, Device));
}

inline bool GetDeviceArchitectureInformation(ID3D12Device* device, bool& OutTileBasedRenderer, bool& OutUMA, bool& OutCacheCoherentUMA)
{
	D3D12_FEATURE_DATA_ARCHITECTURE data = { 0 };
	if (SUCCEEDED(device->CheckFeatureSupport(D3D12_FEATURE_ARCHITECTURE, &data, sizeof(D3D12_FEATURE_DATA_ARCHITECTURE))))
	{
		OutTileBasedRenderer = (bool)(data.TileBasedRenderer);
		OutUMA = (bool)(data.UMA);
		OutCacheCoherentUMA = (bool)(data.CacheCoherentUMA);
		return true;
	}
	return false;
}

inline bool GetSupportGPUUploadHeaps(ID3D12Device* device)
{
	D3D12_FEATURE_DATA_D3D12_OPTIONS16 options16 = {};
	
	if (SUCCEEDED(device->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS16, &options16, sizeof(options16))))
	{
		return (bool)(options16.GPUUploadHeapSupported);
	}

	return false;
}

/**
 * Attempts to create a D3D12 device for the adapter or find an existing one.
 * If creation is successful, true is returned.
 */
static bool SafeTestD3D12CreateDevice(IDXGIAdapter* Adapter, const DXGI_ADAPTER_DESC& AdapterDesc, FD3D12DeviceBasicInfo& OutInfo)
{
	constexpr D3D_FEATURE_LEVEL MinFeatureLevel = D3D_FEATURE_LEVEL_11_0;

	const uint64 Luid = BitCast<uint64>(AdapterDesc.AdapterLuid);
	ID3D12Device* Device = FWindowsD3D::FindCachedD3D12AdapterDevice(Luid);
	if (!Device)
	{
		TRefCountPtr<ID3D12Device> DevicePtr;
		const HRESULT D3D12CreateDeviceResult = D3D12CreateDevice(Adapter, MinFeatureLevel, IID_PPV_ARGS(DevicePtr.GetInitReference()));
		if (FAILED(D3D12CreateDeviceResult))
		{
			UE_LOGF(LogD3D12RHI, Log, "D3D12CreateDevice failed with code 0x%08X", static_cast<int32>(D3D12CreateDeviceResult));
			return false;
		}

		Device = DevicePtr.GetReference();
		FWindowsD3D::CacheD3D12AdapterDevice(Luid, MoveTemp(DevicePtr));
	}

	OutInfo.MaxFeatureLevel = FindHighestFeatureLevel(Device, MinFeatureLevel);
	OutInfo.MaxShaderModel = FindHighestShaderModel(Device);
	GetResourceTiers(Device, OutInfo.ResourceBindingTier, OutInfo.ResourceHeapTier);
	OutInfo.NumDeviceNodes = Device->GetNodeCount();

	OutInfo.bSupportsWaveOps = GetSupportsWaveOps(Device);
	OutInfo.bSupportsAtomic64 = GetSupportsAtomic64(Adapter, Device);

	OutInfo.MaxRHIFeatureLevel = FindMaxRHIFeatureLevel(OutInfo.MaxFeatureLevel, OutInfo.MaxShaderModel, OutInfo.ResourceBindingTier, OutInfo.bSupportsWaveOps, OutInfo.bSupportsAtomic64);

	if (!GetDeviceArchitectureInformation(Device, OutInfo.bTileBasedRenderer, OutInfo.bUMA, OutInfo.bCacheCoherentUMA))
	{
		OutInfo.bTileBasedRenderer = OutInfo.bUMA = OutInfo.bCacheCoherentUMA = false;
	}

	OutInfo.bSupportsGPUUploadHeaps = GetSupportGPUUploadHeaps(Device);

	return true;
}

static bool SupportsDepthBoundsTest(FD3D12DynamicRHI* D3DRHI)
{
	// Determines if the primary adapter supports depth bounds test
	check(D3DRHI && D3DRHI->HasAdapter());

	return D3DRHI->GetAdapter().IsDepthBoundsTestSupported();
}

static UE::Color::FColorSpace MakeColorSpaceFromDXGI(const DXGI_OUTPUT_DESC1& OutputDesc)
{
	return UE::Color::FColorSpace(
		FVector2d(OutputDesc.RedPrimary[0], OutputDesc.RedPrimary[1]),
		FVector2d(OutputDesc.GreenPrimary[0], OutputDesc.GreenPrimary[1]),
		FVector2d(OutputDesc.BluePrimary[0], OutputDesc.BluePrimary[1]),
		FVector2d(OutputDesc.WhitePoint[0], OutputDesc.WhitePoint[1]));
}

bool FD3D12DynamicRHI::SetupDisplayHDRMetaData()
{
	// Determines if any displays support HDR
	check(HasAdapter());

	TArray<TRefCountPtr<IDXGIAdapter> > DXGIAdapters;

	FD3D12Adapter& CurrentAdapter = *ChosenAdapter.Get();
	DXGIAdapters.Add(CurrentAdapter.GetAdapter());
	const bool bStaleAdapters = CurrentAdapter.GetDXGIFactory2() && !CurrentAdapter.GetDXGIFactory2()->IsCurrent();

#if PLATFORM_WINDOWS
	// if we found that the list of adapters is stale (changed windows HDR setting), try to update it with the new list
	if (bStaleAdapters)
	{
		if (!DXGIFactoryForDisplayList.IsValid() || !DXGIFactoryForDisplayList->IsCurrent())
		{
			FD3D12Adapter::CreateDXGIFactory(DXGIFactoryForDisplayList, false);
		}

		if (DXGIFactoryForDisplayList.IsValid() && DXGIFactoryForDisplayList->IsCurrent())
		{
			DXGIAdapters.Empty();
			TRefCountPtr<IDXGIAdapter> TempAdapter;
			for (uint32 AdapterIndex = 0; DXGIFactoryForDisplayList->EnumAdapters(AdapterIndex, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
			{
				DXGIAdapters.Add(TempAdapter);
			}
		}
	}
#endif

	DisplayList.Empty();

	bool bSupportsHDROutput = false;
	const int32 NumDXGIAdapters = DXGIAdapters.Num();
	for (int32 AdapterIndex = 0; AdapterIndex < NumDXGIAdapters; ++AdapterIndex)
	{
		IDXGIAdapter* DXGIAdapter = DXGIAdapters[AdapterIndex];

		for (uint32 DisplayIndex = 0; true; ++DisplayIndex)
		{
			TRefCountPtr<IDXGIOutput> DXGIOutput;
			if (S_OK != DXGIAdapter->EnumOutputs(DisplayIndex, DXGIOutput.GetInitReference()))
			{
				break;
			}

			TRefCountPtr<IDXGIOutput6> Output6;
			if (SUCCEEDED(DXGIOutput->QueryInterface(IID_PPV_ARGS(Output6.GetInitReference()))))
			{
				DXGI_OUTPUT_DESC1 OutputDesc;
				VERIFYD3D12RESULT(Output6->GetDesc1(&OutputDesc));

				// Check for HDR support on the display.
				const bool bDisplaySupportsHDROutput = (OutputDesc.ColorSpace == DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020);
				if (bDisplaySupportsHDROutput)
				{
					UE_LOGF(LogD3D12RHI, Log, "HDR output is supported on adapter %i, display %u:", AdapterIndex, DisplayIndex);
					UE_LOGF(LogD3D12RHI, Log, "\t\tMinLuminance = %f", OutputDesc.MinLuminance);
					UE_LOGF(LogD3D12RHI, Log, "\t\tMaxLuminance = %f", OutputDesc.MaxLuminance);
					UE_LOGF(LogD3D12RHI, Log, "\t\tMaxFullFrameLuminance = %f", OutputDesc.MaxFullFrameLuminance);

					bSupportsHDROutput = true;
				}

				FDisplayInformation& DisplayInformation = DisplayList.Emplace_GetRef();
				DisplayInformation.bHDRSupported = bDisplaySupportsHDROutput;
				const RECT& DisplayCoords = OutputDesc.DesktopCoordinates;
				DisplayInformation.DesktopCoordinates = FIntRect(DisplayCoords.left, DisplayCoords.top, DisplayCoords.right, DisplayCoords.bottom);
				DisplayInformation.MinimumLuminanceInNits = OutputDesc.MinLuminance;
				DisplayInformation.MaximumLuminanceInNits = OutputDesc.MaxLuminance ?
					OutputDesc.MaxLuminance :
					(bDisplaySupportsHDROutput ? 1000.f : 100.f);
				DisplayInformation.MaximumFullFrameLuminanceInNits = OutputDesc.MaxFullFrameLuminance ?
					OutputDesc.MaxFullFrameLuminance :
					(bDisplaySupportsHDROutput ? 600.f : 100.f);
				const bool bIsGamutValid = OutputDesc.RedPrimary[0] && OutputDesc.RedPrimary[1] &&
					OutputDesc.GreenPrimary[0] && OutputDesc.GreenPrimary[1] &&
					OutputDesc.BluePrimary[0] && OutputDesc.BluePrimary[1] &&
					OutputDesc.WhitePoint[0] && OutputDesc.WhitePoint[1];
				DisplayInformation.LimitingColorSpace = bIsGamutValid ?
					MakeColorSpaceFromDXGI(OutputDesc) :
					(bDisplaySupportsHDROutput ?
						UE::Color::FColorSpace::GetP3D65() :
						UE::Color::FColorSpace::GetSRGB());
			}
		}
	}

	return bSupportsHDROutput;
}

#if PLATFORM_WINDOWS
void FD3D12DynamicRHI::RHIHandleDisplayChange()
{
	RHIBlockUntilGPUIdle();
	GRHISupportsHDROutput = SetupDisplayHDRMetaData();
#if WITH_ENGINE
	// make sure CVars are being updated properly
	extern void HDRSettingChangedSinkCallback();
	HDRSettingChangedSinkCallback();
#endif
}
#endif

static bool IsAdapterBlocked(const FD3D12Adapter* InAdapter)
{
#if !UE_BUILD_SHIPPING
	if (InAdapter)
	{
		FString BlockedIHVString;
		if (GConfig->GetString(TEXT("SystemSettings"), TEXT("RHI.BlockIHVD3D12"), BlockedIHVString, GEngineIni))
		{
			TArray<FString> BlockedIHVs;
			BlockedIHVString.ParseIntoArray(BlockedIHVs, TEXT(","));

			const TCHAR* VendorId = RHIVendorIdToString(EGpuVendorId(InAdapter->GetD3DAdapterDesc().VendorId));
			for (const FString& BlockedVendor : BlockedIHVs)
			{
				if (BlockedVendor.Equals(VendorId, ESearchCase::IgnoreCase))
				{
					return true;
				}
			}
		}
	}
#endif

	return false;
}

inline ERHIFeatureLevel::Type GetAdapterMaxFeatureLevel(const FD3D12Adapter* InAdapter)
{
	if (InAdapter)
	{
		if (const FD3D12AdapterDesc& Desc = InAdapter->GetDesc(); Desc.IsValid())
		{
			return Desc.MaxRHIFeatureLevel;
		}
	}

	return ERHIFeatureLevel::Num;
}

static bool IsAdapterSupported(const FD3D12Adapter* InAdapter, ERHIFeatureLevel::Type InRequestedFeatureLevel)
{
	const ERHIFeatureLevel::Type AdapterMaxFeatureLevel = GetAdapterMaxFeatureLevel(InAdapter);
	return AdapterMaxFeatureLevel != ERHIFeatureLevel::Num && AdapterMaxFeatureLevel >= InRequestedFeatureLevel;
}

#if D3D12_CORE_ENABLED
static bool CheckIfAgilitySDKLoaded()
{
	const TCHAR* AgilitySDKDllName = TEXT("D3D12Core.dll");
	HMODULE AgilitySDKDllHandle = ::GetModuleHandle(AgilitySDKDllName);
	return AgilitySDKDllHandle != NULL;
}
#endif

bool FD3D12DynamicRHIModule::IsSupported(ERHIFeatureLevel::Type RequestedFeatureLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12DynamicRHIModule::IsSupported);

	// Windows version 15063 is Windows 1703 aka "Windows Creator Update"
	// This is the first version that supports ID3D12Device2 which is our minimum runtime device version.
	if (!FPlatformMisc::VerifyWindowsVersion(10, 0, 15063))
	{
		UE_LOGF(LogD3D12RHI, Warning, "Missing full support for Direct3D 12. Update to Windows 1703 or newer for D3D12 support.");
		return false;
	}

	// If not computed yet
	if (!ChosenAdapter.IsValid())
	{
		FindAdapter();
	}

	if (!ChosenAdapter.IsValid())
	{
		UE_LOGF(LogD3D12RHI, Log, "No adapters were found.");
		return false;
	}

	const FD3D12Adapter* const Adapter = ChosenAdapter.Get();
	if (!Adapter || !Adapter->GetDesc().IsValid())
	{
		UE_LOGF(LogD3D12RHI, Log, "Adapter was not found");
		return false;
	}

	if (IsAdapterBlocked(Adapter))
	{
		UE_LOGF(LogD3D12RHI, Log, "Adapter was blocked by RHI.BlockIHVD3D12");
		return false;
	}

	if (!IsAdapterSupported(Adapter, RequestedFeatureLevel))
	{
		auto GetFeatureLevelNameInline = [](ERHIFeatureLevel::Type InFeatureLevel)
		{
			FString FeatureLevelName;
			::GetFeatureLevelName(InFeatureLevel, FeatureLevelName);
			return FeatureLevelName;
		};

		const FString SupportedFeatureLevelName = GetFeatureLevelNameInline(GetAdapterMaxFeatureLevel(Adapter));
		const FString RequestedFeatureLevelName = GetFeatureLevelNameInline(RequestedFeatureLevel);

		UE_LOGF(LogD3D12RHI, Log,
			"Adapter only supports up to Feature Level '%ls', requested Feature Level was '%ls'",
			*SupportedFeatureLevelName,
			*RequestedFeatureLevelName
		);

		return false;
	}

	return true;
}

namespace D3D12RHI
{
	const TCHAR* GetFeatureLevelString(D3D_FEATURE_LEVEL FeatureLevel)
	{
		switch (FeatureLevel)
		{
		case D3D_FEATURE_LEVEL_9_1:		return TEXT("9_1");
		case D3D_FEATURE_LEVEL_9_2:		return TEXT("9_2");
		case D3D_FEATURE_LEVEL_9_3:		return TEXT("9_3");
		case D3D_FEATURE_LEVEL_10_0:	return TEXT("10_0");
		case D3D_FEATURE_LEVEL_10_1:	return TEXT("10_1");
		case D3D_FEATURE_LEVEL_11_0:	return TEXT("11_0");
		case D3D_FEATURE_LEVEL_11_1:	return TEXT("11_1");
		case D3D_FEATURE_LEVEL_12_0:	return TEXT("12_0");
		case D3D_FEATURE_LEVEL_12_1:	return TEXT("12_1");
#if D3D12_CORE_ENABLED
		case D3D_FEATURE_LEVEL_12_2:	return TEXT("12_2");
#endif
		}
		return TEXT("X_X");
	}
}

static uint32 CountAdapterOutputs(TRefCountPtr<IDXGIAdapter>& Adapter)
{
	uint32 OutputCount = 0;
	for (;;)
	{
		TRefCountPtr<IDXGIOutput> Output;
		HRESULT hr = Adapter->EnumOutputs(OutputCount, Output.GetInitReference());
		if (FAILED(hr))
		{
			break;
		}
		++OutputCount;
	}

	return OutputCount;
}

void FD3D12DynamicRHIModule::FindAdapter()
{
	// Once we've chosen one we don't need to do it again.
	check(!ChosenAdapter.IsValid());

#if !UE_BUILD_SHIPPING
	if (CVarExperimentalShaderModels.GetValueOnAnyThread() == 1)
	{
		// Experimental features must be enabled before doing anything else with D3D.

		UUID ExperimentalFeatures[] =
		{
			D3D12ExperimentalShaderModels
		};
		HRESULT hr = D3D12EnableExperimentalFeatures(UE_ARRAY_COUNT(ExperimentalFeatures), ExperimentalFeatures, nullptr, nullptr);
		if (SUCCEEDED(hr))
		{
			UE_LOGF(LogD3D12RHI, Log, "D3D12 experimental shader models enabled");
		}
	}
#endif

	// Try to create the DXGIFactory.
	TRefCountPtr<IDXGIFactory4> DXGIFactory4;
#if !defined(D3D12_CUSTOM_VIEWPORT_CONSTRUCTOR) || !D3D12_CUSTOM_VIEWPORT_CONSTRUCTOR
	bIsQuadBufferStereoEnabled = FParse::Param(FCommandLine::Get(), TEXT("quad_buffer_stereo"));
	verify(SUCCEEDED(CreateDXGIFactory1(IID_PPV_ARGS(DXGIFactory4.GetInitReference()))));
#endif
	if (!DXGIFactory4)
	{
		return;
	}

	TRefCountPtr<IDXGIFactory6> DXGIFactory6;
	DXGIFactory4->QueryInterface(IID_PPV_ARGS(DXGIFactory6.GetInitReference()));

	// Allow HMD to override which graphics adapter is chosen, so we pick the adapter where the HMD is connected
#if WITH_ENGINE
	const uint64 HmdGraphicsAdapterLuid = IHeadMountedDisplayModule::IsAvailable() ? IHeadMountedDisplayModule::Get().GetGraphicsAdapterLuid() : 0;
#else
	const uint64 HmdGraphicsAdapterLuid = 0;
#endif
	const uint64 AdapterLuid = FWindowsD3D::ChooseD3D12Adapter(HmdGraphicsAdapterLuid);

	TRefCountPtr<IDXGIAdapter> TempAdapter;

	int GpuPreferenceInt = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE;
	FParse::Value(FCommandLine::Get(), TEXT("-gpupreference="), GpuPreferenceInt);
	DXGI_GPU_PREFERENCE GpuPreference;
	switch(GpuPreferenceInt)
	{
	case 1: GpuPreference = DXGI_GPU_PREFERENCE_MINIMUM_POWER; break;
	case 2: GpuPreference = DXGI_GPU_PREFERENCE_HIGH_PERFORMANCE; break;
	default: GpuPreference = DXGI_GPU_PREFERENCE_UNSPECIFIED; break;
	}

	// Enumerate the DXGIFactory's adapters.
	for (uint32 AdapterIndex = 0; FD3D12AdapterDesc::EnumAdapters(AdapterIndex, GpuPreference, DXGIFactory4, DXGIFactory6, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
	{
		// Check that if adapter supports D3D12.
		if (TempAdapter)
		{
			FD3D12DeviceBasicInfo DeviceInfo;
			DXGI_ADAPTER_DESC AdapterDesc{};
			VERIFYD3D12RESULT(TempAdapter->GetDesc(&AdapterDesc));

			if (SafeTestD3D12CreateDevice(TempAdapter, AdapterDesc, DeviceInfo))
			{
				check(DeviceInfo.NumDeviceNodes > 0);

				const uint32 OutputCount = CountAdapterOutputs(TempAdapter);

				UE_LOGF(LogD3D12RHI, Log,
					"Found D3D12 adapter %u: %ls (VendorId: %04x, DeviceId: %04x, SubSysId: %04x, Revision: %04x",
					AdapterIndex,
					AdapterDesc.Description,
					AdapterDesc.VendorId, AdapterDesc.DeviceId, AdapterDesc.SubSysId, AdapterDesc.Revision
				);
				UE_LOGF(LogD3D12RHI, Log,
					"  Max supported Feature Level %ls, shader model %d.%d, binding tier %d, wave ops %ls, atomic64 %ls",
					GetFeatureLevelString(DeviceInfo.MaxFeatureLevel),
					(DeviceInfo.MaxShaderModel >> 4), (DeviceInfo.MaxShaderModel & 0xF),
					DeviceInfo.ResourceBindingTier,
					DeviceInfo.bSupportsWaveOps ? TEXT("supported") : TEXT("unsupported"),
					DeviceInfo.bSupportsAtomic64 ? TEXT("supported") : TEXT("unsupported")
				);

				UE_LOGF(LogD3D12RHI, Log,
					"  Adapter has %uMB of dedicated video memory, %uMB of dedicated system memory, and %uMB of shared system memory, %d output[s], TileBased:%ls UMA:%ls CacheCoherentUMA:%ls GPU Upload Heaps:%ls",
					(uint32)(AdapterDesc.DedicatedVideoMemory / (1024 * 1024)),
					(uint32)(AdapterDesc.DedicatedSystemMemory / (1024 * 1024)),
					(uint32)(AdapterDesc.SharedSystemMemory / (1024 * 1024)),
					OutputCount,
					DeviceInfo.bTileBasedRenderer ? TEXT("true") : TEXT("false"),
					DeviceInfo.bUMA ? TEXT("true") : TEXT("false"),
					DeviceInfo.bCacheCoherentUMA ? TEXT("true") : TEXT("false"),
					DeviceInfo.bSupportsGPUUploadHeaps ? TEXT("true") : TEXT("false")
				);

				const bool bIsWARP = (RHIConvertToGpuVendorId(AdapterDesc.VendorId) == EGpuVendorId::Microsoft);

				if (!bIsWARP)
				{
					const FGPUDriverInfo GPUDriverInfo = FPlatformMisc::GetGPUDriverInfo(AdapterDesc.Description, false);
					UE_LOGF(LogD3D12RHI, Log, "  Driver Version: %ls (internal:%ls, unified:%ls)", *GPUDriverInfo.UserDriverVersion, *GPUDriverInfo.InternalDriverVersion, *GPUDriverInfo.GetUnifiedDriverVersion());
					UE_LOGF(LogD3D12RHI, Log, "     Driver Date: %ls", *GPUDriverInfo.DriverDate);
				}
				else
				{
					const TCHAR* WarpDllName = TEXT("d3d10warp.dll");
					void* WarpDllHandle = FPlatformProcess::GetDllHandle(WarpDllName);
					if(WarpDllHandle)
					{
						const uint64 WarpVersion = FPlatformMisc::GetFileVersion(WarpDllName);
						const uint64 Major = ((WarpVersion >> 48) & 0xffff);
						const uint64 Minor = ((WarpVersion >> 32) & 0xffff);
						const uint64 Build = ((WarpVersion >> 16) & 0xffff);
						const uint64 Revision = (WarpVersion & 0xffff);
						UE_LOGF(LogD3D12RHI, Log, "  D3D10Warp Version: %lld.%lld.%lld.%lld", Major, Minor, Build, Revision);
					}
				}

				FD3D12AdapterDesc CurrentAdapter(AdapterDesc, AdapterIndex, DeviceInfo);

				CurrentAdapter.NumDeviceNodes = DeviceInfo.NumDeviceNodes;
				CurrentAdapter.GpuPreference = GpuPreference;

				if (!FMemory::Memcmp(&AdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)))
				{
					ChosenAdapter = MakeShareable(new FWindowsD3D12Adapter(CurrentAdapter));
					// Don't break here so we can log all adapters.
				}
			}
		}
	}

#if D3D12_CORE_ENABLED
	UE_LOGF(LogD3D12RHI, Log, "DirectX Agility SDK runtime %ls.", CheckIfAgilitySDKLoaded() ? TEXT("found") : TEXT("not found"));
#endif

	if (ChosenAdapter.IsValid() && ChosenAdapter->GetDesc().IsValid())
	{
		UE_LOGF(LogD3D12RHI, Log, "Chosen D3D12 Adapter Id = %u", ChosenAdapter->GetAdapterIndex());

		const DXGI_ADAPTER_DESC& AdapterDesc = ChosenAdapter->GetD3DAdapterDesc();
		GRHIVendorId = AdapterDesc.VendorId;
		GRHIAdapterName = AdapterDesc.Description;
		GRHIDeviceId = AdapterDesc.DeviceId;
		GRHIDeviceRevision = AdapterDesc.Revision;
		GRHIGlobals.GpuInfo.DedicatedVideoMemory = AdapterDesc.DedicatedVideoMemory;
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Error, "Failed to choose a D3D12 Adapter.");
	}
}

FDynamicRHI* FD3D12DynamicRHIModule::CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel)
{
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_PCD3D_ES3_1;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_PCD3D_SM5;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_NumPlatforms;
	if (IsAdapterSupported(ChosenAdapter.Get(), ERHIFeatureLevel::SM6))
	{
		GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_PCD3D_SM6;
	}

	ERHIFeatureLevel::Type PreviewFeatureLevel;
	if (!GIsEditor && RHIGetPreviewFeatureLevel(PreviewFeatureLevel))
	{
		check(PreviewFeatureLevel == ERHIFeatureLevel::ES3_1);

		// ES3.1 feature level emulation in D3D
		GMaxRHIFeatureLevel = PreviewFeatureLevel;
	}
	else
	{
		GMaxRHIFeatureLevel = RequestedFeatureLevel;
	}

	if (!ensure(GMaxRHIFeatureLevel < ERHIFeatureLevel::Num))
	{
		GMaxRHIFeatureLevel = ERHIFeatureLevel::SM5;
	}

	GMaxRHIShaderPlatform = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	check(GMaxRHIShaderPlatform != SP_NumPlatforms);

#if D3D12RHI_SUPPORTS_WIN_PIX
	bool bPixEventEnabled = (WindowsPixDllHandle != nullptr);
#else
	bool bPixEventEnabled = false;
#endif // USE_PIX
	
	if (ChosenAdapter.IsValid())
	{
		const FD3D12AdapterDesc& Desc = ChosenAdapter->GetDesc();
		const bool bIsIntegrated = Desc.bUMA;
		FGenericCrashContext::SetEngineData(TEXT("RHI.IntegratedGPU"), bIsIntegrated ? TEXT("true") : TEXT("false"));
		GRHIDeviceIsIntegrated = bIsIntegrated;
		GRHIDeviceIsUMASeparateVA = Desc.bUMA;
		GRHIDeviceIsCacheCoherentUMA = Desc.bCacheCoherentUMA;
		GRHIDeviceSupportsGPUUploadHeaps = Desc.bSupportsGPUUploadHeaps;
		UE_LOGF(LogD3D12RHI, Log, "Integrated GPU (iGPU): %ls", GRHIDeviceIsIntegrated ? TEXT("true") : TEXT("false"));
	}

	const FString FeatureLevelString = LexToString(GMaxRHIFeatureLevel);
	UE_LOGF(LogD3D12RHI, Display, "Creating D3D12 RHI with Max Feature Level %ls", *FeatureLevelString);

	GD3D12RHI = new FD3D12DynamicRHI(ChosenAdapter, bPixEventEnabled);
#if ENABLE_RHI_VALIDATION
	if (FParse::Param(FCommandLine::Get(), TEXT("RHIValidation")))
	{
		return new FValidationRHI(GD3D12RHI);
	}
#endif

	for (int32 Index = 0; Index < ERHIFeatureLevel::Num; ++Index)
	{
		if (GShaderPlatformForFeatureLevel[Index] != SP_NumPlatforms)
		{
			check(GMaxTextureSamplers >= (int32)FDataDrivenShaderPlatformInfo::GetMaxSamplers(GShaderPlatformForFeatureLevel[Index]));
			if (GMaxTextureSamplers < (int32)FDataDrivenShaderPlatformInfo::GetMaxSamplers(GShaderPlatformForFeatureLevel[Index]))
			{
				UE_LOGF(LogD3D12RHI, Error, "Shader platform requires at least: %d samplers, device supports: %d.", FDataDrivenShaderPlatformInfo::GetMaxSamplers(GShaderPlatformForFeatureLevel[Index]), GMaxTextureSamplers);
			}
		}
	}

	return GD3D12RHI;
}

void FD3D12DynamicRHIModule::StartupModule()
{
#if D3D12RHI_SUPPORTS_WIN_PIX
#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
	static FString WindowsPixDllRelativePath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/WinPixEventRuntime/arm64"));
	static const TCHAR* WindowsPixDll = TEXT("WinPixEventRuntime.dll");
#else
	static FString WindowsPixDllRelativePath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/WinPixEventRuntime/x64"));
	static const TCHAR* WindowsPixDll = TEXT("WinPixEventRuntime.dll");
#endif

	UE_LOGF(LogD3D12RHI, Log, "Loading %ls for PIX profiling (from %ls).", WindowsPixDll, *WindowsPixDllRelativePath);
	WindowsPixDllHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(WindowsPixDllRelativePath, WindowsPixDll));
	if (WindowsPixDllHandle == nullptr)
	{
		const int32 ErrorNum = FPlatformMisc::GetLastError();
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, ErrorNum);
		UE_LOGF(LogD3D12RHI, Error, "Failed to get %ls handle: %ls (%d)", WindowsPixDll, ErrorMsg, ErrorNum);
	}
#endif
}

void FD3D12DynamicRHIModule::ShutdownModule()
{
#if D3D12RHI_SUPPORTS_WIN_PIX
	if (WindowsPixDllHandle)
	{
		FPlatformProcess::FreeDllHandle(WindowsPixDllHandle);
		WindowsPixDllHandle = nullptr;
	}
#endif
}

static bool IsRayTracingEmulated(uint32 DeviceId)
{
	uint32 EmulatedRayTracingIds[] =
	{
		0x1B80, // "NVIDIA GeForce GTX 1080"
		0x1B81, // "NVIDIA GeForce GTX 1070"
		0x1B82, // "NVIDIA GeForce GTX 1070 Ti"
		0x1B83, // "NVIDIA GeForce GTX 1060 6GB"
		0x1B84, // "NVIDIA GeForce GTX 1060 3GB"
		0x1C01, // "NVIDIA GeForce GTX 1050 Ti"
		0x1C02, // "NVIDIA GeForce GTX 1060 3GB"
		0x1C03, // "NVIDIA GeForce GTX 1060 6GB"
		0x1C04, // "NVIDIA GeForce GTX 1060 5GB"
		0x1C06, // "NVIDIA GeForce GTX 1060 6GB"
		0x1C08, // "NVIDIA GeForce GTX 1050"
		0x1C81, // "NVIDIA GeForce GTX 1050"
		0x1C82, // "NVIDIA GeForce GTX 1050 Ti"
		0x1C83, // "NVIDIA GeForce GTX 1050"
		0x1B06, // "NVIDIA GeForce GTX 1080 Ti"
		0x1B00, // "NVIDIA TITAN X (Pascal)"
		0x1B02, // "NVIDIA TITAN Xp"
		0x1D81, // "NVIDIA TITAN V"
	};

	for (int Index = 0; Index < UE_ARRAY_COUNT(EmulatedRayTracingIds); ++Index)
	{
		if (DeviceId == EmulatedRayTracingIds[Index])
		{
			return true;
		}
	}

	return false;
}

static bool IsNvidiaAmpereGPU(uint32 DeviceId)
{
	uint32 DeviceList[] =
	{
		0x2200,	// GA102
		0x2204,	// GA102 - GeForce RTX 3090
		0x2205,	// GA102 - GeForce RTX 3080 Ti 20GB
		0x2206,	// GA102 - GeForce RTX 3080
		0x2208,	// GA102 - GeForce RTX 3080 Ti
		0x220a,	// GA102 - GeForce RTX 3080 12GB
		0x220d,	// GA102 - CMP 90HX
		0x2216,	// GA102 - GeForce RTX 3080 Lite Hash Rate
		0x2230,	// GA102GL - RTX A6000
		0x2231,	// GA102GL - RTX A5000
		0x2232,	// GA102GL - RTX A4500
		0x2233,	// GA102GL - RTX A5500
		0x2235,	// GA102GL - A40
		0x2236,	// GA102GL - A10
		0x2237,	// GA102GL - A10G
		0x2238,	// GA102GL - A10M
		0x223f,	// GA102G
		0x2302,	// GA103
		0x2321,	// GA103
		0x2414,	// GA103 - GeForce RTX 3060 Ti
		0x2420,	// GA103M - GeForce RTX 3080 Ti Mobile
		0x2460,	// GA103M - GeForce RTX 3080 Ti Laptop GPU
		0x2482,	// GA104 - GeForce RTX 3070 Ti
		0x2483,	// GA104
		0x2484,	// GA104 - GeForce RTX 3070
		0x2486,	// GA104 - GeForce RTX 3060 Ti
		0x2487,	// GA104 - GeForce RTX 3060
		0x2488,	// GA104 - GeForce RTX 3070 Lite Hash Rate
		0x2489,	// GA104 - GeForce RTX 3060 Ti Lite Hash Rate
		0x248a,	// GA104 - CMP 70HX
		0x249c,	// GA104M - GeForce RTX 3080 Mobile / Max-Q 8GB/16GB
		0x249f,	// GA104M
		0x24a0,	// GA104 - Geforce RTX 3070 Ti Laptop GPU
		0x24b0,	// GA104GL - RTX A4000
		0x24b6,	// GA104GLM - RTX A5000 Mobile
		0x24b7,	// GA104GLM - RTX A4000 Mobile
		0x24b8,	// GA104GLM - RTX A3000 Mobile
		0x24dc,	// GA104M - GeForce RTX 3080 Mobile / Max-Q 8GB/16GB
		0x24dd,	// GA104M - GeForce RTX 3070 Mobile / Max-Q
		0x24e0,	// GA104M - Geforce RTX 3070 Ti Laptop GPU
		0x24fa,	// GA104 - RTX A4500 Embedded GPU 
		0x2501,	// GA106 - GeForce RTX 3060
		0x2503,	// GA106 - GeForce RTX 3060
		0x2504,	// GA106 - GeForce RTX 3060 Lite Hash Rate
		0x2505,	// GA106
		0x2507,	// GA106 - Geforce RTX 3050
		0x2520,	// GA106M - GeForce RTX 3060 Mobile / Max-Q
		0x2523,	// GA106M - GeForce RTX 3050 Ti Mobile / Max-Q
		0x2531,	// GA106 - RTX A2000
		0x2560,	// GA106M - GeForce RTX 3060 Mobile / Max-Q
		0x2563,	// GA106M - GeForce RTX 3050 Ti Mobile / Max-Q
		0x2571,	// GA106 - RTX A2000 12GB
		0x2583,	// GA107 - GeForce RTX 3050
		0x25a0,	// GA107M - GeForce RTX 3050 Ti Mobile
		0x25a2,	// GA107M - GeForce RTX 3050 Mobile
		0x25a4,	// GA107
		0x25a5,	// GA107M - GeForce RTX 3050 Mobile
		0x25a6,	// GA107M - GeForce MX570
		0x25a7,	// GA107M - GeForce MX570
		0x25a9,	// GA107M - GeForce RTX 2050
		0x25b5,	// GA107GLM - RTX A4 Mobile
		0x25b8,	// GA107GLM - RTX A2000 Mobile
		0x25b9,	// GA107GLM - RTX A1000 Laptop GPU
		0x25e0,	// GA107BM - GeForce RTX 3050 Ti Mobile
		0x25e2,	// GA107BM - GeForce RTX 3050 Mobile
		0x25e5,	// GA107BM - GeForce RTX 3050 Mobile
		0x25f9,	// GA107 - RTX A1000 Embedded GPU 
		0x25fa,	// GA107 - RTX A2000 Embedded GPU
	};

	for (uint32 KnownDeviceId : DeviceList)
	{
		if (DeviceId == KnownDeviceId)
		{
			return true;
		}
	}

	return false;
}

static void DisableRayTracingSupport()
{
	GRHISupportsRayTracing = false;
	GRHISupportsRayTracingPSOAdditions = false;
	GRHISupportsRayTracingDispatchIndirect = false;
	GRHISupportsRayTracingAsyncBuildAccelerationStructure = false;
	GRHISupportsRayTracingAMDHitToken = false;
	GRHISupportsInlineRayTracing = false;
	GRHISupportsRayTracingShaders = false;

	GRHIGlobals.RayTracing.SupportsAsyncRayTraceDispatch = false;
}

static void ClearPSODriverCache()
{
	FString LocalAppDataFolder = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
	if (!LocalAppDataFolder.IsEmpty())
	{
		auto DeleteFolder = [](const FString& PSOFolderPath) {
			IFileManager& FileManager = IFileManager::Get();
			FileManager.DeleteDirectory(*PSOFolderPath, /*RequireExists*/ false, /*Tree*/ true);
		};

		auto ClearFolder = [](const FString& PSOPath, const TCHAR* Extension)
		{
			TArray<FString> Files;
			IFileManager& FileManager = IFileManager::Get();
			FileManager.FindFiles(Files, *PSOPath, Extension);
			for (FString& File : Files)
			{
				FString FilePath = FString::Printf(L"%s\\%s", *PSOPath, *File);
				FileManager.Delete(*FilePath, /*RequireExists*/ false, /*EvenReadOnly*/ true, /*Quiet*/ true);
			}
		};

		if (IsRHIDeviceNVIDIA())
		{
			// NVIDIA used to have a global cache, but now also has a per-driver cache in a different folder in LocalLow.
			FString GlobalPSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT("NVIDIA"), TEXT("DXCache"));
			ClearFolder(GlobalPSOPath, nullptr);

			FString PerDriverPSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT(".."), TEXT("LocalLow"), TEXT("NVIDIA"), TEXT("PerDriverVersion"), TEXT("DXCache"));
			ClearFolder(PerDriverPSOPath, nullptr);
		}
		else if (IsRHIDeviceAMD())
		{
			FString PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT("AMD"), TEXT("DxCache"));
			ClearFolder(PSOPath, nullptr);

			PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT("AMD"), TEXT("DxcCache"));
			ClearFolder(PSOPath, nullptr);

			PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT(".."), TEXT("LocalLow"), TEXT("AMD"), TEXT("DxCache"));
			ClearFolder(PSOPath, nullptr);

			PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT(".."), TEXT("LocalLow"), TEXT("AMD"), TEXT("DxcCache"));
			ClearFolder(PSOPath, nullptr);
		}
		else if (IsRHIDeviceIntel())
		{
			// Intel stores the cache in LocalLow.
			FString PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT(".."), TEXT("LocalLow"), TEXT("Intel"), TEXT("ShaderCache"));
			ClearFolder(PSOPath, nullptr);
		}
		else if (IsRHIDeviceQualcomm())
		{
			// Qualcomm uses D3DSCache. The folder contains sub-folders for each application's cache entry so it needs recursive deletion.
			FString PSOPath = FPaths::Combine(*LocalAppDataFolder, TEXT("D3DSCache"));
			DeleteFolder(PSOPath);
		}
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Error, "clearPSODriverCache failed: please ensure that LOCALAPPDATA points to C:\\Users\\<username>\\AppData\\Local");
	}
}

void FD3D12DynamicRHI::InitializeDeviceAgnosticVendorAPI()
{
	// We set up AMD and NVIDIA APIs here since they don't
	// need a device. Intel extensions are enabled after
	// device creation.
	if (!UE::RHICore::AllowVendorDevice())
	{
		return;
	}

	const DXGI_ADAPTER_DESC& AdapterDesc = GetAdapter().GetD3DAdapterDesc();

#ifdef AMD_AGS_API
	if (IsRHIDeviceAMD())
	{
		check(AmdAgsContext == nullptr);

		// agsInitialize should be called before D3D device creation.
		if (agsInitialize(AGS_MAKE_VERSION(AMD_AGS_VERSION_MAJOR, AMD_AGS_VERSION_MINOR, AMD_AGS_VERSION_PATCH), nullptr, &AmdAgsContext, &AmdInfo.AmdGpuInfo) == AGS_SUCCESS)
		{
			AmdInfo.AmdAgsContext = AmdAgsContext;
			for (int32 DeviceIndex = 0; DeviceIndex < AmdInfo.AmdGpuInfo.numDevices; DeviceIndex++)
			{
				const AGSDeviceInfo& DeviceInfo = AmdInfo.AmdGpuInfo.devices[DeviceIndex];
				if (AdapterDesc.VendorId == DeviceInfo.vendorId && AdapterDesc.DeviceId == DeviceInfo.deviceId)
				{
					const uint32 AsicFamily = DeviceInfo.asicFamily;
					UE_LOGF(LogD3D12RHI, Log, "AMD asicFamily=%u", AsicFamily);
					GRHIDeviceIsAMDPreGCNArchitecture = DeviceInfo.asicFamily == AGSDeviceInfo::AsicFamily_PreGCN;
					GRHIGlobals.GpuInfo.AdapterGeneration = AsicFamily;
				}
			}

			if (GRHIDeviceIsAMDPreGCNArchitecture)
			{
				UE_LOGF(LogD3D12RHI, Log, "AMD Pre GCN architecture detected, some driver workarounds will be in place");
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Warning, "Failed to initialize AMD AGS API");

			FMemory::Memzero(&AmdInfo, sizeof(AmdInfo));
			// If agsInitialize returns anything but AGS_SUCCESS, the context pointer should be
			// guaranteed to be NULL, but we'll set it here explicitly, just to be safe.
			AmdAgsContext = nullptr;
		}
	}
	else
	{
		FMemory::Memzero(&AmdInfo, sizeof(AmdInfo));
	}
#endif // AMD_AGS_API

#ifdef NVAPI_INTERFACE
	if (IsRHIDeviceNVIDIA())
	{
		if (NvAPI_Initialize() == NVAPI_OK)
		{
			bNvAPIInitialized = true;

			NvPhysicalGpuHandle GPUHandles[NVAPI_MAX_PHYSICAL_GPUS];
			NvU32 NumGPUs = 0;
			if (NvAPI_EnumPhysicalGPUs(GPUHandles, &NumGPUs) == NVAPI_OK)
			{
				for (NvU32 GPU = 0; GPU < NumGPUs; ++GPU)
				{
					const NvPhysicalGpuHandle& GPUHandle = GPUHandles[GPU];

					NvU32 Unused = 0, SubsystemId = 0, RevisionId = 0, ExtDeviceId = 0;
					if (NvAPI_GPU_GetPCIIdentifiers(GPUHandle, &Unused, &SubsystemId, &RevisionId, &ExtDeviceId) != NVAPI_OK)
					{
						continue;
					}

					if (ExtDeviceId != AdapterDesc.DeviceId || SubsystemId != AdapterDesc.SubSysId || RevisionId != AdapterDesc.Revision)
					{
						continue;
					}

					NV_GPU_ARCH_INFO ArchInfo;
					ArchInfo.version = NV_GPU_ARCH_INFO_VER_2;
					if (NvAPI_GPU_GetArchInfo(GPUHandle, &ArchInfo) != NVAPI_OK)
					{
						continue;
					}

					const uint32 Architecture = ArchInfo.architecture;
					UE_LOGF(LogD3D12RHI, Log, "NVIDIA Architecture=%u", Architecture);
					GRHIGlobals.GpuInfo.AdapterGeneration = Architecture;
				}
			}
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Warning, "Failed to initialize NVAPI");
		}
	}
#endif // NVAPI_INTERFACE
}

void FD3D12DynamicRHI::Init()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12DynamicRHI::Init);

#if D3D12RHI_SUPPORTS_WIN_PIX
	// PIX GPU capture dll: makes PIX be able to attach to our process. !GetModuleHandle() is required because PIX may already be attached.
	if (GAutoAttachPIX || FParse::Param(FCommandLine::Get(), TEXT("attachPIX")))
	{
		// If PIX is not already attached, load its dll to auto attach ourselves
		if (!FPlatformProcess::GetDllHandle(L"WinPixGpuCapturer.dll"))
		{
			// This should always be loaded from the installed PIX directory.
			// This function does assume it's installed under Program Files so we may have to revisit for custom install locations.
			WinPixGpuCapturerHandle = PIXLoadLatestWinPixGpuCapturerLibrary();
		}
	}
#endif

	SetupD3D12Debug();

	check(!GIsRHIInitialized);

	if (FParse::Param(FCommandLine::Get(), TEXT("clearPSODriverCache")))
	{
		ClearPSODriverCache();
	}

	const DXGI_ADAPTER_DESC& AdapterDesc = GetAdapter().GetD3DAdapterDesc();

	UE_LOGF(LogD3D12RHI, Log, "    GPU DeviceId: 0x%x (for the marketing name, search the web for \"GPU Device Id\")", AdapterDesc.DeviceId);

	// Create a device chain for the adapter we have chosen. This could be a single discrete card,
	// a set discrete cards linked together (i.e. SLI/Crossfire) an Integrated device or any combination of the above
	if (FD3D12Adapter* const Adapter = ChosenAdapter.Get())
	{
		check(Adapter->GetDesc().IsValid());
		Adapter->InitializeDevices();
	}

	if (GDevDisableD3DRuntimeBackgroundThreads)
	{
#if D3D12_MAX_DEVICE_INTERFACE >= 6
		ID3D12Device6* Device6 = GetAdapter().GetD3DDevice6();
		if (Device6)
		{
			HRESULT Res = Device6->SetBackgroundProcessingMode(
				D3D12_BACKGROUND_PROCESSING_MODE_DISABLE_PROFILING_BY_SYSTEM,
				D3D12_MEASUREMENTS_ACTION_KEEP_ALL,
				nullptr, nullptr);

			if (SUCCEEDED(Res))
			{
				UE_LOGF(LogD3D12RHI, Log, "Disabled D3D runtime's background threads");
			}
			else
			{
				UE_LOGF(LogD3D12RHI, Error, "Could not disable D3D runtime's background threads: SetBackgroundProcessingMode returned error 0x%08X", Res);
			}
		}
		else
#endif  
		{
			UE_LOGF(LogD3D12RHI, Warning, "Could not disable D3D runtime's background threads because the ID3D12Device6 interface is not available");
		}
	}

	bool bHasVendorSupportForAtomic64 = false;

#if WITH_AMD_AGS
	// Check if the AMD device is pre-RDNA, and ensure it doesn't misreport wave32 support
	if (IsRHIDeviceAMD() && UE::RHICore::AllowVendorDevice() && AmdAgsContext)
	{
		for (int32 DeviceIndex = 0; DeviceIndex < AmdInfo.AmdGpuInfo.numDevices; DeviceIndex++)
		{
			const AGSDeviceInfo& DeviceInfo = AmdInfo.AmdGpuInfo.devices[DeviceIndex];
			if (DeviceInfo.deviceId == AdapterDesc.DeviceId && DeviceInfo.vendorId == AdapterDesc.VendorId)
			{
				if (DeviceInfo.asicFamily != AGSDeviceInfo::AsicFamily_Unknown && DeviceInfo.asicFamily < AGSDeviceInfo::AsicFamily_RDNA)
				{
					GRHIMinimumWaveSize = 64;
					GRHIMaximumWaveSize = 64;
				}

				break;
			}
		}
	}

	// Warn if we are trying to use RGP frame markers but are either running on a non-AMD device
	// or using an older AMD driver without RGP marker support
	if (IsRHIDeviceAMD())
	{
		if (UE::RHICore::AllowVendorDevice())
		{
			static_assert(sizeof(AGSDX12ReturnedParams::ExtensionsSupported) == sizeof(uint32));
			AGSDX12ReturnedParams::ExtensionsSupported AMDSupportedExtensions;
			FMemory::Memcpy(&AMDSupportedExtensions, &AmdSupportedExtensionFlags, sizeof(uint32));

			if (GEmitRgpFrameMarkers && AMDSupportedExtensions.userMarkers == 0)
		    {
			    UE_LOGF(LogD3D12RHI, Warning, "Attempting to use RGP frame markers without driver support. Update AMD driver.");
		    }

			bHasVendorSupportForAtomic64 = AMDSupportedExtensions.intrinsics19 != 0;  // "intrinsics19" includes AtomicU64
			bHasVendorSupportForAtomic64 = bHasVendorSupportForAtomic64 && AMDSupportedExtensions.UAVBindSlot != 0;
		}
	}
	else if (GEmitRgpFrameMarkers)
	{
		UE_LOGF(LogD3D12RHI, Warning, "Attempting to use RGP frame markers on a non-AMD device.");
	}
#endif

	// Disable ray tracing for Windows build versions
	if (GRHISupportsRayTracing
		&& GMinimumWindowsBuildVersionForRayTracing > 0
		&& !FPlatformMisc::VerifyWindowsVersion(10, 0, GMinimumWindowsBuildVersionForRayTracing))
	{
		DisableRayTracingSupport();
		UE_LOGF(LogD3D12RHI, Warning, "Ray tracing is disabled because it requires Windows 10 version %u", (uint32)GMinimumWindowsBuildVersionForRayTracing);
	}

	const FDriverVersion CurrentDriverVersion(GRHIGlobals.GpuInfo.AdapterUserDriverVersion);

#if WITH_NVAPI
	if (IsRHIDeviceNVIDIA() && UE::RHICore::AllowVendorDevice())
	{
		NvU32 PipelineFlags = NVAPI_D3D12_PIPELINE_CREATION_STATE_FLAGS_NONE;

		// Helper to allow all pipeline state capabilities to test and accumulate their support, so all can be applied atomically
		auto CheckPipelineStateCap = [&](NVAPI_D3D12_PIPELINE_CREATION_STATE_FLAGS Flag) -> bool
			{
				NVAPI_D3D12_SET_CREATE_PIPELINE_STATE_OPTIONS_PARAMS PipelineStateOptions;
				PipelineStateOptions.version = NVAPI_D3D12_SET_CREATE_PIPELINE_STATE_OPTIONS_PARAMS_VER;
				PipelineStateOptions.flags = Flag;

				if (NvAPI_D3D12_SetCreatePipelineStateOptions(GetAdapter().GetD3DDevice5(), &PipelineStateOptions) == NVAPI_OK)
				{
					PipelineFlags |= Flag;
					return true;
				}
				return false;
			};

		if (bNvAPIInitialized)
		{
			NvAPI_Status NvStatusAtomicU64 = NvAPI_D3D12_IsNvShaderExtnOpCodeSupported(GetAdapter().GetD3DDevice(), NV_EXTN_OP_UINT64_ATOMIC, &bHasVendorSupportForAtomic64);
			if (NvStatusAtomicU64 != NVAPI_OK)
			{
				UE_LOGF(LogD3D12RHI, Warning, "Failed to query support for 64 bit atomics");
			}

			// NvAPI_D3D12_GetRaytracingCaps appears to leak about 100 MB so avoid calling if RayTracing is impossible
			if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6)
			{
#if WITH_NVAPI_RAYTRACING_VALIDATION
				{
					ID3D12Device5* Device5 = GetAdapter().GetD3DDevice5();
					if (GRHISupportsRayTracing
						&& Device5 != nullptr
						&& (GEnableNVAPIRaytracingValidation || FParse::Param(FCommandLine::Get(), TEXT("validateRayTracing"))))
					{
						const NvAPI_Status NvStatusVal = NvAPI_D3D12_EnableRaytracingValidation(Device5, NVAPI_D3D12_RAYTRACING_VALIDATION_FLAG_NONE);
						if (NvStatusVal == NVAPI_OK)
						{
							auto RtValidationMessageCB = [](void* pUserData, NVAPI_D3D12_RAYTRACING_VALIDATION_MESSAGE_SEVERITY Severity, const char* MessageCode, const char* Message, const char* MessageDetails)
								{
									TArray<FString>* RtValidationCodesToIgnoreList = (TArray<FString>*)pUserData;

									if (RtValidationCodesToIgnoreList->Contains(MessageCode))
									{
										return;
									}

									switch (Severity)
									{
									case NVAPI_D3D12_RAYTRACING_VALIDATION_MESSAGE_SEVERITY_ERROR:
									{
										UE_LOGF(LogD3D12RHI, Error, "Ray Tracing Validation message: [%s] %s\n%s", MessageCode, Message, MessageDetails);
										break;
									}
									case NVAPI_D3D12_RAYTRACING_VALIDATION_MESSAGE_SEVERITY_WARNING:
									{
										UE_LOGF(LogD3D12RHI, Warning, "Ray Tracing Validation message: [%s] %s\n%s", MessageCode, Message, MessageDetails);
										break;
									}
									default:
									{
										UE_LOGF(LogD3D12RHI, Warning, "Ray Tracing Validation message: [%s] %s\n%s", MessageCode, Message, MessageDetails);
										break;
									}
									}
								};

							static TArray<FString> RtValidationCodesToIgnoreList = []() -> TArray<FString> {
								TArray<FString> Res;
								FString CodesToIgnoreStr = CVarNVAPIRaytracingFilterMessages.GetValueOnRenderThread();
								CodesToIgnoreStr.ParseIntoArray(Res, TEXT(";"));
								return Res;
								}();

							const NvAPI_Status NvStatusCb = NvAPI_D3D12_RegisterRaytracingValidationMessageCallback(Device5, RtValidationMessageCB, &RtValidationCodesToIgnoreList, &RayTracingValidationCallbackHandle);
							checkf(NvStatusCb == NVAPI_OK, TEXT("NvAPI Raytracing Validation Enabled, but NvAPI_D3D12_RegisterRaytracingValidationMessageCallback failed with %d."), (uint32)NvStatusCb);
							if (NvStatusCb == NVAPI_OK)
							{
								UE_LOGF(LogD3D12RHI, Log, "NvAPI Raytracing Validation Enabled (-validateRayTracing)");
							}
						}
						else if (NvStatusVal == NVAPI_ACCESS_DENIED)
						{
							checkf(false, TEXT("Failed to enable NvAPI Raytracing Validation. The env variable NV_ALLOW_RAYTRACING_VALIDATION=1 must be set on the system."));
						}
						else if (NvStatusVal == NVAPI_INVALID_CALL)
						{
							checkf(false, TEXT("Failed to enable NvAPI Raytracing Validation (NVAPI_INVALID_CALL). Raytracing Validation must be initialized before any RT-Related calls."));
						}
						else
						{
							checkf(false, TEXT("Failed to enable NvAPI Raytracing Validation. Error=%d"), (uint32)NvStatusVal);
						}
					}
				}
#endif // WITH_NVAPI_RAYTRACING_VALIDATION
	
				bool bSkipSERSupportCheck = false;

#if D3D12RHI_SUPPORTS_WIN_PIX
				// Pix only supports captures with SER enabled when using D3D12's official SER API, which is currently not available in a non-preview Agility SDK
				if (PIXIsAttachedForGpuCapture())
				{
					bSkipSERSupportCheck = true;
					GRHIGlobals.SupportsShaderExecutionReordering = false;
					UE_LOGF(LogD3D12RHI, Log, "Skipping NvAPI Shader Execution Reordering support as Pix is attached to the process.");
				}
#endif
				if (!bSkipSERSupportCheck)
				{
					ID3D12Device* D3DDevice = GetAdapter().GetD3DDevice();
					NvAPI_Status NvStatusSER = NvAPI_D3D12_IsNvShaderExtnOpCodeSupported(D3DDevice, NV_EXTN_OP_HIT_OBJECT_REORDER_THREAD, &GRHIGlobals.SupportsShaderExecutionReordering);
					if (NvStatusSER == NVAPI_OK && GRHIGlobals.SupportsShaderExecutionReordering)
					{
						NVAPI_D3D12_RAYTRACING_THREAD_REORDERING_CAPS ReorderCaps = NVAPI_D3D12_RAYTRACING_THREAD_REORDERING_CAP_NONE;
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(NvAPI_D3D12_GetRaytracingCaps);
							NvStatusSER = NvAPI_D3D12_GetRaytracingCaps(
								D3DDevice,
								NVAPI_D3D12_RAYTRACING_CAPS_TYPE_THREAD_REORDERING,
								&ReorderCaps,
								sizeof(ReorderCaps));
						}

						if (NvStatusSER != NVAPI_OK || ReorderCaps == NVAPI_D3D12_RAYTRACING_THREAD_REORDERING_CAP_NONE)
						{
							GRHIGlobals.SupportsShaderExecutionReordering = false;
						}
					}
				}

				if (GRHIGlobals.SupportsShaderExecutionReordering)
				{
					UE_LOGF(LogD3D12RHI, Log, "NVIDIA Shader Execution Reordering interface supported!");
				}
				else
				{
					UE_LOGF(LogD3D12RHI, Log, "NVIDIA Shader Execution Reordering NOT supported!");
				}

				// Ray Tracing Cluster Ops
				{
					GRHIGlobals.RayTracing.SupportsClusterOps = false;

					ID3D12Device5* Device5 = GetAdapter().GetD3DDevice5();
					NVAPI_D3D12_RAYTRACING_CLUSTER_OPERATIONS_CAPS ClusterOpsCaps = {};
					NvAPI_Status NvStatusRayTracingClusterOps;
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(NvAPI_D3D12_GetRaytracingCaps);
						NvStatusRayTracingClusterOps = NvAPI_D3D12_GetRaytracingCaps(Device5, NVAPI_D3D12_RAYTRACING_CAPS_TYPE_CLUSTER_OPERATIONS, &ClusterOpsCaps, sizeof(NVAPI_D3D12_RAYTRACING_CLUSTER_OPERATIONS_CAPS));
					}

					if (NvStatusRayTracingClusterOps == NVAPI_OK && ClusterOpsCaps == NVAPI_D3D12_RAYTRACING_CLUSTER_OPERATIONS_CAP_STANDARD)
					{
						GRHIGlobals.RayTracing.SupportsClusterOps = CheckPipelineStateCap(NVAPI_D3D12_PIPELINE_CREATION_STATE_FLAGS_ENABLE_CLUSTER_SUPPORT);
					}

					if (GRHIGlobals.RayTracing.SupportsClusterOps)
					{
						GRHIGlobals.RayTracing.ClusterAccelerationStructureAlignment = NVAPI_D3D12_RAYTRACING_CLAS_BYTE_ALIGNMENT;
						GRHIGlobals.RayTracing.ClusterAccelerationStructureTemplateAlignment = NVAPI_D3D12_RAYTRACING_CLUSTER_TEMPLATE_BYTE_ALIGNMENT;
						UE_LOGF(LogD3D12RHI, Log, "NVIDIA Ray Tracing Cluster Operations supported and enabled");
					}
					else
					{
						UE_LOGF(LogD3D12RHI, Log, "NVIDIA Ray Tracing Cluster Operations not supported on this device");
					}
				}
			}
			else
			{
				GRHIGlobals.SupportsShaderExecutionReordering = false;
				GRHIGlobals.RayTracing.SupportsClusterOps = false;
				UE_LOGF(LogD3D12RHI, Log, "Skipped NVAPI RT queries since we the feature level is below SM6");
			}

			// LSS primitive support
			GRHIGlobals.RayTracing.SupportsLinearSweptSpheres = false;
			if (GRHISupportsRayTracing)
			{
				ID3D12Device5* Device5 = GetAdapter().GetD3DDevice5();
				
				NVAPI_D3D12_RAYTRACING_LINEAR_SWEPT_SPHERES_CAPS LSSCaps = NVAPI_D3D12_RAYTRACING_LINEAR_SWEPT_SPHERES_CAP_NONE;
				NvAPI_Status NvStatusLSS;
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(NvAPI_D3D12_GetRaytracingCaps);
					NvStatusLSS = NvAPI_D3D12_GetRaytracingCaps(Device5, NVAPI_D3D12_RAYTRACING_CAPS_TYPE_LINEAR_SWEPT_SPHERES, &LSSCaps, sizeof(NVAPI_D3D12_RAYTRACING_LINEAR_SWEPT_SPHERES_CAPS));
				}

				if (NvStatusLSS == NVAPI_OK && LSSCaps != NVAPI_D3D12_RAYTRACING_LINEAR_SWEPT_SPHERES_CAP_NONE)
				{
					GRHIGlobals.RayTracing.SupportsLinearSweptSpheres = CheckPipelineStateCap(NVAPI_D3D12_PIPELINE_CREATION_STATE_FLAGS_ENABLE_LSS_SUPPORT);
				}

				if (GRHIGlobals.RayTracing.SupportsLinearSweptSpheres)
				{
					UE_LOGF(LogD3D12RHI, Log, "LSS support is enabled.");
				}
				else
				{
					UE_LOGF(LogD3D12RHI, Log, "LSS not supported on this device.");
				}
			}
		}

		// Disable ray tracing for old Nvidia drivers
		if (GRHISupportsRayTracing 
			&& !GMinimumDriverVersionForRayTracingNVIDIA.IsEmpty()
			&& CurrentDriverVersion < FDriverVersion(GMinimumDriverVersionForRayTracingNVIDIA))
		{
			DisableRayTracingSupport();
			UE_LOGF(LogD3D12RHI, Warning, "Ray tracing is disabled because the driver is too old");
		}

		// Disable indirect ray tracing dispatch on drivers that have a known bug.
		if (GRHISupportsRayTracingDispatchIndirect 
			&& CurrentDriverVersion < FDriverVersion(TEXT("466.11")))
		{
			GRHISupportsRayTracingDispatchIndirect = false;
			UE_LOGF(LogD3D12RHI, Warning,
				"Indirect ray tracing dispatch is disabled because of known bugs in the current driver. " "Please update to NVIDIA driver version 466.11 or newer.");
		}

		if (GRHISupportsRayTracing
			&& IsRayTracingEmulated(AdapterDesc.DeviceId))
		{
			DisableRayTracingSupport();
			UE_LOGF(LogD3D12RHI, Warning, "Ray tracing is disabled for NVIDIA cards with the Pascal architecture.");
		}

		if (GRHISupportsRayTracing
			&& PipelineFlags != NVAPI_D3D12_PIPELINE_CREATION_STATE_FLAGS_NONE)
		{
			// Set the accumulated feature flags for pipeline creation
			NVAPI_D3D12_SET_CREATE_PIPELINE_STATE_OPTIONS_PARAMS PipelineStateOptions;
			PipelineStateOptions.version = NVAPI_D3D12_SET_CREATE_PIPELINE_STATE_OPTIONS_PARAMS_VER;
			PipelineStateOptions.flags = PipelineFlags;

			NvAPI_Status Status = NvAPI_D3D12_SetCreatePipelineStateOptions(GetAdapter().GetD3DDevice5(), &PipelineStateOptions);
			if (Status == NVAPI_OK)
			{
				UE_LOGF(LogD3D12RHI, Log, "NVIDIA Extended ray tracing pipeline state applied.");
			}
			else
			{
				checkf(0, TEXT("NvAPI SetCreatePipelineStateOptions failed, incompatible feature flags are enabled"));
			}
		}
	} // if NVIDIA
#endif // NVAPI

	if (IsRHIDeviceNVIDIA())
	{
		//
		// See Jira UE-233616
		//
		// The NVIDIA driver drops barriers at the beginning of command lists on the async queue because
		// it assumes each list is executed within its own ECL scope and therefore the GPU is idle'd between
		// each. That's not the case if we execute more than one list at a time, so break it up into individual
		// executions to make it true.
		//
		// Update 2026-01-19: NVIDIA reports the bug was fixed in driver version 571.23 and newer.
		//
		const auto CVarD3D12SubmissionAsyncMaxExecuteBatchSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.D3D12.Submission.MaxExecuteBatchSize.Async"));
		if (!ensure(CVarD3D12SubmissionAsyncMaxExecuteBatchSize))
		{
			UE_LOGF(LogD3D12RHI, Error, "Couldn't find CVar r.D3D12.Submission.MaxExecuteBatchSize.Async to work around a known bug in certain drivers.");
		}
		else if (CurrentDriverVersion < FDriverVersion(TEXT("571.23")))
		{
			CVarD3D12SubmissionAsyncMaxExecuteBatchSize->Set(1);
			UE_LOGF(LogD3D12RHI, Warning, "Batched command list execution is disabled for async queues due to known bugs in the current driver.");
		}

		// NVIDIA's shader compiler had a bug that produced visible image corruption when TSR's
		// 16-bit VALU compute shader permutation was used. The bug was in an optimization pass
		// that silently dropped operand modifiers when vectorizing pairs of 16-bit instructions.
		//
		// Update 2026-05-29: NVIDIA reports the bug was fixed in driver version 610.00 and newer.
		//
		const auto CVarTSR16BitVALUNvidia = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TSR.16BitVALU.Nvidia"));
		if (!ensure(CVarTSR16BitVALUNvidia))
		{
			UE_LOGF(LogD3D12RHI, Error, "Couldn't find CVar r.TSR.16BitVALU.Nvidia to work around a known bug in certain drivers.");
		}
		else if (CurrentDriverVersion < FDriverVersion(TEXT("610.00")))
		{
			CVarTSR16BitVALUNvidia->Set(0);
			UE_LOGF(LogD3D12RHI, Warning, "TSR 16-bit VALU is disabled on NVIDIA due to known compiler bugs in drivers older than 610.00.");
		}
	} // if NVIDIA

#if WITH_AMD_AGS
	if (GRHISupportsRayTracing
		&& IsRHIDeviceAMD()
		&& AmdAgsContext)
	{
		if (!GMinimumDriverVersionForRayTracingAMD.IsEmpty() &&
			CurrentDriverVersion < FDriverVersion(GMinimumDriverVersionForRayTracingAMD))
		{
			DisableRayTracingSupport();
			UE_LOGF(LogD3D12RHI, Warning, "Ray tracing is disabled because the driver is too old");
		}
		else
		{
			static_assert(sizeof(AGSDX12ReturnedParams::ExtensionsSupported) == sizeof(uint32));
			AGSDX12ReturnedParams::ExtensionsSupported AMDSupportedExtensions;
			FMemory::Memcpy(&AMDSupportedExtensions, &AmdSupportedExtensionFlags, sizeof(uint32));

			GRHISupportsRayTracingAMDHitToken = AMDSupportedExtensions.rayHitToken;
			UE_LOGF(LogD3D12RHI, Log, "AMD hit token extension is %ls", GRHISupportsRayTracingAMDHitToken ? TEXT("supported") : TEXT("not supported"));
		}
	}
#endif // WITH_AMD_AGS

	if (IsRHIDeviceIntel())
	{
		if (!GMinimumDriverVersionForRayTracingIntel.IsEmpty() &&
			CurrentDriverVersion < FDriverVersion(GMinimumDriverVersionForRayTracingIntel))
		{
			DisableRayTracingSupport();
		}

#if INTEL_EXTENSIONS
		if (UE::RHICore::AllowVendorDevice())
		{
			if (GDX12INTCAtomicUInt64Emulation)
			{
				bHasVendorSupportForAtomic64 = GDX12INTCAtomicUInt64Emulation;
			}

			// Create a new Intel Extensions Device Extension Context to support DX12 extension calls
			INTCExtensionInfo INTCExtensionInfo{};
			IntelExtensionContext = CreateIntelExtensionsContext(GetAdapter().GetD3DDevice(), INTCExtensionInfo);
		}
#endif // INTEL_EXTENSIONS
	}

	GRHIPersistentThreadGroupCount = 1440; // TODO: Revisit based on vendor/adapter/perf query

	GTexturePoolSize = 0;

	// Issue: 32bit windows doesn't report 64bit value, we take what we get.
	FD3D12GlobalStats::GDedicatedVideoMemory = int64(AdapterDesc.DedicatedVideoMemory);
	FD3D12GlobalStats::GDedicatedSystemMemory = int64(AdapterDesc.DedicatedSystemMemory);
	FD3D12GlobalStats::GSharedSystemMemory = int64(AdapterDesc.SharedSystemMemory);

	// Total amount of system memory, clamped to 8 GB
	int64 TotalPhysicalMemory = FMath::Min(int64(FPlatformMemory::GetConstants().TotalPhysicalGB), 8ll) * (1024ll * 1024ll * 1024ll);

	// Consider 50% of the shared memory but max 25% of total system memory.
	int64 ConsideredSharedSystemMemory = FMath::Min(FD3D12GlobalStats::GSharedSystemMemory / 2ll, TotalPhysicalMemory / 4ll);

	TRefCountPtr<IDXGIAdapter3> DxgiAdapter3;
	VERIFYD3D12RESULT(GetAdapter().GetAdapter()->QueryInterface(IID_PPV_ARGS(DxgiAdapter3.GetInitReference())));
	DXGI_QUERY_VIDEO_MEMORY_INFO LocalVideoMemoryInfo;
	VERIFYD3D12RESULT(DxgiAdapter3->QueryVideoMemoryInfo(0, DXGI_MEMORY_SEGMENT_GROUP_LOCAL, &LocalVideoMemoryInfo));
	const int64 TargetBudget = LocalVideoMemoryInfo.Budget * 0.90f;	// Target using 90% of our budget to account for some fragmentation.
	FD3D12GlobalStats::GTotalGraphicsMemory = TargetBudget;

	if (sizeof(SIZE_T) < 8)
	{
		// Clamp to 1 GB if we're less than 64-bit
		FD3D12GlobalStats::GTotalGraphicsMemory = FMath::Min(FD3D12GlobalStats::GTotalGraphicsMemory, 1024ll * 1024ll * 1024ll);
	}

	if (GPoolSizeVRAMPercentage > 0)
	{
		float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(FD3D12GlobalStats::GTotalGraphicsMemory);

		// Truncate GTexturePoolSize to MB (but still counted in bytes)
		GTexturePoolSize = int64(FGenericPlatformMath::TruncToFloat(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;

		UE_LOGF(LogRHI, Log, "Texture pool is %llu MB (%d%% of %llu MB)",
			GTexturePoolSize / 1024 / 1024,
			GPoolSizeVRAMPercentage,
			FD3D12GlobalStats::GTotalGraphicsMemory / 1024 / 1024);
	}

	RequestedTexturePoolSize = GTexturePoolSize;

	VERIFYD3D12RESULT(DxgiAdapter3->SetVideoMemoryReservation(0, DXGI_MEMORY_SEGMENT_GROUP_LOCAL, FMath::Min((int64)LocalVideoMemoryInfo.AvailableForReservation, FD3D12GlobalStats::GTotalGraphicsMemory)));

	// Multi-threaded resource creation is always supported in DX12, but allow users to disable it.
	GRHISupportsAsyncTextureCreation = D3D12RHI_ShouldAllowAsyncResourceCreation();
	if (GRHISupportsAsyncTextureCreation)
	{
		GRHISupportAsyncTextureStreamOut = true;
		UE_LOGF(LogD3D12RHI, Log, "Async texture creation enabled");
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Log, "Async texture creation disabled: %ls",
			D3D12RHI_ShouldAllowAsyncResourceCreation() ? TEXT("no driver support") : TEXT("disabled by user"));
	}

	if (GRHISupportsAtomicUInt64)
	{
		UE_LOGF(LogD3D12RHI, Log, "RHI has support for 64 bit atomics");
	}
	else if (bHasVendorSupportForAtomic64)
	{
		GRHISupportsAtomicUInt64 = true;

		UE_LOGF(LogD3D12RHI, Log, "RHI has vendor support for 64 bit atomics");
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Log, "RHI does not have support for 64 bit atomics");
	}

	// Detect reserved resource support
	{
		D3D12_FEATURE_DATA_D3D12_OPTIONS Options = {};
		if (SUCCEEDED(GetAdapter().GetD3DDevice()->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS, &Options, sizeof(Options))))
		{
			// Tier 2 is guaranteed for all adapters with feature level 12_0.
			GRHIGlobals.ReservedResources.Supported = Options.TiledResourcesTier >= D3D12_TILED_RESOURCES_TIER_2;

			// Tier 3 is required to create volume textures. Some hardware may support it.
			GRHIGlobals.ReservedResources.SupportsVolumeTextures = Options.TiledResourcesTier >= D3D12_TILED_RESOURCES_TIER_3;
		}
	}

	D3D12_FEATURE_DATA_D3D12_OPTIONS6 options = {};
	HRESULT Options6HR = GetAdapter().GetD3DDevice()->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS6, &options, sizeof(options));

#if WITH_MGPU
	// Disallow async compute in mGPU mode due to submission order bugs (UE-193929).
	if (GNumExplicitGPUsForRendering > 1)
	{
		GSupportsEfficientAsyncCompute = false;
	}
#endif

	GSupportsDepthBoundsTest = SupportsDepthBoundsTest(this);

	{
		GRHISupportsHDROutput = SetupDisplayHDRMetaData();

		// Specify the desired HDR pixel format.
		// Possible values are:
		//	1) PF_FloatRGBA - FP16 format that allows for linear gamma. This is the current engine default.
		//					r.HDR.Display.ColorGamut = 0 (sRGB which is the same gamut as ScRGB)
		//					r.HDR.Display.OutputDevice = 5 or 6 (ScRGB)
		//	2) PF_A2B10G10R10 - Save memory vs FP16 as well as allow for possible performance improvements 
		//						in fullscreen by avoiding format conversions.
		//					r.HDR.Display.ColorGamut = 2 (Rec2020 / BT2020)
		//					r.HDR.Display.OutputDevice = 3 or 4 (ST-2084)
#if WITH_EDITOR
		GRHIHDRDisplayOutputFormat = PF_FloatRGBA;
#else
		GRHIHDRDisplayOutputFormat = PF_A2B10G10R10;
#endif
	}

#if WITH_ENGINE
	FHardwareInfo::RegisterHardwareInfo(NAME_RHI, TEXT("D3D12"));
#endif

	GRHISupportsTextureStreaming = true;
	GRHISupportsFirstInstance = true;
	GRHISupportsShaderRootConstants = true;

	GRHIGlobals.ShaderBundles.SupportsDispatch = false; // Shader Bundles only implemented with Work Graphs
	GRHIGlobals.ShaderBundles.SupportsWorkGraphDispatch = GRHIGlobals.SupportsShaderWorkGraphsTier1;
	GRHIGlobals.ShaderBundles.SupportsWorkGraphGraphicsDispatch = GRHIGlobals.SupportsShaderWorkGraphsTier1_1;
	GRHIGlobals.ShaderBundles.SupportsParallel = false; // TODO: FD3D12ExplicitDescriptorHeap::UpdateSyncPoint() is not safe for parallel translate due to frame fencing

	// PC D3D12 support async dispatch rays calls
	if (GRHIGlobals.RayTracing.Supported)
	{
		GRHIGlobals.RayTracing.SupportsAsyncRayTraceDispatch = true;
		GRHIGlobals.RayTracing.SupportsPersistentSBTs = true;
	}

	GRHISupportsDynamicResolution = true;

	// Indicate that the RHI needs to use the engine's deferred deletion queue.
	GRHINeedsExtraDeletionLatency = true;

	// There is no need to defer deletion of streaming textures
	// - Suballocated ones are defer-deleted by their allocators
	// - Standalones are added to the deferred deletion queue of its parent FD3D12Adapter
	GRHIForceNoDeletionLatencyForStreamingTextures = !!PLATFORM_WINDOWS;

	if(Options6HR == S_OK && options.VariableShadingRateTier != D3D12_VARIABLE_SHADING_RATE_TIER_NOT_SUPPORTED && HardwareVariableRateShadingSupportedByPlatform(GMaxRHIShaderPlatform))
	{
		GRHISupportsPipelineVariableRateShading = true;		// We have at least tier 1.
		GRHISupportsLargerVariableRateShadingSizes = (options.AdditionalShadingRatesSupported != 0);

		if (options.VariableShadingRateTier == D3D12_VARIABLE_SHADING_RATE_TIER_2)
		{
			GRHISupportsAttachmentVariableRateShading = true;
			GRHISupportsComplexVariableRateShadingCombinerOps = true;

			GRHIVariableRateShadingImageTileMinWidth = options.ShadingRateImageTileSize;
			GRHIVariableRateShadingImageTileMinHeight = options.ShadingRateImageTileSize;
			GRHIVariableRateShadingImageTileMaxWidth = options.ShadingRateImageTileSize;
			GRHIVariableRateShadingImageTileMaxHeight = options.ShadingRateImageTileSize;

			GRHIVariableRateShadingImageDataType = VRSImage_Palette;
			GRHIVariableRateShadingImageFormat = PF_R8_UINT;
		}
	}
	else
	{
		GRHISupportsAttachmentVariableRateShading = GRHISupportsPipelineVariableRateShading = false;
		GRHIVariableRateShadingImageTileMinWidth = 1;
		GRHIVariableRateShadingImageTileMinHeight = 1;
		GRHIVariableRateShadingImageTileMaxWidth = 1;
		GRHIVariableRateShadingImageTileMaxHeight = 1;
	}

#if D3D12RHI_SUPPORTS_UNCOMPRESSED_UAV
	D3D12_FEATURE_DATA_D3D12_OPTIONS12 Options12{};
	if (SUCCEEDED(GetAdapter().GetD3DDevice()->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS12, &Options12, sizeof(Options12))))
	{
		GRHIGlobals.SupportsUAVFormatAliasing =
			(Options12.RelaxedFormatCastingSupported != 0)
			// Our BCn casting requires enhanced barrier support
			&& (Options12.EnhancedBarriersSupported != 0)
			// We require ID3D12Device12 for GetResourceAllocationInfo3
			&& GetAdapter().GetD3DDevice12() != nullptr
#if PLATFORM_WINDOWS
			// Make sure RenderDoc supports the new interfaces
			&& (D3D12RHI_IsRenderDocPresent(GetAdapter().GetD3DDevice()) == D3D12RHI_IsRenderDocPresent(GetAdapter().GetD3DDevice12()))
#endif
			;
	}
#else
	GRHIGlobals.SupportsUAVFormatAliasing = (GetAdapter().GetResourceHeapTier() > D3D12_RESOURCE_HEAP_TIER_1 && IsRHIDeviceNVIDIA());
#endif

	InitializeSubmissionPipe();

	FRenderResource::InitPreRHIResources();
	GIsRHIInitialized = true;
}

bool FD3D12DynamicRHI::IsQuadBufferStereoEnabled() const
{
	return bIsQuadBufferStereoEnabled;
}

void FD3D12DynamicRHI::DisableQuadBufferStereo()
{
	bIsQuadBufferStereoEnabled = false;
}

void FD3D12Device::CreateSamplerInternal(const D3D12_SAMPLER_DESC& Desc, D3D12_CPU_DESCRIPTOR_HANDLE Descriptor)
{
	GetDevice()->CreateSampler(&Desc, Descriptor);
}

void FD3D12Device::CopyDescriptors(ID3D12Device* D3DDevice, D3D12_CPU_DESCRIPTOR_HANDLE Destination, const D3D12_CPU_DESCRIPTOR_HANDLE* Source, uint32 NumSourceDescriptors, D3D12_DESCRIPTOR_HEAP_TYPE Type)
{
	D3DDevice->CopyDescriptors(1, &Destination, &NumSourceDescriptors, NumSourceDescriptors, Source, nullptr, Type);
}

#if D3D12_RHI_RAYTRACING
void FD3D12Device::GetRaytracingAccelerationStructurePrebuildInfo(
	const D3D12_BUILD_RAYTRACING_ACCELERATION_STRUCTURE_INPUTS* pDesc,
	D3D12_RAYTRACING_ACCELERATION_STRUCTURE_PREBUILD_INFO* pInfo)
{
	ID3D12Device5* RayTracingDevice = GetParentAdapter()->GetD3DDevice5();
	RayTracingDevice->GetRaytracingAccelerationStructurePrebuildInfo(pDesc, pInfo);
}

TRefCountPtr<ID3D12StateObject> FD3D12Device::DeserializeRayTracingStateObject(D3D12_SHADER_BYTECODE Bytecode, ID3D12RootSignature* RootSignature)
{
	checkNoEntry();
	TRefCountPtr<ID3D12StateObject> Result;
	return Result;
}

bool FD3D12Device::GetRayTracingPipelineInfo(ID3D12StateObject* Pipeline, FD3D12RayTracingPipelineInfo* OutInfo)
{
	// Return a safe default result on Windows, as there is no API to query interesting pipeline metrics.
	FD3D12RayTracingPipelineInfo Result = {};
	*OutInfo = Result;

	return false;
}

#endif // D3D12_RHI_RAYTRACING

static void GetAvailableResolutionsForOutput(IDXGIOutput* Output, FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	// TODO: GetDisplayModeList is a terribly SLOW call.  It can take up to a second per invocation.
	//  We might want to work around some DXGI badness here.
	DXGI_FORMAT Format = DXGI_FORMAT_R8G8B8A8_UNORM;
	uint32 NumModes = 0;
	HRESULT HResult = Output->GetDisplayModeList(Format, 0, &NumModes, nullptr);
	if (HResult == DXGI_ERROR_NOT_FOUND)
	{
		return;
	}

	if (HResult == DXGI_ERROR_NOT_CURRENTLY_AVAILABLE)
	{
		UE_LOGF(LogD3D12RHI, Warning,
			"RHIGetAvailableResolutions() can not be used over a remote desktop configuration"
		);
		return;
	}

	if (FAILED(HResult))
	{
		UE_LOGF(LogD3D12RHI, Warning, "RHIGetAvailableResolutions failed with unknown error (0x%x).", HResult);
		return;
	}

	if (NumModes > 0)
	{
		TArray<DXGI_MODE_DESC> ModeList;
		do
		{
			VERIFYD3D12RESULT(Output->GetDisplayModeList(Format, 0, &NumModes, nullptr));
			ModeList.SetNum(NumModes);
			HResult = Output->GetDisplayModeList(Format, 0, &NumModes, ModeList.GetData());
		} while (HResult == DXGI_ERROR_MORE_DATA);
		VERIFYD3D12RESULT(HResult);

		constexpr int32 MinAllowableResolutionX = 0;
		constexpr int32 MinAllowableResolutionY = 0;
		constexpr int32 MaxAllowableResolutionX = 10480;
		constexpr int32 MaxAllowableResolutionY = 10480;
		constexpr int32 MinAllowableRefreshRate = 0;
		constexpr int32 MaxAllowableRefreshRate = 10480;
		for (uint32 m = 0; m < NumModes; m++)
		{
			CA_SUPPRESS(6385);
			if (((int32)ModeList[m].Width >= MinAllowableResolutionX) &&
				((int32)ModeList[m].Width <= MaxAllowableResolutionX) &&
				((int32)ModeList[m].Height >= MinAllowableResolutionY) &&
				((int32)ModeList[m].Height <= MaxAllowableResolutionY)
				)
			{
				bool bAddIt = true;
				if (bIgnoreRefreshRate == false)
				{
					if (((int32)ModeList[m].RefreshRate.Numerator < MinAllowableRefreshRate * ModeList[m].RefreshRate.Denominator) ||
						((int32)ModeList[m].RefreshRate.Numerator > MaxAllowableRefreshRate * ModeList[m].RefreshRate.Denominator)
						)
					{
						continue;
					}
				}
				else
				{
					// See if it is in the list already
					for (int32 CheckIndex = 0; CheckIndex < Resolutions.Num(); CheckIndex++)
					{
						FScreenResolutionRHI& CheckResolution = Resolutions[CheckIndex];
						if ((CheckResolution.Width == ModeList[m].Width) &&
							(CheckResolution.Height == ModeList[m].Height))
						{
							// Already in the list...
							bAddIt = false;
							break;
						}
					}
				}

				if (bAddIt)
				{
					FScreenResolutionRHI& ScreenResolution = Resolutions.Emplace_GetRef();
					ScreenResolution.Width = ModeList[m].Width;
					ScreenResolution.Height = ModeList[m].Height;
					ScreenResolution.RefreshRate = ModeList[m].RefreshRate.Numerator / ModeList[m].RefreshRate.Denominator;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogD3D12RHI, Warning, "No display modes found for the standard format DXGI_FORMAT_R8G8B8A8_UNORM!");
	}
}

/**
 *	Retrieve available screen resolutions.
 *
 *	@param	Resolutions			TArray<FScreenResolutionRHI> parameter that will be filled in.
 *	@param	bIgnoreRefreshRate	If true, ignore refresh rates.
 *
 *	@return	bool				true if successfully filled the array
 */
bool FD3D12DynamicRHI::RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	// TODO: The resolution information can be stale if the adapter's factory isn't current. Use RHIGetAvailableResolutionsForDisplay instead.
	HRESULT HResult = S_OK;
	TRefCountPtr<IDXGIAdapter> Adapter;
	//TODO: should only be called on display out device
	HResult = ChosenAdapter->EnumAdapters(Adapter.GetInitReference());

	if (DXGI_ERROR_NOT_FOUND == HResult)
	{
		return false;
	}
	if (FAILED(HResult))
	{
		return false;
	}

	// get the description of the adapter
	DXGI_ADAPTER_DESC AdapterDesc;
	if (FAILED(Adapter->GetDesc(&AdapterDesc)))
	{
		return false;
	}

	TRefCountPtr<IDXGIOutput> Output;
	HResult = Adapter->EnumOutputs(0, Output.GetInitReference());
	if (SUCCEEDED(HResult))
	{
		GetAvailableResolutionsForOutput(Output, Resolutions, bIgnoreRefreshRate);
	}
	return !Resolutions.IsEmpty();
}

bool FD3D12DynamicRHI::RHIGetAvailableResolutionsForDisplay(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate, void* NativeMonitorHandle)
{
	HMONITOR MonitorHandle = static_cast<HMONITOR>(NativeMonitorHandle);
	if (MonitorHandle == NULL)
	{
		return false;
	}

	if (!DXGIFactoryForResolutions.IsValid() || !DXGIFactoryForResolutions->IsCurrent())
	{
		FD3D12Adapter::CreateDXGIFactory(DXGIFactoryForResolutions, false);
	}

	HRESULT HResult;
	TRefCountPtr<IDXGIAdapter> Adapter;
	for (int32 CurrentAdapter = 0; (HResult = DXGIFactoryForResolutions->EnumAdapters(CurrentAdapter, Adapter.GetInitReference())) != DXGI_ERROR_NOT_FOUND; ++CurrentAdapter)
	{
		if (FAILED(HResult))
		{
			return false;
		}

		TRefCountPtr<IDXGIOutput> Output;
		for (int32 CurrentOutput = 0; (HResult = Adapter->EnumOutputs(CurrentOutput, Output.GetInitReference())) != DXGI_ERROR_NOT_FOUND; ++CurrentOutput)
		{
			if (FAILED(HResult))
			{
				return false;
			}

			DXGI_OUTPUT_DESC OutputDesc{};
			HRESULT OutputDescRes = Output->GetDesc(&OutputDesc);
			check(SUCCEEDED(OutputDescRes));
			if (MonitorHandle == OutputDesc.Monitor)
			{
				GetAvailableResolutionsForOutput(Output, Resolutions, bIgnoreRefreshRate);

				// Only allow the desktop resolution if SetFullscreenState is expected to fail to allow borderless to seemlessly take over.
				// Undocumented but assume exclusive only works if the the display is primary, i.e. top-left at (0, 0), or the display is connected to the chosen adapter.
				if (OutputDesc.DesktopCoordinates.left != 0 || OutputDesc.DesktopCoordinates.top != 0)
				{
					if (ChosenAdapter.IsValid())
					{
						const FD3D12AdapterDesc& ChosenDesc = ChosenAdapter->GetDesc();
						if (ChosenDesc.IsValid())
						{
							DXGI_ADAPTER_DESC AdapterDesc;
							if (SUCCEEDED(Adapter->GetDesc(&AdapterDesc)) &&
								FMemory::Memcmp(&ChosenAdapter->GetDesc().Desc.AdapterLuid, &AdapterDesc.AdapterLuid, sizeof(LUID)))
							{
								Resolutions.Reset(1);
								FScreenResolutionRHI& ScreenResolution = Resolutions.Emplace_GetRef();
								ScreenResolution.Width = OutputDesc.DesktopCoordinates.right - OutputDesc.DesktopCoordinates.left;
								ScreenResolution.Height = OutputDesc.DesktopCoordinates.bottom - OutputDesc.DesktopCoordinates.top;
								ScreenResolution.RefreshRate = 0;
							}
						}
					}
				}

				return !Resolutions.IsEmpty();
			}
		}
	}

	return false;
}

void FWindowsD3D12Adapter::CreateCommandSignatures()
{
	ID3D12Device* Device = GetD3DDevice();

#if D3D12_RHI_RAYTRACING

	if (GRHISupportsRayTracing)
	{
		D3D12_FEATURE_DATA_D3D12_OPTIONS5 D3D12Caps5 = {};
		if (SUCCEEDED(Device->CheckFeatureSupport(D3D12_FEATURE_D3D12_OPTIONS5, &D3D12Caps5, sizeof(D3D12Caps5))))
		{
			if (D3D12Caps5.RaytracingTier >= D3D12_RAYTRACING_TIER_1_1)
			{
				GRHISupportsRayTracingDispatchIndirect = true;
			}
		}
	}

	if (GRHISupportsRayTracingDispatchIndirect)
	{
		D3D12_COMMAND_SIGNATURE_DESC SignatureDesc = {};
		SignatureDesc.NumArgumentDescs = 1;
		SignatureDesc.ByteStride = sizeof(D3D12_DISPATCH_RAYS_DESC);
		SignatureDesc.NodeMask = FRHIGPUMask::All().GetNative();

		D3D12_INDIRECT_ARGUMENT_DESC ArgumentDesc[1] = {};
		ArgumentDesc[0].Type = D3D12_INDIRECT_ARGUMENT_TYPE_DISPATCH_RAYS;
		SignatureDesc.pArgumentDescs = ArgumentDesc;

		checkf(DispatchRaysIndirectCommandSignature == nullptr, TEXT("Indirect ray tracing dispatch command signature is expected to be initialized by FWindowsD3D12Adapter."));
		VERIFYD3D12RESULT(Device->CreateCommandSignature(&SignatureDesc, nullptr, IID_PPV_ARGS(DispatchRaysIndirectCommandSignature.GetInitReference())));
	}

#endif // D3D12_RHI_RAYTRACING

	// Create windows-specific indirect dispatch command signatures
	{
		D3D12_COMMAND_SIGNATURE_DESC CommandSignatureDesc = {};
		CommandSignatureDesc.NumArgumentDescs = 1;
		CommandSignatureDesc.ByteStride = sizeof(D3D12_DISPATCH_ARGUMENTS);
		CommandSignatureDesc.NodeMask = FRHIGPUMask::All().GetNative();

		D3D12_INDIRECT_ARGUMENT_DESC IndirectParameterDesc[1] = {};
		IndirectParameterDesc[0].Type = D3D12_INDIRECT_ARGUMENT_TYPE_DISPATCH;
		CommandSignatureDesc.pArgumentDescs = IndirectParameterDesc;

		checkf(DispatchIndirectGraphicsCommandSignature == nullptr, TEXT("Indirect graphics dispatch command signature is expected to be initialized by FWindowsD3D12Adapter."));
		VERIFYD3D12RESULT(Device->CreateCommandSignature(&CommandSignatureDesc, nullptr, IID_PPV_ARGS(DispatchIndirectGraphicsCommandSignature.GetInitReference())));

		checkf(DispatchIndirectComputeCommandSignature == nullptr, TEXT("Indirect compute dispatch command signature is expected to be initialized by FWindowsD3D12Adapter."));
		VERIFYD3D12RESULT(Device->CreateCommandSignature(&CommandSignatureDesc, nullptr, IID_PPV_ARGS(DispatchIndirectComputeCommandSignature.GetInitReference())));

#if PLATFORM_SUPPORTS_MESH_SHADERS
		if (GRHISupportsMeshShadersTier0)
		{
			IndirectParameterDesc[0].Type = D3D12_INDIRECT_ARGUMENT_TYPE_DISPATCH_MESH;
			CommandSignatureDesc.ByteStride = sizeof(D3D12_DISPATCH_ARGUMENTS);
			VERIFYD3D12RESULT(Device->CreateCommandSignature(&CommandSignatureDesc, nullptr, IID_PPV_ARGS(DispatchIndirectMeshCommandSignature.GetInitReference())));
		}
#endif
	}

	// Create all the generic / cross-platform command signatures

	FD3D12Adapter::CreateCommandSignatures();
}

FD3D12DiagnosticBuffer::FD3D12DiagnosticBuffer(FD3D12Queue& Queue)
{
	const static FLazyName D3D12DiagnosticBufferName(TEXT("FD3D12DiagnosticBuffer"));
	const static FLazyName CreateDiagnosticBufferName(TEXT("FD3D12Device::CreateDiagnosticBuffer"));
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(D3D12DiagnosticBufferName, CreateDiagnosticBufferName, NAME_None);

	// Create the platform-specific diagnostic buffer
	FString Name = FString::Printf(TEXT("DiagnosticBuffer (Queue: 0x%p)"), &Queue);

	extern TAutoConsoleVariable<int32> CVarD3D12ExtraDiagnosticBufferMemory;
	const D3D12_RESOURCE_DESC Desc = CD3DX12_RESOURCE_DESC::Buffer(SizeInBytes + FMath::Max(0, CVarD3D12ExtraDiagnosticBufferMemory.GetValueOnAnyThread()), D3D12_RESOURCE_FLAG_ALLOW_CROSS_ADAPTER);

	ID3D12Device3* D3D12Device3 = Queue.Device->GetParentAdapter()->GetD3DDevice3();
	if (!D3D12Device3)
	{
		UE_LOGF(LogD3D12RHI, Warning, "[GPUBreadCrumb] ID3D12Device3 not available (only available on Windows 10 1709+)");
		return;
	}

	D3D12_FEATURE_DATA_EXISTING_HEAPS ExistingHeapSupport{};
	if (FAILED(D3D12Device3->CheckFeatureSupport(D3D12_FEATURE_EXISTING_HEAPS, &ExistingHeapSupport, sizeof(ExistingHeapSupport))) || !ExistingHeapSupport.Supported)
	{
		UE_LOGF(LogD3D12RHI, Warning, "[GPUBreadCrumb] D3D12_FEATURE_EXISTING_HEAPS is not supported.");
		return;
	}

	// Allocate persistent CPU readable memory which will still be valid after a device lost and wrap this data in a placed resource so the GPU command list can write to it
	Data = static_cast<FQueue*>(VirtualAlloc(nullptr, Desc.Width, MEM_COMMIT | MEM_RESERVE, PAGE_READWRITE));
	if (!Data)
	{
		UE_LOGF(LogD3D12RHI, Warning, "[GPUBreadCrumb] Failed to VirtualAlloc resource memory");
		return;
	}

	FMemory::Memzero(Data, Desc.Width);

	ID3D12Heap* D3D12Heap = nullptr;
	HRESULT hr = D3D12Device3->OpenExistingHeapFromAddress(Data, IID_PPV_ARGS(&D3D12Heap));
	if (FAILED(hr))
	{
		VirtualFree(Data, 0, MEM_RELEASE);
		Data = nullptr;

		UE_LOGF(LogD3D12RHI, Warning, "[GPUBreadCrumb] Failed to OpenExistingHeapFromAddress, error: %x", hr);
		return;
	}
	
	Heap = new FD3D12Heap(Queue.Device, Queue.Device->GetGPUMask());
	Heap->SetHeap(D3D12Heap, TEXT("DiagnosticBuffer"));

	hr = Queue.Device->GetParentAdapter()->CreatePlacedResource(
		Desc,
		Heap.GetReference(),
		0,
		ED3D12Access::CopyDest,
		nullptr,
		Resource.GetInitReference(),
		*Name,
		false);

	if (FAILED(hr))
	{
		Heap.SafeRelease();
		VirtualFree(Data, 0, MEM_RELEASE);
		Data = nullptr;

		UE_LOGF(LogD3D12RHI, Warning, "[GPUBreadCrumb] Failed to CreatePlacedResource, error: %x", hr);
		return;
	}

	UE_LOGF(LogD3D12RHI, Log, "[GPUBreadCrumb] Successfully setup breadcrumb resource for %ls", *Name);

	GpuAddress = Resource->GetGPUVirtualAddress();

#if WITH_RHI_BREADCRUMBS
	Data->MarkerIn = 0;
	Data->MarkerOut = 0;
#endif
}

FD3D12DiagnosticBuffer::~FD3D12DiagnosticBuffer()
{
	Resource.SafeRelease();
	Heap.SafeRelease();

	if (Data)
	{
		VirtualFree(Data, 0, MEM_RELEASE);
		Data = nullptr;
	}

	GpuAddress = 0;
}

void FD3D12DynamicRHI::ProcessDeferredDeletionQueue_Platform()
{
	// Nothing Windows-specific here.
}

HRESULT FD3D12Device::CreateCommandList(
	UINT                    nodeMask,
	D3D12_COMMAND_LIST_TYPE type,
	ID3D12CommandAllocator* pCommandAllocator,
	ID3D12PipelineState*    pInitialState,
	REFIID                  riid,
	void**                  ppCommandList
)
{
	return GetDevice()->CreateCommandList(
		nodeMask,
		type,
		pCommandAllocator,
		pInitialState,
		riid,
		ppCommandList
	);
}

void FD3D12Queue::ExecuteCommandLists(TArrayView<ID3D12CommandList*> D3DCommandLists
#if ENABLE_RESIDENCY_MANAGEMENT
	, TArrayView<FD3D12ResidencySet*> ResidencySets
#endif
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FD3D12Queue::ExecuteCommandLists);

#if ENABLE_RESIDENCY_MANAGEMENT
	check(D3DCommandLists.Num() == ResidencySets.Num());

	if (GEnableResidencyManagement)
	{
		VERIFYD3D12RESULT(Device->GetResidencyManager().ExecuteCommandLists(
			D3DCommandQueue,
			D3DCommandLists.GetData(),
			ResidencySets.GetData(),
			D3DCommandLists.Num()
		));
	}
	else
#endif
	{
		D3DCommandQueue->ExecuteCommandLists(
			D3DCommandLists.Num(),
			D3DCommandLists.GetData()
		);
	}
}
