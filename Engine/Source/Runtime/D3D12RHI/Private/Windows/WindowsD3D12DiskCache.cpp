// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of D3D12 Disk caching functions to preserve state across runs

//-----------------------------------------------------------------------------
//	Include Files
//-----------------------------------------------------------------------------
#include "WindowsD3D12DiskCache.h"
#include "D3D12RHIPrivate.h"

void FDiskCacheInterface::Init(FString &filename, bool bEnable)
{
	mFileStart = nullptr;
	mFile = 0;
	mMemoryMap = 0;
	mMapAddress = 0;
	mCurrentFileMapSize = 0;
	mCurrentOffset = 0;
	mInErrorState = false;
	mEnableDiskCache = bEnable;

	mFileName = filename;
	mCacheExists = true;
	if (!mEnableDiskCache)
	{
		mInErrorState = true;
		mCacheExists = false;
	}
	else
	{
		WIN32_FIND_DATA fileData;
		HANDLE Handle = FindFirstFile(mFileName.GetCharArray().GetData(), &fileData);
		if (Handle == INVALID_HANDLE_VALUE)
		{
			if (GetLastError() == ERROR_FILE_NOT_FOUND)
			{
				mCacheExists = false;
			}
		}
		else
		{
			FindClose(Handle);
		}
	}
	bool fileFound = mCacheExists;
	GrowMapping(64 * 1024, true);

	if (fileFound && mFileStart)
	{
		mHeader = *(FDiskCacheHeader*)mFileStart;
		if (mHeader.mHeaderVersion != mCurrentHeaderVersion)
		{
			UE_LOGF(LogD3D12RHI, Warning, "Disk cache is stale. Disk Cache version: %d App version: %d", mHeader.mHeaderVersion, mCurrentHeaderVersion);
			ClearAndReinitialize();
		}
	}
	else
	{
		mHeader.mHeaderVersion = mCurrentHeaderVersion;
		mHeader.mNumPsos = 0;
		mHeader.mSizeInBytes = 0;
	}
}

void FDiskCacheInterface::GrowMapping(SIZE_T size, bool firstrun)
{
	if (IsInErrorState())
	{
		return;
	}

	if (mCurrentOffset + size > mCurrentFileMapSize)
	{
		mCurrentFileMapSize = Align(mCurrentOffset + size, mFileGrowSize);
	}
	else
	{
		return;
	}

	if (mMapAddress)
	{
		FlushViewOfFile(mMapAddress, mCurrentOffset);
		UnmapViewOfFile(mMapAddress);
	}
	if (mMemoryMap)
	{
		CloseHandle(mMemoryMap);
	}
	if (mFile)
	{
		CloseHandle(mFile);
	}

	uint32 flag = (mCacheExists) ? OPEN_EXISTING : CREATE_NEW;
	// open the shader cache file
	mFile = CreateFile(mFileName.GetCharArray().GetData(), GENERIC_READ | GENERIC_WRITE, FILE_SHARE_READ, NULL, flag, FILE_ATTRIBUTE_NORMAL, NULL);

	if (mFile == INVALID_HANDLE_VALUE)
	{
		//error state!
		mInErrorState = true;
		return;
	}

	mCacheExists = true;

	uint32 fileSize = GetFileSize(mFile, NULL);

	if (fileSize == 0)
	{
		uint8 data[64];
		FMemory::Memzero(data);
		//It's invalide to map a zero sized file so write some junk data in that case
		WriteFile(mFile, data, sizeof(data), NULL, NULL);
	}
	else if (firstrun)
	{
		mCurrentFileMapSize = fileSize;
	}

	mMemoryMap = CreateFileMapping(mFile, NULL, PAGE_READWRITE, (uint32)(mCurrentFileMapSize >> 32), (uint32)(mCurrentFileMapSize & 0xFFFFFFFF), NULL);
	if (mMemoryMap == (HANDLE)nullptr)
	{
		//error state!
		mInErrorState = true;
		ClearDiskCache();
		return;
	}

	mMapAddress = MapViewOfFile(mMemoryMap, FILE_MAP_ALL_ACCESS, 0, 0, mCurrentFileMapSize);
	if (mMapAddress == (HANDLE)nullptr)
	{
		//error state!
		mInErrorState = true;
		ClearDiskCache();
		return;
	}

	mFileStart = (uint8*)mMapAddress;
}

bool FDiskCacheInterface::AppendData(const void* pData, size_t size)
{
	GrowMapping(size, false);
	if (IsInErrorState())
	{
		return false;
	}

	memcpy((mFileStart + mCurrentOffset), pData, size);
	mCurrentOffset += size;

	return true;
}

bool FDiskCacheInterface::SetPointerAndAdvanceFilePosition(void** pDest, size_t size, bool backWithSystemMemory)
{
	GrowMapping(size, false);
	if (IsInErrorState())
	{
		return false;
	}

	// Back persistent objects in system memory to avoid
	// troubles when re-mapping the file.
	if (backWithSystemMemory)
	{
		// Optimization: most (all?) of the shader input layout semantic names are "ATTRIBUTE"...
		// instead of making 1000's of attribute strings, just set it to a static one.
		static const char attribute[] = "ATTRIBUTE";
		if (size == sizeof(attribute) && FMemory::Memcmp((mFileStart + mCurrentOffset), attribute, sizeof(attribute)) == 0)
		{
			*pDest = (void*)attribute;
		}
		else
		{
			void* newMemory = FMemory::Malloc(size);
			if (newMemory)
			{
				FMemory::Memcpy(newMemory, (mFileStart + mCurrentOffset), size);
				mBackedMemory.Add(newMemory);
				*pDest = newMemory;
			}
			else
			{
				check(false);
				return false;
			}
		}
	}
	else
	{
		*pDest = mFileStart + mCurrentOffset;
	}

	mCurrentOffset += size;

	return true;
}

void FDiskCacheInterface::Reset(RESET_TYPE type)
{
	mCurrentOffset = sizeof(FDiskCacheHeader);

	if (type == RESET_TO_AFTER_LAST_OBJECT)
	{
		mCurrentOffset += mHeader.mSizeInBytes;
	}
}

void FDiskCacheInterface::Close(uint32 numberOfPSOs)
{
	mHeader.mNumPsos = numberOfPSOs;

	check(mCurrentOffset >= sizeof(FDiskCacheHeader));
	mHeader.mSizeInBytes = mCurrentOffset - sizeof(FDiskCacheHeader);

	if (!IsInErrorState())
	{
		if (mMapAddress)
		{
			*(FDiskCacheHeader*)mFileStart = mHeader;
			FlushViewOfFile(mMapAddress, mCurrentOffset);
			UnmapViewOfFile(mMapAddress);
		}
		if (mMemoryMap)
		{
			CloseHandle(mMemoryMap);
		}
		if (mFile)
		{
			CloseHandle(mFile);
		}
	}
}

void FDiskCacheInterface::ClearDiskCache()
{
	// Prevent reads/writes 
	mInErrorState = true;
	mHeader.mHeaderVersion = mCurrentHeaderVersion;
	mHeader.mNumPsos = 0;

	if (!mEnableDiskCache)
	{
		return;
	}

	if (mMapAddress)
	{
		UnmapViewOfFile(mMapAddress);
	}
	if (mMemoryMap)
	{
		CloseHandle(mMemoryMap);
	}
	if (mFile)
	{
		CloseHandle(mFile);
	}
#ifdef UNICODE
	BOOL result = DeleteFileW(mFileName.GetCharArray().GetData());
#else
	bool result = DeleteFileA(mFileName.GetCharArray().GetData());
#endif
	UE_LOGF(LogD3D12RHI, Warning, "Deleted PSO Cache with result %d", result);
}

void FDiskCacheInterface::Flush(uint32 numberOfPSOs)
{
	mHeader.mNumPsos = numberOfPSOs;

	check(mCurrentOffset >= sizeof(FDiskCacheHeader));
	mHeader.mSizeInBytes = mCurrentOffset - sizeof(FDiskCacheHeader);

	if (mMapAddress && !IsInErrorState())
	{
		*(FDiskCacheHeader*)mFileStart = mHeader;
		FlushViewOfFile(mMapAddress, mCurrentOffset);
	}
}

void* FDiskCacheInterface::GetDataAt(SIZE_T Offset) const
{
	void* data = mFileStart + Offset;

	check(data <= (mFileStart + mCurrentFileMapSize));
	return data;
}

void* FDiskCacheInterface::GetDataAtStart() const
{
	return GetDataAt(sizeof(FDiskCacheHeader));
}
