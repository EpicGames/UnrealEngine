// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of D3D12 Disk caching functions to preserve state across runs

#pragma once

#include "HAL/Platform.h"
#include "Containers/UnrealString.h"
#include "Windows/WindowsHWrapper.h"

#define IL_MAX_SEMANTIC_NAME 255

class FDiskCacheInterface
{
	// Increment this if changes are made to the
	// disk caches so stale caches get updated correctly
	static const uint32 mCurrentHeaderVersion = 11;
	struct FDiskCacheHeader
	{
		uint32 mHeaderVersion;
		uint32 mNumPsos;
		size_t mSizeInBytes; // The number of bytes after the header
	};

private:
	FString mFileName;
	uint8*  mFileStart;
	HANDLE  mFile;
	HANDLE  mMemoryMap;
	HANDLE  mMapAddress;
	SIZE_T  mCurrentFileMapSize;
	SIZE_T  mCurrentOffset;
	bool    mCacheExists;
	bool    mInErrorState;
	bool    mEnableDiskCache;
	FDiskCacheHeader mHeader;

	// There is the potential for the file mapping to grow
	// in that case all of the pointers will be invalid. Back
	// some of the pointers we might read again (i.e. shade byte
	// code for PSO mapping) in persitent system memory.
	TArray<void*> mBackedMemory;

	static const SIZE_T mFileGrowSize = (1024 * 1024); // 1 megabyte;

	void GrowMapping(SIZE_T size, bool firstrun);

public:
	enum RESET_TYPE
	{
		RESET_TO_FIRST_OBJECT,
		RESET_TO_AFTER_LAST_OBJECT
	};

	bool AppendData(const void* pData, size_t size);
	bool SetPointerAndAdvanceFilePosition(void** pDest, size_t size, bool backWithSystemMemory = false);
	void Reset(RESET_TYPE type);
	void Init(FString &filename, bool bEnable = true);
	void Close(uint32 numberOfPSOs);
	void Flush(uint32 numberOfPSOs);
	void ClearDiskCache();

	void ClearAndReinitialize()
	{
		// Must call the normal Init() first.
		if (ensure(!mFileName.IsEmpty()))
		{
			ClearDiskCache();
			Init(mFileName, mEnableDiskCache);
		}
	}

	uint32 GetNumPSOs() const
	{
		return mHeader.mNumPsos;
	}

	uint32 GetSizeInBytes() const
	{
		return mHeader.mSizeInBytes;
	}
	
	FORCEINLINE_DEBUGGABLE bool IsInErrorState() const
	{
		return mInErrorState;
	}

	SIZE_T GetCurrentOffset() const { return mCurrentOffset; }

	void* GetDataAt(SIZE_T Offset) const;
	void* GetDataAtStart() const;

	~FDiskCacheInterface()
	{
		for (void* memory : mBackedMemory)
		{
			if (memory)
			{
				FMemory::Free(memory);
			}
		}
	}
};