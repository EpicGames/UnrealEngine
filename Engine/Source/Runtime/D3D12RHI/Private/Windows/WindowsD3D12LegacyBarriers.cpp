// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12RHIDefinitions.h"

#if D3D12RHI_SUPPORTS_LEGACY_BARRIERS

#include "D3D12ThirdParty.h"

D3D12_RESOURCE_STATES GetSkipFastClearEliminateStateFlags()
{
	return {};
}

#endif // D3D12RHI_SUPPORTS_LEGACY_BARRIERS