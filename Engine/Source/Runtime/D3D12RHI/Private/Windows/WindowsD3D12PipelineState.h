// Copyright Epic Games, Inc. All Rights Reserved.

// Implementation of D3D12 Pipelinestate related functions

#pragma once

#include "D3D12RHICommon.h"

class FD3D12RootSignature; // forward-declare
struct FD3D12ComputePipelineStateDesc;
struct FD3D12LowLevelGraphicsPipelineStateDesc;

// Graphics pipeline stream struct that represents the latest version of PSO subobjects currently used by the RHI.
struct FD3D12_GRAPHICS_PIPELINE_STATE_STREAM
{
	// Note: Unused members are currently commented out to exclude them from the stream.
	// This results in a smaller struct and thus fewer tokens to parse at runtime. Feel free to add/change as necessary.

	//CD3DX12_PIPELINE_STATE_STREAM_FLAGS Flags;
	CD3DX12_PIPELINE_STATE_STREAM_NODE_MASK NodeMask;
	CD3DX12_PIPELINE_STATE_STREAM_ROOT_SIGNATURE pRootSignature;
	CD3DX12_PIPELINE_STATE_STREAM_INPUT_LAYOUT InputLayout;
	CD3DX12_PIPELINE_STATE_STREAM_IB_STRIP_CUT_VALUE IBStripCutValue;
	CD3DX12_PIPELINE_STATE_STREAM_PRIMITIVE_TOPOLOGY PrimitiveTopologyType;
	CD3DX12_PIPELINE_STATE_STREAM_VS VS;
	CD3DX12_PIPELINE_STATE_STREAM_GS GS;
	CD3DX12_PIPELINE_STATE_STREAM_HS HS;
	CD3DX12_PIPELINE_STATE_STREAM_DS DS;
	CD3DX12_PIPELINE_STATE_STREAM_PS PS;
	CD3DX12_PIPELINE_STATE_STREAM_BLEND_DESC BlendState;
	CD3DX12_PIPELINE_STATE_STREAM_DEPTH_STENCIL1 DepthStencilState;
	CD3DX12_PIPELINE_STATE_STREAM_DEPTH_STENCIL_FORMAT DSVFormat;
	CD3DX12_PIPELINE_STATE_STREAM_RASTERIZER RasterizerState;
	CD3DX12_PIPELINE_STATE_STREAM_RENDER_TARGET_FORMATS RTVFormats;
	CD3DX12_PIPELINE_STATE_STREAM_SAMPLE_DESC SampleDesc;
	CD3DX12_PIPELINE_STATE_STREAM_SAMPLE_MASK SampleMask;
	CD3DX12_PIPELINE_STATE_STREAM_CACHED_PSO CachedPSO;
	//CD3DX12_PIPELINE_STATE_STREAM_VIEW_INSTANCING ViewInstancingDesc;
};

#if PLATFORM_SUPPORTS_MESH_SHADERS
struct FD3D12_MESH_PIPELINE_STATE_STREAM
{
	// Note: Unused members are currently commented out to exclude them from the stream.
	// This results in a smaller struct and thus fewer tokens to parse at runtime. Feel free to add/change as necessary.

	//CD3DX12_PIPELINE_STATE_STREAM_FLAGS Flags;
	CD3DX12_PIPELINE_STATE_STREAM_NODE_MASK NodeMask;
	CD3DX12_PIPELINE_STATE_STREAM_ROOT_SIGNATURE pRootSignature;
	CD3DX12_PIPELINE_STATE_STREAM_PRIMITIVE_TOPOLOGY PrimitiveTopologyType;
	CD3DX12_PIPELINE_STATE_STREAM_MS MS;
	CD3DX12_PIPELINE_STATE_STREAM_AS AS;
	CD3DX12_PIPELINE_STATE_STREAM_PS PS;
	CD3DX12_PIPELINE_STATE_STREAM_BLEND_DESC BlendState;
	CD3DX12_PIPELINE_STATE_STREAM_DEPTH_STENCIL1 DepthStencilState;
	CD3DX12_PIPELINE_STATE_STREAM_DEPTH_STENCIL_FORMAT DSVFormat;
	CD3DX12_PIPELINE_STATE_STREAM_RASTERIZER RasterizerState;
	CD3DX12_PIPELINE_STATE_STREAM_RENDER_TARGET_FORMATS RTVFormats;
	CD3DX12_PIPELINE_STATE_STREAM_SAMPLE_DESC SampleDesc;
	CD3DX12_PIPELINE_STATE_STREAM_SAMPLE_MASK SampleMask;
	CD3DX12_PIPELINE_STATE_STREAM_CACHED_PSO CachedPSO;
	//CD3DX12_PIPELINE_STATE_STREAM_VIEW_INSTANCING ViewInstancingDesc;
};
#endif

// Compute pipeline stream struct that represents the latest version of PSO subobjects currently used by the RHI.
struct FD3D12_COMPUTE_PIPELINE_STATE_STREAM
{
	// Note: Unused members are currently commented out to exclude them from the stream.
	// This results in a smaller struct and thus fewer tokens to parse at runtime. Feel free to add/change as necessary.

	//CD3DX12_PIPELINE_STATE_STREAM_FLAGS Flags;
	CD3DX12_PIPELINE_STATE_STREAM_NODE_MASK NodeMask;
	CD3DX12_PIPELINE_STATE_STREAM_ROOT_SIGNATURE pRootSignature;
	CD3DX12_PIPELINE_STATE_STREAM_CS CS;
	CD3DX12_PIPELINE_STATE_STREAM_CACHED_PSO CachedPSO;
};

struct ComputePipelineCreationArgs_POD;
struct GraphicsPipelineCreationArgs_POD;

#include "D3D12PipelineState.h"

void SaveByteCode(D3D12_SHADER_BYTECODE& ByteCode);

struct ComputePipelineCreationArgs_POD
{
	FD3D12ComputePipelineStateDesc Desc;
	ID3D12PipelineLibrary* Library;

	void Init(const ComputePipelineCreationArgs_POD& InArgs)
	{
		Desc = InArgs.Desc;
		Library = InArgs.Library;

		SaveByteCode(Desc.Desc.CS);
	}

	void Destroy();
};

struct GraphicsPipelineCreationArgs_POD
{
	FD3D12LowLevelGraphicsPipelineStateDesc Desc;
	ID3D12PipelineLibrary* Library;

	void Init(const GraphicsPipelineCreationArgs_POD& InArgs)
	{
		Desc = InArgs.Desc;
		Library = InArgs.Library;

		SaveByteCode(Desc.Desc.VS);
		SaveByteCode(Desc.Desc.MS);
		SaveByteCode(Desc.Desc.AS);
		SaveByteCode(Desc.Desc.PS);
		SaveByteCode(Desc.Desc.GS);
	}

	void Destroy();
};

struct ComputePipelineCreationArgs
{
	ComputePipelineCreationArgs(const FD3D12ComputePipelineStateDesc* InDesc, ID3D12PipelineLibrary* InLibrary)
	{
		Args.Desc = *InDesc;
		Args.Library = InLibrary;
	}

	ComputePipelineCreationArgs(const ComputePipelineCreationArgs& InArgs)
		: ComputePipelineCreationArgs(&InArgs.Args.Desc, InArgs.Args.Library)
	{}

	ComputePipelineCreationArgs_POD Args;
};

struct GraphicsPipelineCreationArgs
{
	GraphicsPipelineCreationArgs(const FD3D12LowLevelGraphicsPipelineStateDesc* InDesc, ID3D12PipelineLibrary* InLibrary)
	{
		Args.Desc = *InDesc;
		Args.Library = InLibrary;
	}

	GraphicsPipelineCreationArgs(const GraphicsPipelineCreationArgs& InArgs)
		: GraphicsPipelineCreationArgs(&InArgs.Args.Desc, InArgs.Args.Library)
	{}

	GraphicsPipelineCreationArgs_POD Args;
};

class FD3D12PipelineStateCache : public FD3D12PipelineStateCacheBase
{
private:
	FDiskCacheInterface DiskBinaryCache;
	TRefCountPtr<ID3D12PipelineLibrary> PipelineLibrary;
	bool bUseAPILibaries;

	void WriteOutShaderBlob(PSO_CACHE_TYPE Cache, ID3D12PipelineState* APIPso);

	template<typename PipelineStateDescType>
	void ReadBackShaderBlob(PipelineStateDescType& Desc, PSO_CACHE_TYPE Cache)
	{
		SIZE_T* cachedBlobOffset = nullptr;
		DiskCaches[Cache].SetPointerAndAdvanceFilePosition((void**)&cachedBlobOffset, sizeof(SIZE_T));

		SIZE_T* cachedBlobSize = nullptr;
		DiskCaches[Cache].SetPointerAndAdvanceFilePosition((void**)&cachedBlobSize, sizeof(SIZE_T));

		check(cachedBlobOffset);
		check(cachedBlobSize);

		if (UseCachedBlobs())
		{
			check(*cachedBlobSize);
			Desc.CachedPSO.CachedBlobSizeInBytes = *cachedBlobSize;
			Desc.CachedPSO.pCachedBlob = DiskBinaryCache.GetDataAt(*cachedBlobOffset);
		}
		else
		{
			Desc.CachedPSO.CachedBlobSizeInBytes = 0;
			Desc.CachedPSO.pCachedBlob = nullptr;
		}
	}

	bool UsePipelineLibrary() const
	{
		return bUseAPILibaries && PipelineLibrary != nullptr;
	}

	bool UseCachedBlobs() const
	{
		// Use Cached Blobs if Pipeline Librarys aren't supported.
		//return bUseAPILibaries && !UsePipelineLibrary();
		return false; // Don't try to use cached blobs (for now).
	}

protected:

	void OnPSOCreated(FD3D12PipelineState* PipelineState, const FD3D12LowLevelGraphicsPipelineStateDesc& Desc) final override;
	void OnPSOCreated(FD3D12PipelineState* PipelineState, const FD3D12ComputePipelineStateDesc& Desc) final override;

	void AddToDiskCache(const FD3D12LowLevelGraphicsPipelineStateDesc& Desc, FD3D12PipelineState* PipelineState);
	void AddToDiskCache(const FD3D12ComputePipelineStateDesc& Desc, FD3D12PipelineState* PipelineState);

public:
#if !D3D12RHI_USE_HIGH_LEVEL_PSO_CACHE
	using FD3D12PipelineStateCacheBase::FindInLoadedCache;
	using FD3D12PipelineStateCacheBase::CreateAndAdd;
#endif
	void RebuildFromDiskCache();

	void Close();

	void Init(FString &GraphicsCacheFilename, FString &ComputeCacheFilename, FString &DriverBlobFilename);
	bool IsInErrorState() const;

	FD3D12PipelineStateCache(FD3D12Adapter* InParent);
	virtual ~FD3D12PipelineStateCache();
};
