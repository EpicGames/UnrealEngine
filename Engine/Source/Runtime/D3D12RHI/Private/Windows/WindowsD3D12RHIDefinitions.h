// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12ThirdParty.h"

#define UE_D3D12_SDK_VERSION (D3D12_SDK_VERSION)

#define D3D12RHI_DEFAULT_NUM_BACKBUFFER 3

// Windows uses the graphics command list interface for the copy queue.
typedef ID3D12GraphicsCommandList ID3D12CopyCommandList;

#define D3D12RHI_PLATFORM_COPY_COMMAND_LIST_TYPE D3D12_COMMAND_LIST_TYPE_COPY

#define FD3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER D3D12_DESCRIPTOR_HEAP_TYPE_SAMPLER

// Windows uses standard D3D12 query heaps for timestamps
#define D3D12RHI_PLATFORM_USES_TIMESTAMP_QUERIES 1

#define D3D12RHI_RESOURCE_FLAG_ALLOW_INDIRECT_BUFFER D3D12_RESOURCE_FLAG_NONE
#define D3D12RHI_HEAP_FLAG_ALLOW_INDIRECT_BUFFERS		D3D12_HEAP_FLAG_NONE

#define FD3D12_HEAP_FLAG_CREATE_NOT_ZEROED				D3D12_HEAP_FLAG_CREATE_NOT_ZEROED

#define D3D12RHI_NEEDS_VENDOR_EXTENSIONS     1
#define D3D12RHI_NEEDS_SHADER_FEATURE_CHECKS 1

#define USE_STATIC_ROOT_SIGNATURE			0
#define D3D12RHI_USE_CONSTANT_BUFFER_VIEWS	0
#define D3D12RHI_USE_DUMMY_BACKBUFFER		1
#define D3D12RHI_USE_SDR_BACKBUFFER			0
#define D3D12RHI_SUPPORTS_UAV_BACKBUFFER	0
#define D3D12RHI_USE_DXGI_COLOR_SPACE		1

// Only enable pipeline statistics if we've got the stats system enabled to display them
#define D3D12RHI_ENABLE_PIPELINE_STATISTICS (1 && STATS)

#define D3D12RHI_ALLOW_ENHANCED_BARRIERS 1
#define D3D12RHI_ALLOW_LEGACY_BARRIERS   1

#define D3D12RHI_HAS_LAYOUT_VIDEO_QUEUE_COMMON                                  (UE_D3D12_SDK_VERSION  < 618)
#define D3D12RHI_HAS_LAYOUT_DIRECT_QUEUE_GENERIC_READ_COMPUTE_QUEUE_ACCESSIBLE  (UE_D3D12_SDK_VERSION >= 618)
#define D3D12RHI_COMMON_AND_PRESENT_ARE_DISTINCT                                0

// Needed for XR
#define D3D12RHI_SUPPORTS_ALIASED_RHI_RESOURCES 1

#define USE_BUFFER_POOL_ALLOCATOR 1
#define USE_TEXTURE_POOL_ALLOCATOR 1
