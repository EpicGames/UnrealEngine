// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Must be included through D3D12Submission.h

#ifndef PLATFORM_COMPILER_IWYU
#include "D3D12Submission.h" // Circular include
#endif

#include "D3D12Tracing.h"

// Standard Windows implementation, used to mark the type as 'final'.
struct FD3D12Payload final : public FD3D12PayloadBase
{
	FD3D12Payload(FD3D12Queue& Queue)
		: FD3D12PayloadBase(Queue)
	{
		IF_RENDER_TRACING_ENABLED(RenderTracing::D3D12::PayloadCreated(this, Queue));
	}
};
