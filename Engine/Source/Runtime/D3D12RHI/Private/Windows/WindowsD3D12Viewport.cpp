// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12Viewport.cpp: D3D viewport RHI implementation.
=============================================================================*/

#include "D3D12RHIPrivate.h"
#include "Features/IModularFeatures.h"
#include "HDRHelper.h"
#include "RenderingThread.h"
#include "RHIUtilities.h"
#include "HAL/ThreadHeartBeat.h"
#include "Windows/IDXGISwapchainProvider.h"

#include "Windows/AllowWindowsPlatformTypes.h"

extern FD3D12Texture* GetSwapChainSurface(FD3D12Device* Parent, EPixelFormat PixelFormat, uint32 SizeX, uint32 SizeY, IDXGISwapChain* SwapChain, uint32 BackBufferIndex, TRefCountPtr<ID3D12Resource> BackBufferResourceOverride);

static int32 GD3D12UseAllowTearing = 1;
static FAutoConsoleVariableRef CVarD3DUseAllowTearing(
	TEXT("r.D3D12.UseAllowTearing"),
	GD3D12UseAllowTearing,
	TEXT("Enable new dxgi flip mode with d3d12"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

FD3D12Viewport::FD3D12Viewport(class FD3D12Adapter* InParent, HWND InWindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InPreferredPixelFormat, uint32 InNumBackBuffers)
	: FD3D12AdapterChild(InParent)
	, WindowHandle(InWindowHandle)
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, PixelFormat(InPreferredPixelFormat)
	, bIsFullscreen(bInIsFullscreen)
	, bNeedSwapChain(!FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	, NumBackBuffers(InNumBackBuffers)
{
	check(NumBackBuffers > 0);
	BackBuffers.SetNum(NumBackBuffers);
	check(IsInGameThread());
	GetParentAdapter()->GetViewports().Add(this);
}

//Init for a Viewport that will do the presenting
void FD3D12Viewport::Init()
{
	FD3D12Adapter* Adapter = GetParentAdapter();

	// Make sure we flush any pending GPU work that uses existing swapchains before we start using a potentially recycled HWND.
	FlushRenderingCommands();
	Adapter->BlockUntilIdle();

	IDXGIFactory2* Factory2 = Adapter->GetDXGIFactory2();

	TArray<IDXGISwapchainProvider*> DXGISwapchainProviderModules = IModularFeatures::Get().GetModularFeatureImplementations<IDXGISwapchainProvider>(IDXGISwapchainProvider::GetModularFeatureName());
	IDXGISwapchainProvider* DXGISwapchainProvider = nullptr;
	for (IDXGISwapchainProvider* ProviderModule : DXGISwapchainProviderModules)
	{
		if (ProviderModule->SupportsRHI(ERHIInterfaceType::D3D12))
		{
			DXGISwapchainProvider = ProviderModule;
			break;
		}
	}

	if (DXGISwapchainProvider)
	{
		static bool bCustomSwapchainLogged = false;
		if (!bCustomSwapchainLogged)
		{
			UE_LOGF(LogD3D12RHI, Log, "Found a custom swapchain provider: '%ls'.", DXGISwapchainProvider->GetProviderName());
			bCustomSwapchainLogged = true;
		}
	}

	bAllowTearing = false;
	if (GD3D12UseAllowTearing)
	{
		if (IDXGIFactory5* Factory5 = Adapter->GetDXGIFactory5())
		{
			BOOL AllowTearing = FALSE;
			Factory5->CheckFeatureSupport(DXGI_FEATURE_PRESENT_ALLOW_TEARING, &AllowTearing, sizeof(AllowTearing));
			bAllowTearing = (AllowTearing != FALSE);
		}
	}

	bool bStereoMode = false;
	if (FD3D12DynamicRHI::GetD3DRHI()->IsQuadBufferStereoEnabled())
	{
		if (Factory2->IsWindowedStereoEnabled())
		{
			bStereoMode = true;
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Log, "FD3D12Viewport::FD3D12Viewport was not able to create stereo SwapChain; Please enable stereo in driver settings.");
			FD3D12DynamicRHI::GetD3DRHI()->DisableQuadBufferStereo();
		}
	}

	InitializeBackBufferArrays();

	// Create the swapchain.
	if (bNeedSwapChain)
	{
		// The command queue used here is irrelevant in regard to multi - GPU as it gets overriden in the Resize
		ID3D12CommandQueue* CommandQueue = Adapter->GetDevice(0)->GetQueue(ED3D12QueueType::Direct).D3DCommandQueue;

		UINT SwapChainFlags = DXGI_SWAP_CHAIN_FLAG_ALLOW_MODE_SWITCH;
		if (bAllowTearing)
		{
			SwapChainFlags |= DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING;
		}

		DXGI_SWAP_CHAIN_DESC1 SwapChainDesc1{};

		SwapChainDesc1.Width       = SizeX;
		SwapChainDesc1.Height      = SizeY;
		SwapChainDesc1.Format      = UE::DXGIUtilities::GetSwapChainFormat(PixelFormat);
		SwapChainDesc1.Stereo      = bStereoMode ? TRUE : FALSE;
		SwapChainDesc1.SampleDesc.Count = 1;
		SwapChainDesc1.SampleDesc.Quality = 0;
		SwapChainDesc1.BufferUsage = DXGI_USAGE_RENDER_TARGET_OUTPUT | DXGI_USAGE_SHADER_INPUT;
		SwapChainDesc1.BufferCount = NumBackBuffers;
		SwapChainDesc1.Scaling     = DXGI_SCALING_NONE;
		SwapChainDesc1.SwapEffect  = DXGI_SWAP_EFFECT_FLIP_DISCARD;
		SwapChainDesc1.AlphaMode   = DXGI_ALPHA_MODE_UNSPECIFIED;
		SwapChainDesc1.Flags       = SwapChainFlags;

		DXGI_SWAP_CHAIN_FULLSCREEN_DESC FullscreenDesc{};
		FullscreenDesc.RefreshRate.Numerator   = 0;
		FullscreenDesc.RefreshRate.Denominator = 0;
		FullscreenDesc.ScanlineOrdering        = DXGI_MODE_SCANLINE_ORDER_UNSPECIFIED;
		FullscreenDesc.Scaling                 = DXGI_MODE_SCALING_UNSPECIFIED;
		FullscreenDesc.Windowed                = !bIsFullscreen;

		HRESULT hr = DXGISwapchainProvider ?
			DXGISwapchainProvider->CreateSwapChainForHwnd(Factory2, CommandQueue, WindowHandle, &SwapChainDesc1, &FullscreenDesc, nullptr, SwapChain1.GetInitReference()) :
			Factory2->CreateSwapChainForHwnd(CommandQueue, WindowHandle, &SwapChainDesc1, &FullscreenDesc, nullptr, SwapChain1.GetInitReference());

		if (FAILED(hr))
		{
			UE_LOGF(LogD3D12RHI, Warning, "Failed to create swapchain with the following parameters:");
			UE_LOGF(LogD3D12RHI, Warning, "\tWidth: %d Height: %d DXGI format: %d", SwapChainDesc1.Width, SwapChainDesc1.Height, SwapChainDesc1.Format);
			UE_LOGF(LogD3D12RHI, Warning, "\tBack buffer count: %d", SwapChainDesc1.BufferCount);
			UE_LOGF(LogD3D12RHI, Warning, "\tWindows handle: 0x%llx (IsWindow: %ls)", WindowHandle, IsWindow(WindowHandle) ? TEXT("true") : TEXT("false"));
			UE_LOGF(LogD3D12RHI, Warning, "\tFullscreen: %ls", bIsFullscreen ? TEXT("true") : TEXT("false"));
			UE_LOGF(LogD3D12RHI, Warning, "\tSwapchain flags: 0x%08x", SwapChainDesc1.Flags);
			UE_LOGF(LogD3D12RHI, Warning, "\tCustom swapchain provider: %ls", DXGISwapchainProvider ? DXGISwapchainProvider->GetProviderName() : TEXT("none"));

			// DXGI does not support CreateSwapChainForHwnd on WS_EX_LAYERED windows; DWM owns the composition path for layered windows.
			// The call usually succeeds quietly on most driver/OS combos but DWM can reject it with E_ACCESSDENIED under stress (e.g. during a recursive FlushRenderingCommands triggered by another subsystem's teardown).
			// In that case gracefully degrade rather than fatal: leave SwapChain1 null and clear bNeedSwapChain so the rest of Init (and subsequent Resize / Present) falls through the existing null-swapchain code paths used by pixel streaming windowless mode (GetSwapChainSurface creates CPU-side back buffers when SwapChain is null, Present is guarded by if (SwapChain1)).
			const bool bIsLayeredWindow = IsWindow(WindowHandle) &&
				(GetWindowLongPtr(WindowHandle, GWL_EXSTYLE) & WS_EX_LAYERED) != 0;
			if (hr == E_ACCESSDENIED && bIsLayeredWindow)
			{
				UE_LOGF(LogD3D12RHI, Warning, "Window 0x%llx has WS_EX_LAYERED. DXGI does not support swapchains on layered windows; continuing without a swapchain for this viewport.", WindowHandle);
				bNeedSwapChain = false;
			}
			else
			{
				VERIFYD3D12RESULT(hr);
			}
		}

		if (SwapChain1)
		{
			SwapChain1->QueryInterface(IID_PPV_ARGS(SwapChain2.GetInitReference()));
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 3
			SwapChain1->QueryInterface(IID_PPV_ARGS(SwapChain3.GetInitReference()));
#endif
#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 4
			SwapChain1->QueryInterface(IID_PPV_ARGS(SwapChain4.GetInitReference()));
#endif

#if UE_INPUT_LATENCY_TRACKING
			FRHIFrameFlipStatTrackingRunnable::Create(SwapChain1);
#endif

		}
	}

	{
		// Don't make the windows association call and release back buffer at the same time (see notes on critical section)
		FScopeLock Lock(&DXGIBackBufferLock);

		// Set the DXGI message hook to not change the window behind our back.
		Factory2->MakeWindowAssociation(WindowHandle, DXGI_MWA_NO_WINDOW_CHANGES);
	}

	// Resize to setup mGPU correctly.
	Resize(SizeX, SizeY, bIsFullscreen, PixelFormat);

	// Tell the window to redraw when they can.
	// @todo: For Slate viewports, it doesn't make sense to post WM_PAINT messages (we swallow those.)
	::PostMessageW(WindowHandle, WM_PAINT, 0, 0);
}

void FD3D12Viewport::FinalDestroyInternal()
{
#if UE_INPUT_LATENCY_TRACKING
	if (SwapChain1.IsValid())
	{
		FRHIFrameFlipStatTrackingRunnable::Release();
	}
#endif
}

void FD3D12Viewport::ClearPresentQueue()
{
}

void FD3D12Viewport::ConditionalResetSwapChain(bool bIgnoreFocus)
{
	if (!bIsValid)
	{
		if (bFullscreenLost)
		{
			FlushRenderingCommands();
			bFullscreenLost = false;
			Resize(SizeX, SizeY, false, PixelFormat);
		}
		else
		{
			// Check if the viewport's window is focused before resetting the swap chain's fullscreen state.
			HWND FocusWindow = ::GetFocus();
			const bool bIsFocused = FocusWindow == WindowHandle;
			const bool bIsIconic = !!::IsIconic(WindowHandle);
			if (bIgnoreFocus || (bIsFocused && !bIsIconic))
			{
				FlushRenderingCommands();

				HRESULT Result = SwapChain1->SetFullscreenState(bIsFullscreen, nullptr);
				if (SUCCEEDED(Result))
				{
					bIsValid = true;
				}
				else if (Result != DXGI_ERROR_NOT_CURRENTLY_AVAILABLE && Result != DXGI_STATUS_MODE_CHANGE_IN_PROGRESS)
				{
					const TCHAR* Name = nullptr;
					switch (Result)
					{
#define CASE_ERROR_NAME(x)	case x: Name = TEXT(#x); break
						EMBED_DXGI_ERROR_LIST(CASE_ERROR_NAME, ;)
#undef CASE_ERROR_NAME
					default:
						Name = TEXT("unknown error status");
						break;
					}
					UE_LOGF(LogD3D12RHI, Error, "IDXGISwapChain::SetFullscreenState returned 0x%08x, %ls.", Result, Name);

					if (bIsFullscreen)
					{
						// Something went wrong, attempt to proceed in windowed mode.
						Result = SwapChain1->SetFullscreenState(FALSE, nullptr);
						if (SUCCEEDED(Result))
						{
							bIsValid = true;
							bIsFullscreen = false;
						}
					}
				}
			}
		}
	}
}

#if D3D12RHI_USE_DUMMY_BACKBUFFER
/**
* Create the dummy back buffer textures - They don't have actual D3D resource but are used to always reference the current back buffer index on the RHI thread
*/
FD3D12Texture* FD3D12Viewport::CreateDummyBackBufferTextures(FD3D12Adapter* InAdapter, EPixelFormat InPixelFormat, uint32 InSizeX, uint32 InSizeY)
{
	// active stereoscopy uses RHI resources with 2DArray type
	bool const bQuadBufferStereo = FD3D12DynamicRHI::GetD3DRHI()->IsQuadBufferStereoEnabled();
	FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create(TEXT("BackBufferReference"),
			bQuadBufferStereo ? ETextureDimension::Texture2DArray : ETextureDimension::Texture2D)
		.SetArraySize(bQuadBufferStereo ? 2 : 1)
		.SetExtent(FIntPoint(InSizeX, InSizeY))
		.SetFormat(InPixelFormat)
		.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Presentable | ETextureCreateFlags::ResolveTargetable)
		.SetInitialState(ERHIAccess::Present);

	FD3D12Texture* Result = InAdapter->CreateLinkedObject<FD3D12Texture>(FRHIGPUMask::All(), [this, &CreateDesc](FD3D12Device* Device, FD3D12Texture* FirstLinkedObject)
		{
			return new FD3D12BackBufferReferenceTexture2D(CreateDesc, this, Device);
		});
	return Result;
}
#endif

void FD3D12Viewport::ResizeInternal()
{
	FD3D12Adapter* Adapter = GetParentAdapter();

	InitializeBackBufferArrays();

	UINT SwapChainFlags = DXGI_SWAP_CHAIN_FLAG_ALLOW_MODE_SWITCH;
	if (bAllowTearing)
	{
		SwapChainFlags |= DXGI_SWAP_CHAIN_FLAG_ALLOW_TEARING;
	}

#if WITH_MGPU
	if (GNumExplicitGPUsForRendering > 1)
	{
		TArray<ID3D12CommandQueue*> CommandQueues;
		TArray<uint32> NodeMasks;

		for (uint32 Index = 0; Index < NumBackBuffers; Index++)
		{
			// When BackbufferMultiGPUBinding == INDEX_NONE, cycle through each GPU.
			BackBuffers[Index].GPUIndex = BackbufferMultiGPUBinding == INDEX_NONE ? (Index % GNumExplicitGPUsForRendering) : BackbufferMultiGPUBinding;
		}

		// Select the GPU for each element in the swapchain 
		for (uint32 Index = 0; Index < NumBackBuffers; Index++)
		{
			FD3D12Device* Device = Adapter->GetDevice(BackBuffers[Index].GPUIndex);

			CommandQueues.Add(Device->GetQueue(ED3D12QueueType::Direct).D3DCommandQueue);
			NodeMasks.Add(Device->GetGPUMask().GetNative());
		}

		if (SwapChain3)
		{
			VERIFYD3D12RESULT_EX(SwapChain3->ResizeBuffers1(NumBackBuffers, SizeX, SizeY, UE::DXGIUtilities::GetSwapChainFormat(PixelFormat), SwapChainFlags, NodeMasks.GetData(), (IUnknown**)CommandQueues.GetData()), Adapter->GetD3DDevice());
		}

		for (uint32 Index = 0; Index < NumBackBuffers; Index++)
		{
			FD3D12Device* Device = Adapter->GetDevice(BackBuffers[Index].GPUIndex);

			check(BackBuffers[Index].Texture.GetReference() == nullptr);
			BackBuffers[Index].Texture = GetSwapChainSurface(Device, PixelFormat, SizeX, SizeY, SwapChain1, Index, nullptr);
		}
	}
	else
#endif // WITH_MGPU
	{
		if (SwapChain1)
		{
			VERIFYD3D12RESULT_LAMBDA(SwapChain1->ResizeBuffers(NumBackBuffers, SizeX, SizeY, UE::DXGIUtilities::GetSwapChainFormat(PixelFormat), SwapChainFlags), Adapter->GetD3DDevice(), [this]
			{
				return GetStateString();
			});
		}

		FD3D12Device* Device = Adapter->GetDevice(0);
		for (uint32 i = 0; i < NumBackBuffers; ++i)
		{
			check(BackBuffers[i].Texture.GetReference() == nullptr);
			BackBuffers[i].Texture = GetSwapChainSurface(Device, PixelFormat, SizeX, SizeY, SwapChain1, i, nullptr);
		}
	}

	uint32 CurrentBackBufferIndex = 0;

#if DXGI_MAX_SWAPCHAIN_INTERFACE >= 3
	if (SwapChain3)
	{
		CurrentBackBufferIndex = SwapChain3->GetCurrentBackBufferIndex();
	}
#endif

	SetBackBufferIndex_RHIThread(CurrentBackBufferIndex);

#if D3D12RHI_USE_DUMMY_BACKBUFFER
	// Create dummy back buffer which always reference to the actual RHI thread back buffer - can't be bound directly to D3D12
	DummyBackBuffer_RenderThread = CreateDummyBackBufferTextures(Adapter, PixelFormat, SizeX, SizeY);
#else
	SetBackBufferIndex_RenderThread(CurrentBackBufferIndex);
#endif
}

HRESULT FD3D12Viewport::PresentInternal(int32 SyncInterval)
{
	UINT Flags = 0;

	if(!SyncInterval && !bIsFullscreen && bAllowTearing)
	{
		Flags |= DXGI_PRESENT_ALLOW_TEARING;
	}

	FThreadHeartBeat::Get().PresentFrame();

	if (SwapChain1)
	{
		// Ignore time spent waiting in Present. This function blocks based on GPU progress and space in the swap chain.
		FRenderThreadIdleScope Scope(ERenderThreadIdleTypes::WaitingForGPUPresent);

		HRESULT PresentHR;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(D3D12_Present);
			PresentHR = SwapChain1->Present(SyncInterval, Flags);
		}

		UINT PresentID;
		if (SUCCEEDED(SwapChain1->GetLastPresentCount(&PresentID)))
		{
			// Assuming Next PresentID == Current PresentID + 1 , this might be not true in some cases
			GRHIPresentCounter = PresentID + 1;
		}
		else
		{
			GRHIPresentCounter++;
		}

		return PresentHR;
	}

	return S_OK;
}

void FD3D12Viewport::EnableHDR()
{
	if ( GRHISupportsHDROutput && IsHDREnabled() )
	{
		// Ensure we have the correct color space set.
		EnsureColorSpace(DisplayColorGamut, DisplayOutputFormat);
	}
}

void FD3D12Viewport::ShutdownHDR()
{
	// Make sure to set the appropriate color space even if GRHISupportsHDROutput is false because we 
	// might have toggled HDR on and off in the windows settings
	if (PixelFormat == PF_FloatRGBA)
	{
		// Per DXGI spec, float-valued swapchains must contain linear data.
		// The swapchain color space must therefore be G10 instead of G22.
		EnsureColorSpace(DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709);
	}
	else
	{
		// Ensure we have the correct color space set.
		EnsureColorSpace(EDisplayColorGamut::sRGB_D65, EDisplayOutputFormat::SDR_sRGB);
	}
}

bool FD3D12Viewport::CurrentOutputSupportsHDR() const
{
	// Default to no support.
	bool bOutputSupportsHDR = false;

	if (SwapChain4.GetReference())
	{
		// Output information is cached on the DXGI Factory. If it is stale we need to create
		// a new factory which will re-enumerate the displays.
		FD3D12Adapter* Adapter = GetParentAdapter();
		IDXGIFactory2* DxgiFactory2 = Adapter->GetDXGIFactory2();
		if (DxgiFactory2)
		{
			if (!DxgiFactory2->IsCurrent())
			{
				Adapter->CreateDXGIFactory(false);
			}

			check(Adapter->GetDXGIFactory2()->IsCurrent());

			// Get information about the display we are presenting to.
			TRefCountPtr<IDXGIOutput> Output;
			VERIFYD3D12RESULT(SwapChain4->GetContainingOutput(Output.GetInitReference()));

			TRefCountPtr<IDXGIOutput6> Output6;
			if (SUCCEEDED(Output->QueryInterface(IID_PPV_ARGS(Output6.GetInitReference()))))
			{
				DXGI_OUTPUT_DESC1 OutputDesc;
				VERIFYD3D12RESULT(Output6->GetDesc1(&OutputDesc));

				// Check for HDR support on the display.
				bOutputSupportsHDR = (OutputDesc.ColorSpace == DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020);
			}
		}
	}

	return bOutputSupportsHDR;
}

static const FString GetDXGIColorSpaceString(DXGI_COLOR_SPACE_TYPE ColorSpace)
{
	switch (ColorSpace)
	{
	case DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709:
		return TEXT("RGB_FULL_G22_NONE_P709");
	case DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709:
		return TEXT("RGB_FULL_G10_NONE_P709");
	case DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020:
		return TEXT("RGB_FULL_G2084_NONE_P2020");
	case DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P2020:
		return TEXT("RGB_FULL_G22_NONE_P2020");
	default:
		break;
	}

	return FString::FromInt(ColorSpace);
};

void FD3D12Viewport::EnsureColorSpace(DXGI_COLOR_SPACE_TYPE NewColorSpace)
{
	if (!SwapChain4.GetReference())
	{
		return;
	}

	if (ColorSpace != NewColorSpace)
	{
		uint32 ColorSpaceSupport = 0;
		HRESULT hr = SwapChain4->CheckColorSpaceSupport(NewColorSpace, &ColorSpaceSupport);
		FString NewColorSpaceName = GetDXGIColorSpaceString(NewColorSpace);
		if (SUCCEEDED(hr) && ((ColorSpaceSupport & DXGI_SWAP_CHAIN_COLOR_SPACE_SUPPORT_FLAG_PRESENT) != 0))
		{
			VERIFYD3D12RESULT(SwapChain4->SetColorSpace1(NewColorSpace));
			UE_LOGF(LogD3D12RHI, Verbose, "Setting color space on swap chain (%p): %ls", SwapChain4.GetReference(), *NewColorSpaceName);
			ColorSpace = NewColorSpace;
		}
		else
		{
			UE_LOGF(LogD3D12RHI, Error, "Warning: unable to set color space %ls to the swapchain: verify EDisplayOutputFormat / swapchain format", *NewColorSpaceName);
		}
	}
}

void FD3D12Viewport::EnsureColorSpace(EDisplayColorGamut DisplayGamut, EDisplayOutputFormat OutputDevice)
{
	DXGI_COLOR_SPACE_TYPE NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709;	// sRGB;
	const bool bPrimaries2020 = (DisplayGamut == EDisplayColorGamut::Rec2020_D65);

	// See console variable r.HDR.Display.OutputDevice.
	switch (OutputDevice)
	{
		// Gamma 2.2
	case EDisplayOutputFormat::SDR_sRGB:
	case EDisplayOutputFormat::SDR_Rec709:
		NewColorSpace = bPrimaries2020 ? DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P2020 : DXGI_COLOR_SPACE_RGB_FULL_G22_NONE_P709;
		break;

		// Gamma ST.2084
	case EDisplayOutputFormat::HDR_ACES_1000nit_ST2084:
	case EDisplayOutputFormat::HDR_ACES_2000nit_ST2084:
		NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G2084_NONE_P2020;
		break;

		// Gamma 1.0 (Linear)
	case EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB:
	case EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB:
		// Linear. Still supports expanded color space with values >1.0f and <0.0f.
		// The actual range is determined by the pixel format (e.g. a UNORM format can only ever have 0-1).
		NewColorSpace = DXGI_COLOR_SPACE_RGB_FULL_G10_NONE_P709;
		break;
	}

	EnsureColorSpace(NewColorSpace);
}

void FD3D12Viewport::OnResumeRendering()
{}


void FD3D12Viewport::OnSuspendRendering()
{}

bool FD3D12Viewport::IsPresentAllowed()
{
	return !FD3D12DynamicRHI::GetD3DRHI()->RHIIsRenderingSuspended();
}

void FD3D12DynamicRHI::RHIGetDisplaysInformation(FDisplayInformationArray& OutDisplayInformation)
{
	OutDisplayInformation.Append(DisplayList);
}

#include "Windows/HideWindowsPlatformTypes.h"
