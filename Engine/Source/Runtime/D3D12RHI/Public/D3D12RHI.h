// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12RHI.h: Public D3D RHI definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"

// Note: We do not need to worry about these counts exceeding resource binding tiers 1 & 2 due to 2 reasons:
// 1) On PC the actual counts used to build the root signature are derived from the shader and the shader compiler
// profile required for a desired feature level will enforce the proper limits.
// 2) All platforms currently using the global root signature support resource binding tier 3, these counts are well within the limits
// of tier 3.
// More info here: https://learn.microsoft.com/en-us/windows/win32/direct3d12/hardware-support, https://en.wikipedia.org/wiki/Feature_levels_in_Direct3D
#define MAX_SRVS		64 // Cannot be set higher than 64 due to 64b Cache.BoundMask logic
#define MAX_SAMPLERS	32
#define MAX_UAVS		16
#define MAX_CBS			16

#if !defined(WITH_PIX_EVENT_RUNTIME)
	#define WITH_PIX_EVENT_RUNTIME 0
#endif

#if PLATFORM_WINDOWS
	#define D3D12RHI_PLATFORM_HAS_UNIFIED_MEMORY	0
	#define D3D12RHI_PLATFORM_LOW_LEVEL_RES_TRACK	0
	#define ENABLE_RESIDENCY_MANAGEMENT				1
	#define PIPELINE_STATE_FILE_LOCATION			FPaths::ProjectSavedDir()
	#define USE_PIX									WITH_PIX_EVENT_RUNTIME
	#define D3D12RHI_SUPPORTS_WIN_PIX				USE_PIX

	#define FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT		D3D12_TEXTURE_DATA_PITCH_ALIGNMENT
#else
	#include "D3D12RHIPlatformPublic.h"
#endif
