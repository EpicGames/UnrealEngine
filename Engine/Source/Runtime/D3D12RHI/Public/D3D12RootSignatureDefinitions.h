// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12RHI.h"
#include "D3D12RayTracingResources.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/StringBuilder.h"
#include "RayTracingBuiltInResources.h"
#include "RHIDefinitions.h"
#include "RHIShaderBindingLayout.h"

#if !defined(D3D12_MAJOR_VERSION)
	#include "D3D12ThirdParty.h"
#endif

namespace D3D12ShaderUtils
{
	namespace StaticRootSignatureConstants
	{
		// Assume descriptors are volatile because we don't initialize all the descriptors in a table, just the ones used by the current shaders.
		constexpr D3D12_DESCRIPTOR_RANGE_FLAGS SRVDescriptorRangeFlags = D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;
		constexpr D3D12_DESCRIPTOR_RANGE_FLAGS CBVDescriptorRangeFlags = D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;
		constexpr D3D12_DESCRIPTOR_RANGE_FLAGS UAVDescriptorRangeFlags = D3D12_DESCRIPTOR_RANGE_FLAG_DATA_VOLATILE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;
		constexpr D3D12_DESCRIPTOR_RANGE_FLAGS SamplerDescriptorRangeFlags = D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE;

		// We always set the data in an upload heap before calling Set*RootConstantBufferView.
		constexpr D3D12_ROOT_DESCRIPTOR_FLAGS CBVRootDescriptorFlags = D3D12_ROOT_DESCRIPTOR_FLAG_DATA_STATIC;
	}

	enum class ERootSignatureRangeType
	{
		CBV,
		SRV,
		UAV,
		Sampler,
	};

	enum class ERootSignatureVisibility
	{
		Vertex,
		Pixel,
		Geometry,
		Mesh,
		Amplification,
		All
	};

	inline D3D12_DESCRIPTOR_RANGE_TYPE GetD3D12DescriptorRangeType(ERootSignatureRangeType Type)
	{
		switch (Type)
		{
		case ERootSignatureRangeType::SRV:
			return D3D12_DESCRIPTOR_RANGE_TYPE_SRV;
		case ERootSignatureRangeType::UAV:
			return D3D12_DESCRIPTOR_RANGE_TYPE_UAV;
		case ERootSignatureRangeType::Sampler:
			return D3D12_DESCRIPTOR_RANGE_TYPE_SAMPLER;
		default:
			return D3D12_DESCRIPTOR_RANGE_TYPE_CBV;
		}
	}

	inline D3D12_DESCRIPTOR_RANGE_FLAGS GetD3D12DescriptorRangeFlags(ERootSignatureRangeType Type)
	{
		switch (Type)
		{
		case ERootSignatureRangeType::SRV:
			return D3D12ShaderUtils::StaticRootSignatureConstants::SRVDescriptorRangeFlags;
		case ERootSignatureRangeType::CBV:
			return D3D12ShaderUtils::StaticRootSignatureConstants::CBVDescriptorRangeFlags;
		case ERootSignatureRangeType::UAV:
			return D3D12ShaderUtils::StaticRootSignatureConstants::UAVDescriptorRangeFlags;
		case ERootSignatureRangeType::Sampler:
			return D3D12ShaderUtils::StaticRootSignatureConstants::SamplerDescriptorRangeFlags;
		default:
			return D3D12_DESCRIPTOR_RANGE_FLAG_NONE;
		}
	}

	inline D3D12_SHADER_VISIBILITY GetD3D12ShaderVisibility(ERootSignatureVisibility Visibility)
	{
		switch (Visibility)
		{
		case ERootSignatureVisibility::Vertex:
			return D3D12_SHADER_VISIBILITY_VERTEX;
		case ERootSignatureVisibility::Pixel:
			return D3D12_SHADER_VISIBILITY_PIXEL;
		case ERootSignatureVisibility::Geometry:
			return D3D12_SHADER_VISIBILITY_GEOMETRY;
#if !defined(D3D12RHI_TOOLS_MESH_SHADERS_UNSUPPORTED)
		case ERootSignatureVisibility::Mesh:
			return D3D12_SHADER_VISIBILITY_MESH;
		case ERootSignatureVisibility::Amplification:
			return D3D12_SHADER_VISIBILITY_AMPLIFICATION;
#endif
		default:
			return D3D12_SHADER_VISIBILITY_ALL;
		}
	}

	inline const TCHAR* GetVisibilityFlag(ERootSignatureVisibility Visibility)
	{
		switch (Visibility)
		{
		case ERootSignatureVisibility::Vertex:
			return TEXT("SHADER_VISIBILITY_VERTEX");
		case ERootSignatureVisibility::Geometry:
			return TEXT("SHADER_VISIBILITY_GEOMETRY");
		case ERootSignatureVisibility::Pixel:
			return TEXT("SHADER_VISIBILITY_PIXEL");
#if !defined(D3D12RHI_TOOLS_MESH_SHADERS_UNSUPPORTED)
		case ERootSignatureVisibility::Mesh:
			return TEXT("SHADER_VISIBILITY_MESH");
		case ERootSignatureVisibility::Amplification:
			return TEXT("SHADER_VISIBILITY_AMPLIFICATION");
#endif
		default:
			return TEXT("SHADER_VISIBILITY_ALL");
		}
	};

	inline const TCHAR* GetTypePrefix(ERootSignatureRangeType Type)
	{
		switch (Type)
		{
		case ERootSignatureRangeType::SRV:
			return TEXT("SRV(t");
		case ERootSignatureRangeType::UAV:
			return TEXT("UAV(u");
		case ERootSignatureRangeType::Sampler:
			return TEXT("Sampler(s");
		default:
			return TEXT("CBV(b");
		}
	}

	inline const TCHAR* GetFlagName(D3D12_ROOT_SIGNATURE_FLAGS Flag)
	{
		switch (Flag)
		{
		case D3D12_ROOT_SIGNATURE_FLAG_ALLOW_INPUT_ASSEMBLER_INPUT_LAYOUT:
			return TEXT("ALLOW_INPUT_ASSEMBLER_INPUT_LAYOUT");
		case D3D12_ROOT_SIGNATURE_FLAG_DENY_VERTEX_SHADER_ROOT_ACCESS:
			return TEXT("DENY_VERTEX_SHADER_ROOT_ACCESS");
		case D3D12_ROOT_SIGNATURE_FLAG_DENY_GEOMETRY_SHADER_ROOT_ACCESS:
			return TEXT("DENY_GEOMETRY_SHADER_ROOT_ACCESS");
		case D3D12_ROOT_SIGNATURE_FLAG_DENY_PIXEL_SHADER_ROOT_ACCESS:
			return TEXT("DENY_PIXEL_SHADER_ROOT_ACCESS");
		case D3D12_ROOT_SIGNATURE_FLAG_ALLOW_STREAM_OUTPUT:
			return TEXT("ALLOW_STREAM_OUTPUT");

#if !defined(D3D12RHI_TOOLS_MESH_SHADERS_UNSUPPORTED)
		case D3D12_ROOT_SIGNATURE_FLAG_DENY_AMPLIFICATION_SHADER_ROOT_ACCESS:
			return TEXT("DENY_AMPLIFICATION_SHADER_ROOT_ACCESS");
		case D3D12_ROOT_SIGNATURE_FLAG_DENY_MESH_SHADER_ROOT_ACCESS:
			return TEXT("DENY_MESH_SHADER_ROOT_ACCESS");
#endif

		case D3D12_ROOT_SIGNATURE_FLAG_CBV_SRV_UAV_HEAP_DIRECTLY_INDEXED:
			return TEXT("CBV_SRV_UAV_HEAP_DIRECTLY_INDEXED");
		case D3D12_ROOT_SIGNATURE_FLAG_SAMPLER_HEAP_DIRECTLY_INDEXED:
			return TEXT("SAMPLER_HEAP_DIRECTLY_INDEXED");

		default:
			break;
		}

		return TEXT("");
	};

	// Simple base class to help write out a root signature (subclass to generate either to a binary struct or a #define)
	struct FRootSignatureCreator
	{
		EShaderBindingLayoutFlags Flags = EShaderBindingLayoutFlags::None;
		uint32 RegisterSpace = 0;

		virtual ~FRootSignatureCreator() = default;

		virtual void AddRootFlag(D3D12_ROOT_SIGNATURE_FLAGS Flag) = 0;
		virtual void AddConstantBufferViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) = 0;
		virtual void AddShaderResourceViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) = 0;
		virtual void AddUnorderedAccessViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) = 0;
		virtual void AddTable(ERootSignatureVisibility Visibility, ERootSignatureRangeType Type, int32 NumDescriptors, D3D12_DESCRIPTOR_RANGE_FLAGS FlagsOverride = D3D12_DESCRIPTOR_RANGE_FLAG_NONE) = 0;
		virtual void AddConstantsParameter(uint32 Num32BitValues, uint32 Register, uint32 Space) = 0;

		void SetFlags(EShaderBindingLayoutFlags InFlags)
		{
			Flags = InFlags;

			if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::InputAssembler))
			{
				AddRootFlag(D3D12_ROOT_SIGNATURE_FLAG_ALLOW_INPUT_ASSEMBLER_INPUT_LAYOUT);
			}

			if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::BindlessResources))
			{
				AddRootFlag(D3D12_ROOT_SIGNATURE_FLAG_CBV_SRV_UAV_HEAP_DIRECTLY_INDEXED);
			}

			if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::BindlessSamplers))
			{
				AddRootFlag(D3D12_ROOT_SIGNATURE_FLAG_SAMPLER_HEAP_DIRECTLY_INDEXED);
			}
		}

		bool HasFlags(EShaderBindingLayoutFlags InFlags) const
		{
			return EnumHasAllFlags(Flags, InFlags);
		}

		void SetRegisterSpace(uint32 InSpace)
		{
			RegisterSpace = InSpace;
		}

		uint32 GetRegisterSpace() const
		{
			return RegisterSpace;
		}

		inline bool ShouldSkipType(ERootSignatureRangeType Type) const
		{
			if (Type == ERootSignatureRangeType::SRV || Type == ERootSignatureRangeType::UAV)
			{
				return EnumHasAnyFlags(Flags, EShaderBindingLayoutFlags::BindlessResources);
			}

			if (Type == ERootSignatureRangeType::Sampler)
			{
				return EnumHasAnyFlags(Flags, EShaderBindingLayoutFlags::BindlessSamplers);
			}

			return false;
		}
	};

	struct FBinaryRootSignatureCreator final : public FRootSignatureCreator
	{
		D3D12_ROOT_SIGNATURE_FLAGS RootFlags = D3D12_ROOT_SIGNATURE_FLAG_NONE;
		TArray<CD3DX12_DESCRIPTOR_RANGE1> DescriptorRanges;
		TArray<CD3DX12_ROOT_PARAMETER1> Parameters;
		CD3DX12_VERSIONED_ROOT_SIGNATURE_DESC RootDesc;

		struct FPendingTable
		{
			ERootSignatureVisibility Visibility;
			ERootSignatureRangeType Type;
			int32 NumDescriptors;
			D3D12_DESCRIPTOR_RANGE_FLAGS FlagsOverride;
		};
		TArray<FPendingTable> PendingTables;

		void AddRootFlag(D3D12_ROOT_SIGNATURE_FLAGS RootFlag) override
		{
			RootFlags |= RootFlag;
		}

		void AddConstantBufferViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			CD3DX12_ROOT_PARAMETER1& Parameter = Parameters.AddZeroed_GetRef();
			Parameter.InitAsConstantBufferView(Register, Space, StaticRootSignatureConstants::CBVRootDescriptorFlags, GetD3D12ShaderVisibility(Visibility));
		}

		void AddShaderResourceViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			CD3DX12_ROOT_PARAMETER1& Parameter = Parameters.AddZeroed_GetRef();
			Parameter.InitAsShaderResourceView(Register, Space, D3D12_ROOT_DESCRIPTOR_FLAG_NONE, GetD3D12ShaderVisibility(Visibility));
		}

		void AddUnorderedAccessViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			CD3DX12_ROOT_PARAMETER1& Parameter = Parameters.AddZeroed_GetRef();
			Parameter.InitAsUnorderedAccessView(Register, Space, D3D12_ROOT_DESCRIPTOR_FLAG_NONE, GetD3D12ShaderVisibility(Visibility));
		}

		void AddConstantsParameter(uint32 Num32BitValues, uint32 Register, uint32 Space) override
		{
			CD3DX12_ROOT_PARAMETER1& Parameter = Parameters.AddZeroed_GetRef();
			Parameter.InitAsConstants(Num32BitValues, Register, Space);
		}

		void AddTable(ERootSignatureVisibility Visibility, ERootSignatureRangeType Type, int32 NumDescriptors, D3D12_DESCRIPTOR_RANGE_FLAGS FlagsOverride = D3D12_DESCRIPTOR_RANGE_FLAG_NONE) override
		{
			if (!ShouldSkipType(Type))
			{
				const FPendingTable NewTable{ Visibility, Type, NumDescriptors, FlagsOverride };
				PendingTables.Emplace(NewTable);
			}
		}

		const D3D12_VERSIONED_ROOT_SIGNATURE_DESC& Finalize()
		{
#if !defined(D3D12RHI_TOOLS_RAYTRACING_SHADERS_UNSUPPORTED)
			if (RootFlags & D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE)
			{
				// if set, D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE should be the only flag set
				RootFlags = D3D12_ROOT_SIGNATURE_FLAG_LOCAL_ROOT_SIGNATURE;
			}
#endif

			DescriptorRanges.SetNum(PendingTables.Num());

			for (int32 Index = 0; Index < PendingTables.Num(); Index++)
			{
				const FPendingTable& PendingTable = PendingTables[Index];

				const D3D12_DESCRIPTOR_RANGE_FLAGS RangeFlags = PendingTable.FlagsOverride ? PendingTable.FlagsOverride : GetD3D12DescriptorRangeFlags(PendingTable.Type);

				DescriptorRanges[Index].Init(GetD3D12DescriptorRangeType(PendingTable.Type), PendingTable.NumDescriptors, 0, RegisterSpace, RangeFlags);

				CD3DX12_ROOT_PARAMETER1& Parameter = Parameters.AddZeroed_GetRef();
				Parameter.InitAsDescriptorTable(1, &DescriptorRanges[Index], GetD3D12ShaderVisibility(PendingTable.Visibility));
			}

			RootDesc.Init_1_1(Parameters.Num(), Parameters.GetData(), 0, nullptr, RootFlags);

			return RootDesc;
		}
	};

	/* Root signature generator for DXC */
	struct FTextRootSignatureCreator final : public FRootSignatureCreator
	{
		void AddRootFlag(D3D12_ROOT_SIGNATURE_FLAGS InFlag) override
		{
			if (RootFlags.Len() > 0)
			{
				RootFlags += "|";
			}
			RootFlags += GetFlagName(InFlag);
		}

		void AddConstantBufferViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			FString Line = FString::Printf(TEXT("CBV(t%d, space=%d, visibility=%s))"), Register, Space, GetVisibilityFlag(Visibility));
			if (Table.Len() > 0)
			{
				Table += ",";
			}
			Table += Line;
		}

		void AddShaderResourceViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			FString Line = FString::Printf(TEXT("SRV(t%d, space=%d, visibility=%s))"), Register, Space, GetVisibilityFlag(Visibility));
			if (Table.Len() > 0)
			{
				Table += ",";
			}
			Table += Line;
		}

		void AddUnorderedAccessViewParameter(uint32 Register, uint32 Space, ERootSignatureVisibility Visibility = ERootSignatureVisibility::All) override
		{
			FString Line = FString::Printf(TEXT("UAV(u%d, space=%d, visibility=%s)"), Register, Space, GetVisibilityFlag(Visibility));
			if (Table.Len() > 0)
			{
				Table += ",";
			}
			Table += Line;
		}

		void AddConstantsParameter(uint32 Num32BitValues, uint32 Register, uint32 Space) override
		{
			FString Line = FString::Printf(TEXT("RootConstants(num32BitConstants=%d, b%d, space=%d)"), Num32BitValues, Register, Space);
			if (Table.Len() > 0)
			{
				Table += ",";
			}
			Table += Line;
		}

		void AddTable(ERootSignatureVisibility Visibility, ERootSignatureRangeType Type, int32 NumDescriptors, D3D12_DESCRIPTOR_RANGE_FLAGS FlagsOverride = D3D12_DESCRIPTOR_RANGE_FLAG_NONE) override
		{
			if (!ShouldSkipType(Type))
			{
				FString Line = FString::Printf(TEXT("DescriptorTable(visibility=%s, %s0, space=%d, numDescriptors=%d))"),
					GetVisibilityFlag(Visibility), GetTypePrefix(Type), RegisterSpace, NumDescriptors
				);
				if (Table.Len() > 0)
				{
					Table += ",";
				}
				Table += Line;
			}
		}

		FString GenerateString() const
		{
			FString String = FString::Printf(TEXT("\"RootFlags(%s),%s\""),
				RootFlags.Len() == 0 ? TEXT("0") : *RootFlags,
				*Table);
			return String;
		}

		FString RootFlags;
		FString Table;
	};

	inline void AddAllStandardTablesForVisibility(FRootSignatureCreator& Creator, ERootSignatureVisibility Visibility)
	{
		Creator.AddTable(Visibility, ERootSignatureRangeType::SRV, MAX_SRVS);
		Creator.AddTable(Visibility, ERootSignatureRangeType::CBV, MAX_CBS);
		Creator.AddTable(Visibility, ERootSignatureRangeType::Sampler, MAX_SAMPLERS);
	}

	// Fat/Static Gfx Root Signature
	inline void CreateGfxRootSignature(FRootSignatureCreator& Creator, EShaderBindingLayoutFlags InFlags)
	{
		// Ensure the creator starts in a clean state (in cases of creator reuse, etc.).
		Creator.SetFlags(InFlags | EShaderBindingLayoutFlags::InputAssembler);

		Creator.AddUnorderedAccessViewParameter(0, UE_HLSL_SPACE_DIAGNOSTIC);

		if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::RootConstants))
		{
			const uint32 NumConstants = 4u;
			Creator.AddConstantsParameter(NumConstants, 0u, UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS); // UERootConstants
		}

		// Tables must be added only after other parameters

		AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::Pixel);
		AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::Vertex);
		AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::Geometry);

#if !defined(D3D12RHI_TOOLS_MESH_SHADERS_UNSUPPORTED)
		if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::AllowMeshShaders))
		{
			AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::Mesh);
			AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::Amplification);
		}
#endif

		Creator.AddTable(ERootSignatureVisibility::Pixel, ERootSignatureRangeType::UAV, MAX_UAVS);
		Creator.AddTable(ERootSignatureVisibility::Vertex, ERootSignatureRangeType::UAV, MAX_UAVS);
	}

	// Fat/Static Compute Root Signature
	inline void CreateComputeRootSignature(FRootSignatureCreator& Creator, EShaderBindingLayoutFlags InFlags)
	{
		// Ensure the creator starts in a clean state (in cases of creator reuse, etc.).
		Creator.SetFlags(InFlags);

		Creator.AddUnorderedAccessViewParameter(0, UE_HLSL_SPACE_DIAGNOSTIC);

		if (EnumHasAnyFlags(InFlags, EShaderBindingLayoutFlags::RootConstants))
		{
			const uint32 NumConstants = 4u;
			Creator.AddConstantsParameter(NumConstants, 0u, UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS); // UERootConstants
		}

		// Tables must be added only after other parameters

		AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::All);
		Creator.AddTable(ERootSignatureVisibility::All, ERootSignatureRangeType::UAV, MAX_UAVS);
	}

#if !defined(D3D12RHI_TOOLS_RAYTRACING_SHADERS_UNSUPPORTED)
	inline void CreateRayTracingSignature(FRootSignatureCreator& Creator, bool bLocalRootSignature, D3D12_ROOT_SIGNATURE_FLAGS BaseRootFlags, EShaderBindingLayoutFlags InFlags)
	{
		Creator.SetFlags(InFlags);
		Creator.AddRootFlag(BaseRootFlags);
		Creator.SetRegisterSpace(bLocalRootSignature ? UE_HLSL_SPACE_RAY_TRACING_LOCAL : UE_HLSL_SPACE_RAY_TRACING_GLOBAL);

		if (bLocalRootSignature)
		{
			if (Creator.HasFlags(EShaderBindingLayoutFlags::BindlessResources))
			{
				// Num constants could be 2 smaller if non-bindless data is removed (24 bytes instead of 32)
				// (see notes in FD3D12RootSignatureDesc constructor)
				uint32 NumConstants = sizeof(FD3D12HitGroupSystemParameters) / sizeof(uint32);
				Creator.AddConstantsParameter(NumConstants, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
			}
			else
			{
				uint32 NumConstants = sizeof(FHitGroupSystemRootConstants) / sizeof(uint32);
				Creator.AddConstantsParameter(NumConstants, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);

				Creator.AddShaderResourceViewParameter(RAY_TRACING_SYSTEM_INDEXBUFFER_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
				Creator.AddShaderResourceViewParameter(RAY_TRACING_SYSTEM_VERTEXBUFFER_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
			}
		}
		else if (Creator.HasFlags(EShaderBindingLayoutFlags::ShaderBindingLayoutUsed))
		{
			for (uint32 Index = 0; Index < FRHIShaderBindingLayout::MaxUniformBufferEntries; ++Index)
			{
				Creator.AddConstantBufferViewParameter(Index, UE_HLSL_SPACE_STATIC_SHADER_BINDINGS);
			}
		}

		if (Creator.HasFlags(EShaderBindingLayoutFlags::BindlessResources))
		{
			for (uint32 Index = 0; Index < MAX_CBS; Index++)
			{
				Creator.AddConstantBufferViewParameter(Index, Creator.GetRegisterSpace());
			}
		}
		else
		{
			AddAllStandardTablesForVisibility(Creator, ERootSignatureVisibility::All);
		}

		Creator.AddTable(ERootSignatureVisibility::All, ERootSignatureRangeType::UAV, MAX_UAVS, D3D12_DESCRIPTOR_RANGE_FLAG_DATA_STATIC_WHILE_SET_AT_EXECUTE | D3D12_DESCRIPTOR_RANGE_FLAG_DESCRIPTORS_VOLATILE);
	}

	inline FString GenerateRayTracingSignatureString(bool bLocalRootSignature, D3D12_ROOT_SIGNATURE_FLAGS BaseRootFlags, EShaderBindingLayoutFlags InFlags)
	{
		FTextRootSignatureCreator Creator;
		CreateRayTracingSignature(Creator, bLocalRootSignature, BaseRootFlags, InFlags);
		return Creator.GenerateString();
	}
#endif //!defined(D3D12RHI_TOOLS_RAYTRACING_SHADERS_UNSUPPORTED)

	inline FString GenerateRootSignatureString(EShaderFrequency InFrequency, EShaderBindingLayoutFlags InFlags)
	{
		FTextRootSignatureCreator Creator;

		if (InFrequency < SF_NumGraphicsFrequencies)
		{
			D3D12ShaderUtils::CreateGfxRootSignature(Creator, InFlags);
		}
		else if (InFrequency == SF_Compute)
		{
			D3D12ShaderUtils::CreateComputeRootSignature(Creator, InFlags);
		}
		else
		{
			return TEXT("");
		}

		return Creator.GenerateString();
	}
}
