// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	D3D12ShaderResources.h: Public D3D12 RHI shader definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"

// Key used for determining whether shader code is packed or not.
const int32 PackedShaderKey = 'XSHA';

// Key indicating whether serialized ray tracing shader contains a DXIL library or a precompiled PSO.
const int32 RayTracingPrecompiledPSOKey = 'RTPS';
