// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "D3D12ThirdParty.h"
#include "RHI.h"
#include "Templates/Function.h"

struct FD3D12MinimalAdapterDesc
{
	DXGI_ADAPTER_DESC Desc{};
	uint32 NumDeviceNodes{};
};

enum class ED3D12RHIRunOnQueueType
{
	Graphics = 0,
	Copy,
};

struct ID3D12DynamicRHI : public FDynamicRHI
{
	virtual ERHIInterfaceType     GetInterfaceType() const override final { return ERHIInterfaceType::D3D12; }

	virtual TArray<FD3D12MinimalAdapterDesc> RHIGetAdapterDescs() const = 0;
	virtual bool                             RHIIsPixEnabled() const = 0;

	// Provided for backwards compatibility. Returns the 0th device, equivalent to RHIGetDevice(0), but also triggers an ensure if MGPU is enabled.
	virtual ID3D12Device*              RHIGetDevice_NoMGPU() const = 0;

	virtual ID3D12CommandQueue*        RHIGetCommandQueue() const = 0;
	virtual ID3D12Device*              RHIGetDevice(uint32 InIndex) const = 0;
	virtual uint32                     RHIGetDeviceNodeMask(uint32 InIndex) const = 0;
	virtual ID3D12GraphicsCommandList* RHIGetGraphicsCommandList(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex) const = 0;
	virtual DXGI_FORMAT                RHIGetSwapChainFormat(EPixelFormat InFormat) const = 0;

	virtual FTextureRHIRef        RHICreateTexture2DFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) = 0;
	virtual FTextureRHIRef        RHICreateTexture2DArrayFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) = 0;
	virtual FTextureRHIRef        RHICreateTextureCubeFromResource(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, ID3D12Resource* Resource) = 0;

	virtual ID3D12Resource*       RHIGetResource(FRHIBuffer* InBuffer) const = 0;
	virtual uint32                RHIGetResourceDeviceIndex(FRHIBuffer* InBuffer) const = 0;
	virtual int64                 RHIGetResourceMemorySize(FRHIBuffer* InBuffer) const = 0;
	virtual bool                  RHIIsResourcePlaced(FRHIBuffer* InBuffer) const = 0;
	virtual ID3D12Resource*       RHIGetResource(FRHITexture* InTexture) const = 0;
	virtual uint32                RHIGetResourceDeviceIndex(FRHITexture* InTexture) const = 0;
	virtual int64                 RHIGetResourceMemorySize(FRHITexture* InTexture) const = 0;
	virtual bool                  RHIIsResourcePlaced(FRHITexture* InTexture) const = 0;

	virtual D3D12_CPU_DESCRIPTOR_HANDLE RHIGetRenderTargetView(FRHITexture* InTexture, int32 InMipIndex = 0, int32 InArraySliceIndex = 0) const = 0;

	virtual void                  RHIFinishExternalComputeWork(FRHICommandListBase& ExecutingCmdList, uint32 InDeviceIndex, ID3D12GraphicsCommandList* InCommandList) = 0;
	
	UE_DEPRECATED				  (5.6, "RHITransitionResource without explicit Before and After states is deprecated. Please call RHICmdList.Transition() using ERHIAccess states to issue the transition, ERHIAccess::Unknown as Before state is supported.")
	virtual void                  RHITransitionResource(FRHICommandList& RHICmdList, FRHITexture* InTexture, D3D12_RESOURCE_STATES InState, uint32 InSubResource) = 0;

	virtual void                  RHISignalManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value) = 0;
	virtual void                  RHIWaitManualFence(FRHICommandList& RHICmdList, ID3D12Fence* Fence, uint64 Value, const TSharedPtr<std::atomic<bool>>& bHasSignalled) = 0;

	virtual void                  RHIVerifyResult(ID3D12Device* Device, HRESULT Result, const ANSICHAR* Code, const ANSICHAR* Filename, uint32 Line, FString Message = FString()) const = 0;

	virtual void				  RHIRunOnQueue(ED3D12RHIRunOnQueueType QueueType, TFunction<void(ID3D12CommandQueue*)>&& CodeToRun, bool bWaitForSubmission) = 0;

	virtual void				  RHIFlushResourceBarriers(FRHICommandListBase& RHICmdList, uint32 InGPUIndex) = 0;
	virtual void				  RHIUpdateResourceResidency(FRHICommandListBase& RHICmdList, uint32 InGPUIndex, FRHIResource* InResource) = 0;

	// Returns the RHI viewport, assuming only a single FRHIViewport instance exists. Otherwise, returns nullptr.
	virtual FRHIViewport* RHIGetSingletonViewport() const = 0;
};

inline bool IsRHID3D12()
{
	return GDynamicRHI != nullptr && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D12;
}

inline ID3D12DynamicRHI* GetID3D12DynamicRHI()
{
	check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D12);
	return GetDynamicRHI<ID3D12DynamicRHI>();
}

#if D3D12RHI_PLATFORM_HAS_CUSTOM_INTERFACE
	#include "ID3D12PlatformDynamicRHI.h"
#else
	using ID3D12PlatformDynamicRHI = ID3D12DynamicRHI;

	inline ID3D12PlatformDynamicRHI* GetID3D12PlatformDynamicRHI()
	{
		check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D12);
		return GetDynamicRHI<ID3D12PlatformDynamicRHI>();
	}
#endif
